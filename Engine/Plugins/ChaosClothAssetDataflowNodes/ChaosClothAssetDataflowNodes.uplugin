{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Chaos Cloth Asset Dataflow Nodes",
	"Description": "Dataflow node plugin required to edit a Cloth Asset.",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosClothAssetDataflowNodes",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64", "Mac", "Linux" ]
		}
	],
	"Plugins": [
		{
			"Name": "ChaosCloth",
			"Enabled": true
		},
		{
			"Name": "ChaosClothAsset",
			"Enabled": true
		},
		{
			"Name": "Dataflow",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "MeshResizing",
			"Enabled": true
		}
	]
}
