// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosClothAssetDataflowNodes : ModuleRules
{
	public ChaosClothAssetDataflowNodes(ReadOnlyTargetRules Target) : base(Target)
	{	
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"MeshResizingCore",
				// ... add other public dependencies that you statically link with here ...
			}
		);

		SetupModulePhysicsSupport(Target);
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimationCore",
				"AssetTools",
				"ChaosCaching",
				"ChaosCloth",
				"ChaosClothAsset",
				"ChaosClothAssetEngine",
				"ChaosClothAssetTools",
				"ClothingSystemRuntimeCommon",
				"CoreUObject",
				"DataflowCore",
				"DataflowEditor",
				"DataflowEngine",
				"DataflowNodes",
				"DetailCustomizations",
				"DynamicMesh",
				"Engine",
				"GeometryCore",
				"GeometryFramework",
				"InputCore",
				"MeshConversion",
				"MeshDescription",
				"MeshUtilitiesCommon",
				"ModelingOperatorsEditorOnly",	// TODO: Someday remove editor dependencies, see UE-206172
				"ModelingComponents",
				"ModelingOperators",
				"MeshConversionEngineTypes",
				"RenderCore",
				"SkeletalMeshDescription",
				"Slate",
				"SlateCore",
				"StaticMeshDescription",
				"UnrealEd",
				// ... add private dependencies that you statically link with here ...
			}
		);

		PrivateIncludePaths.Add(System.IO.Path.Combine(ModuleDirectory, "Internal"));
	}
}
