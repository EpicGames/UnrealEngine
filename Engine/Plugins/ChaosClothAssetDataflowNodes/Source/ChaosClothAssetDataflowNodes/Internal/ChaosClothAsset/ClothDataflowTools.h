// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector.h"
#include "Logging/LogMacros.h"
#include "Templates/SharedPointer.h"

#define UE_API CHAOSCLOTHASSETDATAFLOWNODES_API

struct FManagedArrayCollection;
struct FMeshBuildSettings;
struct FMeshDescription;
struct FDataflowNode;
class FSkeletalMeshLODModel;
class FString;
class IPropertyHandle;
struct FMeshResizingRBFInterpolationData;
struct FMeshResizingCustomRegion;

namespace UE::Dataflow
{
	class FContext;
}

namespace UE::Chaos::ClothAsset
{
	/**
	* Tools shared by cloth dataflow nodes
	*/
	struct FClothDataflowTools
	{
		static void AddRenderPatternFromSkeletalMeshSection(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FSkeletalMeshLODModel& SkeletalMeshModel, const int32 SectionIndex, const FSoftObjectPath& RenderMaterialPathName);

		static void AddSimPatternsFromSkeletalMeshSection(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FSkeletalMeshLODModel& SkeletalMeshModel, const int32 SectionIndex, const int32 UVChannelIndex, const FVector2f& UVScale, bool bImportNormals = false, TArray<int32>* OutSim2DToSourceVertex = nullptr);

		UE_API static void LogAndToastWarning(const FDataflowNode& DataflowNode, const FText& Headline, const FText& Details);

		/**
		 * Turn a string into a valid collection group or attribute name.
		 * The resulting name won't contains spaces and any other special characters as listed in
		 * INVALID_OBJECTNAME_CHARACTERS (currently "',/.:|&!~\n\r\t@#(){}[]=;^%$`).
		 * It will also have all leading underscore removed, as these names are reserved for internal use.
		 * @param InOutString The string to turn into a valid collection name.
		 * @return Whether the InOutString was already a valid collection name.
		 */
		static bool MakeCollectionName(FString& InOutString);

		static bool BuildSkeletalMeshModelFromMeshDescription(const FMeshDescription* const InMeshDescription, const FMeshBuildSettings& InBuildSettings, FSkeletalMeshLODModel& SkeletalMeshModel);

		/** Return the Dataflow node owning by this property, and cast it to the desired node type. */
		template<typename T = FDataflowNode>
		static T* GetPropertyOwnerDataflowNode(const TSharedPtr<IPropertyHandle>& PropertyHandle)
		{
			return static_cast<FDataflowNode*>(GetPropertyOwnerDataflowNode(PropertyHandle, T::StaticStruct()));
		}

		/** Simulation mesh cleanup tools. */
		struct FSimMeshCleanup
		{
			TArray<FIntVector3> TriangleToVertexIndex;
			TArray<FVector2f> RestPositions2D;
			TArray<FVector3f> DrapedPositions3D;
			TArray<TSet<int32>> OriginalTriangles;  // New to original face index lookup
			TArray<TSet<int32>> OriginalVertices;  // New to original vertex index lookup

			UE_API FSimMeshCleanup(
				const TArray<FIntVector3>& InTriangleToVertexIndex,
				const TArray<FVector2f>& InRestPositions2D,
				const TArray<FVector3f>& InDrapedPositions3D);

			UE_API bool RemoveDegenerateTriangles();
			UE_API bool RemoveDuplicateTriangles();
		};

		/** Returns the inverse mapping from a reduced set of original indices to the new indices. */
		template<typename T UE_REQUIRES(std::is_same_v<T, TArray<int32>> || std::is_same_v<T, TSet<int32>>)>
		UE_API static TArray<int32> GetOriginalToNewIndices(const TConstArrayView<T>& NewToOriginals, int32 NumOriginalIndices);

		UE_DEPRECATED(5.6, "Use FSimMeshCleanup instead.")
		UE_API static bool RemoveDegenerateTriangles(
			const TArray<FIntVector3>& TriangleToVertexIndex,
			const TArray<FVector2f>& RestPositions2D,
			const TArray<FVector3f>& DrapedPositions3D,
			TArray<FIntVector3>& OutTriangleToVertexIndex,
			TArray<FVector2f>& OutRestPositions2D,
			TArray<FVector3f>& OutDrapedPositions3D,
			TArray<int32>& OutIndices);  // Old to new vertices lookup

		UE_DEPRECATED(5.6, "Use FSimMeshCleanup instead.")
		static bool RemoveDuplicateTriangles(TArray<FIntVector3>& TriangleToVertexIndex)
		{
			FSimMeshCleanup SimMeshCleanup(TriangleToVertexIndex, TArray<FVector2f>(), TArray<FVector3f>());
			const bool bHasDuplicateTriangles = SimMeshCleanup.RemoveDuplicateTriangles();
			TriangleToVertexIndex = MoveTemp(SimMeshCleanup.TriangleToVertexIndex);
			return bHasDuplicateTriangles;
		}

		UE_API static bool RemoveDuplicateStitches(TArray<TArray<FIntVector2>>& SeamStitches);

		/**
		 * Set Group Resizing data from supplied sets and types.
		 */
		static void SetGroupResizingData(const TSharedRef<FManagedArrayCollection>& ClothCollection, const TConstArrayView<FName>& SetNames, const TConstArrayView<int32>& SetTypes);

		/**
		 * Generate FMeshResizingCustomRegion data from Sim Mesh Group Resizing data
		 */
		static void GenerateSimMeshResizingGroupData(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FMeshDescription& SourceMesh, TArray<FMeshResizingCustomRegion>& OutResizingGroupData);

		/**
		 * Generate FMeshResizingCustomRegion data from Render Mesh Group Resizing data
		 */
		static void GenerateRenderMeshResizingGroupData(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FMeshDescription& SourceMesh, TArray<FMeshResizingCustomRegion>& OutResizingGroupData);

		/**
		 * Apply Group Resizing to the sim mesh.
		 */
		static void ApplySimGroupResizing(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FMeshDescription& TargetMesh, const FMeshResizingRBFInterpolationData& InterpolationData, const TArray<FMeshResizingCustomRegion>& ResizingGroupData);

		/**
		 * Apply Group Resizing to the render mesh.
		 */
		static void ApplyRenderGroupResizing(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FMeshDescription& TargetMesh, const FMeshResizingRBFInterpolationData& InterpolationData, const TArray<FMeshResizingCustomRegion>& ResizingGroupData);

	private:
		/** Return the Dataflow node owning by this property. */
		static FDataflowNode* GetPropertyOwnerDataflowNode(const TSharedPtr<IPropertyHandle>& PropertyHandle, const UStruct* DataflowNodeStruct);
	};
}  // End namespace UE::Chaos::ClothAsset

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogChaosClothAssetDataflowNodes, Log, All);

#undef UE_API
