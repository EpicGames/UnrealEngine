// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ReferenceBoneNode.generated.h"

/**
 * The managed array collection group used in the selection.
 * This separate structure is required to allow for customization of the UI.
 */
USTRUCT()
struct FChaosClothAssetReferenceBoneSelection
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Bone Name")
	FName Name = "";
};

/** Explicitly set the cloth Reference Bone (used when calculating SimulationVelocityScale). This will automatically computed as the bone closest to the root with weights if no valid Reference Bone is explicitly set.
* Reference bones are shared by all LODs. Only the bones set for LOD0 will be used.
*/
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetReferenceBoneNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetReferenceBoneNode, "ReferenceBone", "Cloth", "Cloth Reference Bone")

public:

	FChaosClothAssetReferenceBoneNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	void OnCalculateDefaultReferenceBone(UE::Dataflow::FContext& Context);

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Select the cloth reference bone. This will automatically computed as the bone closest to the root with weights if no valid Reference Bone is explicitly set. */
	UPROPERTY(EditAnywhere, Category = "Simulation Reference Bone Selection")
	FChaosClothAssetReferenceBoneSelection ReferenceBone;

	UPROPERTY(EditAnywhere, Transient, Category = "Simulation Reference Bone Selection", Meta = (DisplayName = "Calculate Default"))
	FDataflowFunctionProperty CalculateDefaultReferenceBone;
};


