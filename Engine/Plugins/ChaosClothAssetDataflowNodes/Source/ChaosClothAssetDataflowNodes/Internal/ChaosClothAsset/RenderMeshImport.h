// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/Vector.h"
#include "Math/Color.h"
#include "Templates/SharedPointer.h"

#define UE_API CHAOSCLOTHASSETDATAFLOWNODES_API

struct FManagedArrayCollection;
struct FMeshBuildSettings;
struct FMeshDescription;
struct FStaticMaterial;

namespace UE::Chaos::ClothAsset
{
	/**
	 * Render mesh import from mesh description.
	 */
	struct FRenderMeshImport
	{
		struct FVertex
		{
			FVector3f RenderPosition;
			FVector3f RenderNormal;
			FVector3f RenderTangentU;
			FVector3f RenderTangentV;
			TArray<FVector2f> RenderUVs;
			FLinearColor RenderColor;
			int32 OriginalIndex;
		};

		struct FTriangle
		{
			FIntVector3 VertexIndices;
			int32 OriginalIndex;
			int32 MaterialIndex;
		};

		struct FSection
		{
			TArray<FVertex> Vertices;
			TArray<FTriangle> Triangles;
			int32 NumTexCoords;
		};
		TSortedMap<int32, FSection> Sections;

		UE_API FRenderMeshImport(const FMeshDescription& InMeshDescription, const FMeshBuildSettings& BuildSettings);

		UE_API void AddRenderSections(
			const TSharedRef<FManagedArrayCollection> ClothCollection,
			const TArray<FStaticMaterial>& Materials,
			const FName OriginalTrianglesName,
			const FName OriginalVerticesName);

		UE_API void AddRenderSections(
			const TSharedRef<FManagedArrayCollection> ClothCollection,
			const TArray<FSoftObjectPath>& Materials,
			const FName OriginalTrianglesName,
			const FName OriginalVerticesName);

		UE_API void AddRenderPatternSelectionSets(
			const TSharedRef<FManagedArrayCollection> ClothCollection,
			const TMap<FName, TSet<int32>>& PatternNamesToIndices,
			const FName OriginalTrianglesName);
	};
}  // End namespace UE::Chaos::ClothAsset

#undef UE_API
