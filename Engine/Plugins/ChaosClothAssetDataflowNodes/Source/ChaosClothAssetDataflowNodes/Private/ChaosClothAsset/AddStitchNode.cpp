// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/AddStitchNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AddStitchNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetAddStitchNode"

FChaosClothAssetAddStitchNode::FChaosClothAssetAddStitchNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterInputConnection(&MergeToSingleVertexSelection.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
}

void FChaosClothAssetAddStitchNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		FCollectionClothFacade Cloth(ClothCollection);
		FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);
		const FName InSelectionName(*GetValue<FString>(Context, &MergeToSingleVertexSelection.StringValue));

		if (Cloth.IsValid() && SelectionFacade.IsValid() && InSelectionName != NAME_None)
		{
			const TSet<int32>* const SelectionSet = SelectionFacade.FindSelectionSet(InSelectionName);
			if (SelectionSet && SelectionSet->Num() > 1)
			{
				const FName& SelectionGroup = SelectionFacade.GetSelectionGroup(InSelectionName);
				TArray<FIntVector2> StitchPairs;
				if (SelectionGroup == ClothCollectionGroup::SimVertices2D)
				{
					StitchPairs.Reserve(SelectionSet->Num() - 1);

					int32 PrevIndex = INDEX_NONE;
					for (const int32 Index : *SelectionSet)
					{
						if (PrevIndex != INDEX_NONE)
						{
							StitchPairs.Add({ PrevIndex, Index });
						}
						PrevIndex = Index;
					}
				}
				else if (SelectionGroup == ClothCollectionGroup::SimVertices3D)
				{
					TConstArrayView<TArray<int32>> SimVertex2DLookup = Cloth.GetSimVertex2DLookup();
					StitchPairs.Reserve(SelectionSet->Num() - 1);
					int32 PrevIndex = INDEX_NONE;
					for (const int32 Index : *SelectionSet)
					{
						if (SimVertex2DLookup.IsValidIndex(Index) && !SimVertex2DLookup[Index].IsEmpty())
						{
							// Just pick first 2D vertex that corresponds with this 3D vertex
							const int32 Index2D = SimVertex2DLookup[Index][0];
							if (PrevIndex != INDEX_NONE && Index2D != INDEX_NONE)
							{
								StitchPairs.Add({ PrevIndex, Index2D });
							}
							PrevIndex = Index2D;
						}
					}
				}
				else
				{
					Context.Warning(
						FText::Format(LOCTEXT("SelectionTypeNotCorrectDetails", "Selection \"{0}\" does not have its index dependency group set to \"{1}\" or \"{2}\"."),
							FText::FromName(InSelectionName),
							FText::FromName(ClothCollectionGroup::SimVertices2D),
							FText::FromName(ClothCollectionGroup::SimVertices3D))
						,this, Out
					);
				}

				if (!StitchPairs.IsEmpty())
				{
					FCollectionClothSeamFacade Seam = Cloth.AddGetSeam();
					Seam.Initialize(TConstArrayView<FIntVector2>(StitchPairs));

					FClothGeometryTools::CleanupAndCompactMesh(ClothCollection);
				}
			}
		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
