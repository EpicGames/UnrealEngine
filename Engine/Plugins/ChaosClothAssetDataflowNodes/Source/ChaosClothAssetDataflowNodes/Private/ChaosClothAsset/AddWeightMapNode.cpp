// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/AddWeightMapNode.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowObject.h"
#include "InteractiveToolChange.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AddWeightMapNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetAddWeightMapNode"

namespace UE::Chaos::ClothAsset::Private
{
	void TransferWeightMap(
		const TConstArrayView<FVector2f>& InSourcePositions,
		const TConstArrayView<FIntVector3>& SourceIndices,
		const TConstArrayView<int32> SourceWeightsLookup,
		const TConstArrayView<float>& InSourceWeights,
		const TConstArrayView<FVector2f>& InTargetPositions,
		const TConstArrayView<FIntVector3>& TargetIndices,
		const TConstArrayView<int32> TargetWeightsLookup,
		TArray<float>& OutTargetWeights)
	{
		TArray<FVector3f> SourcePositions;
		SourcePositions.SetNumUninitialized(InSourcePositions.Num());
		for (int32 Index = 0; Index < SourcePositions.Num(); ++Index)
		{
			SourcePositions[Index] = FVector3f(InSourcePositions[Index], 0.f);
		}

		TArray<float> SourceWeights;
		SourceWeights.SetNumUninitialized(InSourcePositions.Num());
		for (int32 Index = 0; Index < SourceWeights.Num(); ++Index)
		{
			SourceWeights[Index] = InSourceWeights[SourceWeightsLookup[Index]];
		}

		TArray<FVector3f> TargetPositions;
		TArray<FVector3f> TargetNormals;
		TargetPositions.SetNumUninitialized(InTargetPositions.Num());
		TargetNormals.SetNumUninitialized(InTargetPositions.Num());
		for (int32 Index = 0; Index < TargetPositions.Num(); ++Index)
		{
			TargetPositions[Index] = FVector3f(InTargetPositions[Index], 0.f);
			TargetNormals[Index] = FVector3f::ZAxisVector;
		}

		TArray<float> TargetWeights;
		TargetWeights.SetNumUninitialized(TargetPositions.Num());

		FClothGeometryTools::TransferWeightMap(SourcePositions, SourceIndices, SourceWeights, TargetPositions, TargetNormals, TargetIndices, TArrayView<float>(TargetWeights));

		for (int32 Index = 0; Index < TargetWeights.Num(); ++Index)
		{
			OutTargetWeights[TargetWeightsLookup[Index]] = TargetWeights[Index];
		}
	}


	void SetVertexWeights(const TConstArrayView<float> InputMap, const TArray<float>& FinalValues, EChaosClothAssetWeightMapOverrideType OverrideType, TArray<float>& SourceVertexWeights)
	{
		if (InputMap.IsEmpty() || OverrideType == EChaosClothAssetWeightMapOverrideType::ReplaceAll)
		{
			// Default input is 0, so OverrideType doesn't matter.
			SourceVertexWeights = FinalValues;
			return;
		}

		check(InputMap.Num() == FinalValues.Num());
		SourceVertexWeights.SetNumUninitialized(FinalValues.Num());
		for (int32 Index = 0; Index < FinalValues.Num(); ++Index)
		{
			switch (OverrideType)
			{
			case EChaosClothAssetWeightMapOverrideType::ReplaceChanged:
				if (InputMap[Index] == FinalValues[Index])
				{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					SourceVertexWeights[Index] = FChaosClothAssetAddWeightMapNode::ReplaceChangedPassthroughValue;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				else
				{
					SourceVertexWeights[Index] = FinalValues[Index];
				}
				break;
			case EChaosClothAssetWeightMapOverrideType::Add:
				SourceVertexWeights[Index] = FinalValues[Index] - InputMap[Index];
				break;
			default: unimplemented();
			}
		}
	}

	void CalculateFinalVertexWeightValues(const TConstArrayView<float> InputMap, TArrayView<float> FinalOutputMap, EChaosClothAssetWeightMapOverrideType OverrideType, const TArray<float>& SourceVertexWeights)
	{
		const int32 EndWeightIndex = FMath::Min(FinalOutputMap.Num(), SourceVertexWeights.Num());
		if (InputMap.IsEmpty())
		{
			for (int32 Index = 0; Index < EndWeightIndex; ++Index)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FinalOutputMap[Index] = FMath::Clamp(SourceVertexWeights[Index] == FChaosClothAssetAddWeightMapNode::ReplaceChangedPassthroughValue ?
					0.f : SourceVertexWeights[Index], 0.f, 1.f);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			return;
		}

		check(InputMap.Num() == FinalOutputMap.Num());
		for (int32 Index = 0; Index < EndWeightIndex; ++Index)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if(SourceVertexWeights[Index] == FChaosClothAssetAddWeightMapNode::ReplaceChangedPassthroughValue)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				// This value is only set when OverrideType == ReplaceChanged, but it's possible the override type changed.
				FinalOutputMap[Index] = FMath::Clamp(InputMap[Index], 0.f, 1.f);
			}
			else
			{
				switch (OverrideType)
				{
				case EChaosClothAssetWeightMapOverrideType::ReplaceAll:
				case EChaosClothAssetWeightMapOverrideType::ReplaceChanged:
					FinalOutputMap[Index] = FMath::Clamp(SourceVertexWeights[Index], 0.f, 1.f);
					break;
				case EChaosClothAssetWeightMapOverrideType::Add:
					FinalOutputMap[Index] = FMath::Clamp(InputMap[Index] + SourceVertexWeights[Index], 0.f, 1.f);
					break;
				default: unimplemented();
				}
			}
		}
		if (InputMap.GetData() != FinalOutputMap.GetData())
		{
			// Fill in remaining values with InputMap
			for (int32 Index = EndWeightIndex; Index < FinalOutputMap.Num(); ++Index)
			{
				FinalOutputMap[Index] = FMath::Clamp(InputMap[Index], 0.f, 1.f);
			}
		}
	}
}

FChaosClothAssetAddWeightMapNode::FChaosClothAssetAddWeightMapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&InputName.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&TransferCollection)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Name);
}

void FChaosClothAssetAddWeightMapNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	using namespace UE::Chaos::ClothAsset;

	if (UChaosClothAsset* const ClothAsset = Cast<UChaosClothAsset>(Asset.Get()))
	{
		if (UDataflow* const DataflowAsset = ClothAsset->GetDataflow())
		{
			const TSharedPtr<UE::Dataflow::FGraph, ESPMode::ThreadSafe> Dataflow = DataflowAsset->GetDataflow();
			if (const TSharedPtr<FDataflowNode> BaseNode = Dataflow->FindBaseNode(this->GetGuid()))  // This is basically a safe const_cast
			{
				FChaosClothAssetAddWeightMapNode* const MutableThis = static_cast<FChaosClothAssetAddWeightMapNode*>(BaseNode.Get());
				check(MutableThis == this);

				// Make the name a valid attribute name, and replace the value in the UI
				FWeightMapTools::MakeWeightMapName(MutableThis->Name);

				// Transfer weight map if the transfer collection input has changed and is valid
				FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
				const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
				FCollectionClothConstFacade ClothFacade(ClothCollection);
				if (ClothFacade.IsValid())  // Can only act on the collection if it is a valid cloth collection
				{
					FManagedArrayCollection InTransferCollection = GetValue<FManagedArrayCollection>(Context, &TransferCollection);
					const TSharedRef<const FManagedArrayCollection> TransferClothCollection = MakeShared<const FManagedArrayCollection>(MoveTemp(InTransferCollection));
					FCollectionClothConstFacade TransferClothFacade(TransferClothCollection);

					const FName InInputName = GetInputName(Context);
					const uint32 NameTypeHash = HashCombineFast(GetTypeHash(InInputName), (uint32)TransferType);
					const uint32 InTransferCollectionHash = (TransferClothFacade.HasValidSimulationData() && InInputName != NAME_None) ?
						HashCombineFast(TransferClothFacade.CalculateWeightMapTypeHash(), NameTypeHash) : 0;  // TODO: Remove after adding the function (currently shelved!)  
				
					if (TransferCollectionHash != InTransferCollectionHash)
					{
						MutableThis->TransferCollectionHash = InTransferCollectionHash;

						if (TransferCollectionHash)
						{
							if (TransferClothFacade.HasWeightMap(InInputName))
							{
								// Remap the weights
								TArray<float> RemappedWeights;
								RemappedWeights.SetNumZeroed(ClothFacade.GetNumSimVertices3D());

								switch (TransferType)
								{
								case EChaosClothAssetWeightMapTransferType::Use2DSimMesh:
									Private::TransferWeightMap(
										TransferClothFacade.GetSimPosition2D(),
										TransferClothFacade.GetSimIndices2D(),
										TransferClothFacade.GetSimVertex3DLookup(),
										TransferClothFacade.GetWeightMap(InInputName),
										ClothFacade.GetSimPosition2D(),
										ClothFacade.GetSimIndices2D(),
										ClothFacade.GetSimVertex3DLookup(),
										RemappedWeights);
										break;
								case EChaosClothAssetWeightMapTransferType::Use3DSimMesh:
									FClothGeometryTools::TransferWeightMap(
										TransferClothFacade.GetSimPosition3D(),
										TransferClothFacade.GetSimIndices3D(),
										TransferClothFacade.GetWeightMap(InInputName),
										ClothFacade.GetSimPosition3D(),
										ClothFacade.GetSimNormal(),
										ClothFacade.GetSimIndices3D(),
										TArrayView<float>(RemappedWeights));
										break;
								default: unimplemented();
								}

								MutableThis->SetVertexWeights(ClothFacade.GetWeightMap(InInputName), RemappedWeights);
							}
						}
					}
				}
			}
		}
	}
}

void FChaosClothAssetAddWeightMapNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	auto CheckSourceVertexWeights = [this, &Context, Out](TArrayView<float>& ClothWeights, const TArray<float>& SourceVertexWeights, bool bIsSim)
	{
		if (SourceVertexWeights.Num() > 0 && SourceVertexWeights.Num() != ClothWeights.Num())
		{
			Context.Warning(
				FText::Format(LOCTEXT("VertexCountMismatchDetails", "{0} vertex weights in the node: {1}\n{0} vertices in the cloth: {2}"),
					bIsSim ? FText::FromString("Sim") : FText::FromString("Render"),
					SourceVertexWeights.Num(),
					ClothWeights.Num())
				, this, Out
			);
		}
	};

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Evaluate InputName
		const FName InInputName = GetInputName(Context);

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid())  // Can only act on the collection if it is a valid cloth collection
		{
			const FName InName(Name.IsEmpty() ? InInputName : FName(Name));

			// Copy simulation weights into cloth collection

			if (MeshTarget == EChaosClothAssetWeightMapMeshType::Simulation || MeshTarget == EChaosClothAssetWeightMapMeshType::Both)
			{
				ClothFacade.AddWeightMap(InName);		// Does nothing if weight map already exists
				TArrayView<float> ClothSimWeights = ClothFacade.GetWeightMap(InName);

				if (ClothSimWeights.Num() != ClothFacade.GetNumSimVertices3D())
				{
					check(ClothSimWeights.Num() == 0);
					Context.Warning(
						FText::Format(LOCTEXT("InvalidSimWeightMapNameDetails", "Could not create a sim weight map with name \"{0}\" (reserved name? wrong type?)."),
							FText::FromName(InName))
						, this, Out
					);
				}
				else
				{
					constexpr bool bIsSim = true;
					CheckSourceVertexWeights(ClothSimWeights, GetVertexWeights(), bIsSim);
					CalculateFinalVertexWeightValues(ClothFacade.GetWeightMap(InInputName), ClothSimWeights);
				}
			}
			
			// Copy render weights into cloth collection

			if (MeshTarget == EChaosClothAssetWeightMapMeshType::Render || MeshTarget == EChaosClothAssetWeightMapMeshType::Both)
			{
				ClothFacade.AddUserDefinedAttribute<float>(InName, ClothCollectionGroup::RenderVertices);
				TArrayView<float> ClothRenderWeights = ClothFacade.GetUserDefinedAttribute<float>(InName, ClothCollectionGroup::RenderVertices);

				if (ClothRenderWeights.Num() != ClothFacade.GetNumRenderVertices())
				{
					check(ClothRenderWeights.Num() == 0);
					Context.Warning(
						FText::Format(LOCTEXT("InvalidRenderWeightMapNameDetails", "Could not create a render weight map with name \"{0}\" (reserved name? wrong type?)."),
							FText::FromName(InName))
						,this, Out
					);
				}
				else
				{
					constexpr bool bIsSim = false;
					CheckSourceVertexWeights(ClothRenderWeights, GetRenderVertexWeights(), bIsSim);
					CalculateFinalRenderVertexWeightValues(ClothFacade.GetUserDefinedAttribute<float>(InInputName, ClothCollectionGroup::RenderVertices), ClothRenderWeights);
				}
			}

		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
	else if (Out->IsA<FString>(&Name))
	{
		FString InputNameString = GetValue<FString>(Context, &InputName.StringValue);
		UE::Chaos::ClothAsset::FWeightMapTools::MakeWeightMapName(InputNameString);
		SetValue(Context, Name.IsEmpty() ? InputNameString : Name, &Name);
	}
}

FName FChaosClothAssetAddWeightMapNode::GetInputName(UE::Dataflow::FContext& Context) const
{
	FString InputNameString = GetValue<FString>(Context, &InputName.StringValue);
	UE::Chaos::ClothAsset::FWeightMapTools::MakeWeightMapName(InputNameString);
	const FName InInputName(*InputNameString);
	return InInputName != NAME_None ? InInputName : FName(Name);
}

void FChaosClothAssetAddWeightMapNode::SetVertexWeights(const TConstArrayView<float> InputMap, const TArray<float>& FinalValues)
{
	UE::Chaos::ClothAsset::Private::SetVertexWeights(InputMap, FinalValues, MapOverrideType, GetVertexWeights());
}

void FChaosClothAssetAddWeightMapNode::SetRenderVertexWeights(const TConstArrayView<float> InputMap, const TArray<float>& FinalValues)
{
	UE::Chaos::ClothAsset::Private::SetVertexWeights(InputMap, FinalValues, MapOverrideType, GetRenderVertexWeights());
}

void FChaosClothAssetAddWeightMapNode::CalculateFinalVertexWeightValues(const TConstArrayView<float> InputMap, TArrayView<float> FinalOutputMap) const
{
	UE::Chaos::ClothAsset::Private::CalculateFinalVertexWeightValues(InputMap, FinalOutputMap, MapOverrideType, GetVertexWeights());
}

void FChaosClothAssetAddWeightMapNode::CalculateFinalRenderVertexWeightValues(const TConstArrayView<float> InputMap, TArrayView<float> FinalOutputMap) const
{
	UE::Chaos::ClothAsset::Private::CalculateFinalVertexWeightValues(InputMap, FinalOutputMap, MapOverrideType, GetRenderVertexWeights());
}


// Object encapsulating a change to the AddWeightMap node's values. Used for Undo/Redo.
class FChaosClothAssetAddWeightMapNode::FWeightMapNodeChange final : public FToolCommandChange
{

public: 

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FWeightMapNodeChange(const FChaosClothAssetAddWeightMapNode& Node) :
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		NodeGuid(Node.GetGuid()),
		SavedWeights(Node.GetVertexWeights()),
		SavedRenderWeights(Node.GetRenderVertexWeights()),
		SavedMapOverrideType(Node.MapOverrideType),
		SavedWeightMapName(Node.Name)
	{}

private:

	FGuid NodeGuid;
	TArray<float> SavedWeights;

	// Note we could store only one set of weights and use a bool to determine whether we are updating sim or render vertices, however in the future 
	// we may enable writing both weight maps to the node at once.
	TArray<float> SavedRenderWeights;

	EChaosClothAssetWeightMapOverrideType SavedMapOverrideType;
	FString SavedWeightMapName;

	virtual FString ToString() const final
	{
		return TEXT("ChaosClothAssetAddWeightMapNodeChange");
	}

	virtual void Apply(UObject* Object) final
	{
		SwapApplyRevert(Object);
	}

	virtual void Revert(UObject* Object) final
	{
		SwapApplyRevert(Object);
	}

	void SwapApplyRevert(UObject* Object)
	{
		if (UDataflow* const Dataflow = Cast<UDataflow>(Object))
		{
			if (const TSharedPtr<FDataflowNode> BaseNode = Dataflow->GetDataflow()->FindBaseNode(NodeGuid))
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (FChaosClothAssetAddWeightMapNode* const Node = BaseNode->AsType<FChaosClothAssetAddWeightMapNode>())
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				{
					Swap(Node->GetVertexWeights(), SavedWeights);
					Swap(Node->GetRenderVertexWeights(), SavedRenderWeights);
					Swap(Node->MapOverrideType, SavedMapOverrideType);
					Swap(Node->Name, SavedWeightMapName);

					Node->Invalidate();
				}
			}
		}
	}
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TUniquePtr<FToolCommandChange> FChaosClothAssetAddWeightMapNode::MakeWeightMapNodeChange(const FChaosClothAssetAddWeightMapNode& Node)
{
	return MakeUnique<FChaosClothAssetAddWeightMapNode::FWeightMapNodeChange>(Node);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
