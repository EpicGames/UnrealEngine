// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ApplyProxyDeformerNode.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ApplyProxyDeformerNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetApplyProxyDeformerNode"

FChaosClothAssetApplyProxyDeformerNode::FChaosClothAssetApplyProxyDeformerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetApplyProxyDeformerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;
		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		FClothGeometryTools::ApplyProxyDeformer(ClothCollection, bIgnoreSkinningBlendWeights);

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
