// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ApplyResizingNode.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Engine/SkeletalMesh.h"
#include "MeshResizing/CustomRegionResizing.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ApplyResizingNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetApplyResizingNode"

FChaosClothAssetApplyResizingNode::FChaosClothAssetApplyResizingNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterInputConnection(&TargetSkeletalMesh);
	RegisterInputConnection(&InterpolationData);
	RegisterInputConnection(&SkeletalMeshLODIndex)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&bForceApplyToRenderMesh)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&SourceSkeletalMesh)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&bSkipCustomRegionResizing)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&bSavePreResizedSimPosition3D)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetApplyResizingNode::EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const
{
	using namespace UE::Chaos::ClothAsset;
	FCollectionClothFacade ClothFacade(ClothCollection);
	if (ClothFacade.IsValid())
	{
		if (TObjectPtr<const USkeletalMesh> InTargetSKM = GetValue(Context, &TargetSkeletalMesh))
		{
			const int32 InLODIndex = GetValue(Context, &SkeletalMeshLODIndex);
			if (InTargetSKM->IsValidLODIndex(InLODIndex) && InTargetSKM->HasMeshDescription(InLODIndex))
			{
				if (const FMeshDescription* const TargetMeshDescription = InTargetSKM->GetMeshDescription(InLODIndex))
				{
					const FMeshResizingRBFInterpolationData& InInterpolationData = GetValue(Context, &InterpolationData);
					if (InInterpolationData.SampleIndices.Num())
					{
						if (!GetValue(Context, &bForceApplyToRenderMesh) && ClothFacade.HasValidSimulationData())
						{
							if (bSavePreResizedSimPosition3D)
							{
								if (!ClothFacade.IsValid(EClothCollectionExtendedSchemas::Resizing))
								{
									ClothFacade.DefineSchema(EClothCollectionExtendedSchemas::Resizing);
								}
								TArrayView<FVector3f> PreResizedView = ClothFacade.GetPreResizedSimPosition3D();
								const TConstArrayView<FVector3f> SimPosView = ClothFacade.GetSimPosition3D();

								if (ensure(PreResizedView.Num() == SimPosView.Num()))
								{
									for (int32 i = 0; i < SimPosView.Num(); ++i)
									{
										PreResizedView[i] = SimPosView[i];
									}
								}
							}
							TArray<FMeshResizingCustomRegion> ResizingGroupData;
							if (!GetValue(Context, &bSkipCustomRegionResizing))
							{
								if (TObjectPtr<const USkeletalMesh> InSourceSKM = GetValue(Context, &SourceSkeletalMesh))
								{
									if (const FMeshDescription* const SourceMeshDescription = InTargetSKM->GetMeshDescription(InLODIndex))
									{
										FClothDataflowTools::GenerateSimMeshResizingGroupData(ClothCollection, *SourceMeshDescription, ResizingGroupData);
									}
								}
							}

							UE::MeshResizing::FRBFInterpolation::DeformPoints(*TargetMeshDescription, InInterpolationData, ClothFacade.GetSimPosition3D(), ClothFacade.GetSimNormal());

							if (!ResizingGroupData.IsEmpty())
							{
								FClothDataflowTools::ApplySimGroupResizing(ClothCollection, *TargetMeshDescription, InInterpolationData, ResizingGroupData);
							}
						}
						else if (ClothFacade.HasValidRenderData())
						{
							TArray<FMeshResizingCustomRegion> ResizingGroupData;
							if (!GetValue(Context, &bSkipCustomRegionResizing))
							{
								if (TObjectPtr<const USkeletalMesh> InSourceSKM = GetValue(Context, &SourceSkeletalMesh))
								{
									if (const FMeshDescription* const SourceMeshDescription = InTargetSKM->GetMeshDescription(InLODIndex))
									{
										FClothDataflowTools::GenerateRenderMeshResizingGroupData(ClothCollection, *SourceMeshDescription, ResizingGroupData);
									}
								}
							}

							UE::MeshResizing::FRBFInterpolation::DeformPoints(*TargetMeshDescription, InInterpolationData, ClothFacade.GetRenderPosition(), ClothFacade.GetRenderNormal(), ClothFacade.GetRenderTangentU(), ClothFacade.GetRenderTangentV());

							if (!ResizingGroupData.IsEmpty())
							{
								FClothDataflowTools::ApplyRenderGroupResizing(ClothCollection, *TargetMeshDescription, InInterpolationData, ResizingGroupData);
							}
						}
					}
				}
			}
		}
	}
}

void FChaosClothAssetApplyResizingNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyBool(this, &bSavePreResizedSimPosition3D);
}
#undef LOCTEXT_NAMESPACE
