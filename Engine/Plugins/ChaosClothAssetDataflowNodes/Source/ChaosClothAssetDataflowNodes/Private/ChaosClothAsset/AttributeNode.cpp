// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/AttributeNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttributeNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetAttributeNode"

FChaosClothAssetAttributeNode_v2::FChaosClothAssetAttributeNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Name.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIOStringValue, StringValue));
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Name.StringValue, &Name.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIOStringValue, StringValue));
}

void FChaosClothAssetAttributeNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		FCollectionClothFacade Cloth(ClothCollection);
		const FName GroupName = *Group.Name;

		const FString& InName = GetValue(Context, &Name.StringValue);

		if (Cloth.IsValid() && !InName.IsEmpty())
		{
			if (ClothCollection->HasGroup(GroupName))
			{
				switch (Type)
				{
				case EChaosClothAssetNodeAttributeType::Integer:
					Cloth.AddUserDefinedAttribute<int32>(*InName, GroupName);
					{
						TArrayView<int32> Values = Cloth.GetUserDefinedAttribute<int32>(*InName, GroupName);
						for (int32& Value : Values)
						{
							Value = IntValue;
						}
					}
					break;
				case EChaosClothAssetNodeAttributeType::Float:
					Cloth.AddUserDefinedAttribute<float>(*InName, GroupName);
					{
						TArrayView<float> Values = Cloth.GetUserDefinedAttribute<float>(*InName, GroupName);
						for (float& Value : Values)
						{
							Value = FloatValue;
						}
					}
					break;
				case EChaosClothAssetNodeAttributeType::Vector:
					Cloth.AddUserDefinedAttribute<FVector3f>(*InName, GroupName);
					{
						TArrayView<FVector3f> Values = Cloth.GetUserDefinedAttribute<FVector3f>(*InName, GroupName);
						for (FVector3f& Value : Values)
						{
							Value = VectorValue;
						}
					}
					break;
				}
			}
			else if (!GroupName.IsNone())
			{
				Context.Warning(
					FText::Format(LOCTEXT("CreateAttributeDetail", "No group \"{0}\" currently exists on the input collection"), FText::FromName(GroupName))
					, this, Out
				);
			}
		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
	else if (Out->IsA<FString>(&Name.StringValue))
	{
		const FString& InName = GetValue(Context, &Name.StringValue);
		SetValue(Context, InName, &Name.StringValue);
	}
}

FChaosClothAssetAttributeNode::FChaosClothAssetAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Name);
}

void FChaosClothAssetAttributeNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		FCollectionClothFacade Cloth(ClothCollection);
		const FName GroupName = *Group.Name;
		
		if (Cloth.IsValid() && !Name.IsEmpty())
		{
			if (ClothCollection->HasGroup(GroupName))
			{
				switch (Type)
				{
				case EChaosClothAssetNodeAttributeType::Integer:
					Cloth.AddUserDefinedAttribute<int32>(*Name, GroupName);
					{
						TArrayView<int32> Values = Cloth.GetUserDefinedAttribute<int32>(*Name, GroupName);
						for (int32& Value : Values)
						{
							Value = IntValue;
						}
					}
					break;
				case EChaosClothAssetNodeAttributeType::Float:
					Cloth.AddUserDefinedAttribute<float>(*Name, GroupName);
					{
						TArrayView<float> Values = Cloth.GetUserDefinedAttribute<float>(*Name, GroupName);
						for (float& Value : Values)
						{
							Value = FloatValue;
						}
					}
					break;
				case EChaosClothAssetNodeAttributeType::Vector:
					Cloth.AddUserDefinedAttribute<FVector3f>(*Name, GroupName);
					{
						TArrayView<FVector3f> Values = Cloth.GetUserDefinedAttribute<FVector3f>(*Name, GroupName);
						for (FVector3f& Value : Values)
						{
							Value = VectorValue;
						}
					}
					break;
				}
			}
			else if (!GroupName.IsNone())
			{
				Context.Warning(
					FText::Format(LOCTEXT("CreateAttributeDetail", "No group \"{0}\" currently exists on the input collection"), FText::FromName(GroupName))
					, this, Out
				);
			}
		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
	else if (Out->IsA<FString>(&Name))
	{
		SetValue(Context, Name, &Name);
	}
}

#undef LOCTEXT_NAMESPACE
