// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/BlendVerticesNode.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlendVerticesNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetBlendVerticesNode"

namespace UE::Chaos::ClothAsset::Private
{
	template<typename T>
	void BlendValues(float BlendingWeight, const TArrayView<T>& Values, const TConstArrayView<T>& BlendingValues)
	{
		const int32 NumValues = FMath::Min(Values.Num(), BlendingValues.Num());
		const float OneMinusScalar = 1.f - BlendingWeight;
		for (int32 Index = 0; Index < NumValues; ++Index)
		{
			Values[Index] = OneMinusScalar * Values[Index] + BlendingWeight * BlendingValues[Index];
		}
	}

	template<typename T>
	void BlendNormalizedValues(float BlendingWeight, const TArrayView<T>& Values, const TConstArrayView<T>& BlendingValues)
	{
		const int32 NumValues = FMath::Min(Values.Num(), BlendingValues.Num());
		const float OneMinusScalar = 1.f - BlendingWeight;
		for (int32 Index = 0; Index < NumValues; ++Index)
		{
			Values[Index] = (OneMinusScalar * Values[Index] + BlendingWeight * BlendingValues[Index]).GetSafeNormal();
		}
	}

	static void BlendUVSets(float BlendingWeight, const TArrayView<TArray<FVector2f>>& Values, const TConstArrayView<TArray<FVector2f>>& BlendingValues)
	{
		const int32 NumValues = FMath::Min(Values.Num(), BlendingValues.Num());
		for (int32 Index = 0; Index < NumValues; ++Index)
		{
			BlendValues(BlendingWeight, TArrayView<FVector2f>(Values[Index]), TConstArrayView<FVector2f>(BlendingValues[Index]));
		}
	}
}

FChaosClothAssetBlendVerticesNode::FChaosClothAssetBlendVerticesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&BlendCollection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetBlendVerticesNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		if (BlendingWeight == 0.f)
		{
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		FManagedArrayCollection InBlendCollection = GetValue<FManagedArrayCollection>(Context, &BlendCollection);  // Can't use a const reference here sadly since the facade needs a SharedRef to be created
		const TSharedRef<const FManagedArrayCollection> BlendClothCollection = MakeShared<const FManagedArrayCollection>(MoveTemp(InBlendCollection));

		// Always check for a valid cloth collection/facade to avoid processing non cloth collections
		FCollectionClothFacade ClothFacade(ClothCollection);
		const FCollectionClothConstFacade BlendClothFacade(BlendClothCollection);

		auto LogVertexCountMismatch = [this, &Context, Out](const FText& VertexType, int32 CollectionCount, int32 BlendCollectionCount)
			{
				Context.Warning(
					FText::Format(LOCTEXT("VertexCountMismatchDetails", "{0} Vertex count mismatch {1} != {2}. Set 'Require Same Vertex Counts' to false to disable this check."),
						VertexType, CollectionCount, BlendCollectionCount)
					, this, Out
				);
			};

		if (ClothFacade.IsValid())
		{
			if (BlendClothFacade.IsValid())
			{
				if (bBlendSimMesh)
				{
					if (bBlend2DSimPositions)
					{
						const int32 NumSimVertices2D = ClothFacade.GetNumSimVertices2D();
						const int32 NumBlendSimVertices2D = BlendClothFacade.GetNumSimVertices2D();
						if (!bRequireSameVertexCounts || NumSimVertices2D == NumBlendSimVertices2D)
						{
							TArrayView<FVector2f> SimPosition2D = ClothFacade.GetSimPosition2D();
							TConstArrayView<FVector2f> BlendSimPosition2D = BlendClothFacade.GetSimPosition2D();
							ensure(SimPosition2D.Num() == NumSimVertices2D);
							ensure(BlendSimPosition2D.Num() == NumBlendSimVertices2D);
							Private::BlendValues(BlendingWeight, SimPosition2D, BlendSimPosition2D);
						}
						else
						{
							LogVertexCountMismatch(FText::FromString(TEXT("Sim Vertices 2D")), NumSimVertices2D, NumBlendSimVertices2D);
						}
					}

					if (bBlend3DSimPositions || bBlendSimNormals)
					{
						const int32 NumSimVertices3D = ClothFacade.GetNumSimVertices3D();
						const int32 NumBlendSimVertices3D = BlendClothFacade.GetNumSimVertices3D();
						if (!bRequireSameVertexCounts || NumSimVertices3D == NumBlendSimVertices3D)
						{
							if (bBlend3DSimPositions)
							{
								TArrayView<FVector3f> SimPosition3D = ClothFacade.GetSimPosition3D();
								TConstArrayView<FVector3f> BlendSimPosition3D = BlendClothFacade.GetSimPosition3D();
								ensure(SimPosition3D.Num() == NumSimVertices3D);
								ensure(BlendSimPosition3D.Num() == NumBlendSimVertices3D);
								Private::BlendValues(BlendingWeight, SimPosition3D, BlendSimPosition3D);
							}
							if (bBlendSimNormals)
							{
								TArrayView<FVector3f> SimNormal = ClothFacade.GetSimNormal();
								TConstArrayView<FVector3f> BlendSimNormal = BlendClothFacade.GetSimNormal();
								ensure(SimNormal.Num() == NumSimVertices3D);
								ensure(BlendSimNormal.Num() == NumBlendSimVertices3D);
								Private::BlendNormalizedValues(BlendingWeight, SimNormal, BlendSimNormal);
							}
						}
						else
						{
							LogVertexCountMismatch(FText::FromString(TEXT("Sim Vertices 3D")), NumSimVertices3D, NumBlendSimVertices3D);
						}
					}
				}

				if (bBlendRenderMesh)
				{
					const int32 NumRenderVertices = ClothFacade.GetNumRenderVertices();
					const int32 NumBlendRenderVertices = BlendClothFacade.GetNumRenderVertices();
					if (!bRequireSameVertexCounts || NumRenderVertices == NumBlendRenderVertices)
					{
						if (bBlendRenderPositions)
						{
							TArrayView<FVector3f> RenderPosition = ClothFacade.GetRenderPosition();
							TConstArrayView<FVector3f> BlendRenderPosition = BlendClothFacade.GetRenderPosition();
							ensure(RenderPosition.Num() == NumRenderVertices);
							ensure(BlendRenderPosition.Num() == NumBlendRenderVertices);
							Private::BlendValues(BlendingWeight, RenderPosition, BlendRenderPosition);
						}

						if (bBlendRenderNormalsAndTangents)
						{
							TArrayView<FVector3f> RenderNormal = ClothFacade.GetRenderNormal();
							TConstArrayView<FVector3f> BlendRenderNormal = BlendClothFacade.GetRenderNormal();
							ensure(RenderNormal.Num() == NumRenderVertices);
							ensure(BlendRenderNormal.Num() == NumBlendRenderVertices);
							Private::BlendNormalizedValues(BlendingWeight, RenderNormal, BlendRenderNormal);
							TArrayView<FVector3f> RenderTangentU = ClothFacade.GetRenderTangentU();
							TConstArrayView<FVector3f> BlendRenderTangentU = BlendClothFacade.GetRenderTangentU();
							ensure(RenderTangentU.Num() == NumRenderVertices);
							ensure(BlendRenderTangentU.Num() == NumBlendRenderVertices);
							Private::BlendNormalizedValues(BlendingWeight, RenderTangentU, BlendRenderTangentU);
							TArrayView<FVector3f> RenderTangentV = ClothFacade.GetRenderTangentV();
							TConstArrayView<FVector3f> BlendRenderTangentV = BlendClothFacade.GetRenderTangentV();
							ensure(RenderTangentV.Num() == NumRenderVertices);
							ensure(BlendRenderTangentV.Num() == NumBlendRenderVertices);
							Private::BlendNormalizedValues(BlendingWeight, RenderTangentV, BlendRenderTangentV);
						}

						if (bBlendRenderUVs)
						{
							TArrayView<TArray<FVector2f>> RenderUVs = ClothFacade.GetRenderUVs();
							TConstArrayView<TArray<FVector2f>> BlendRenderUVs = BlendClothFacade.GetRenderUVs();
							ensure(RenderUVs.Num() == NumRenderVertices);
							ensure(BlendRenderUVs.Num() == NumBlendRenderVertices);
							Private::BlendUVSets(BlendingWeight, RenderUVs, BlendRenderUVs);
						}

						if (bBlendRenderColors)
						{
							TArrayView<FLinearColor> RenderColor = ClothFacade.GetRenderColor();
							TConstArrayView<FLinearColor> BlendRenderColor = BlendClothFacade.GetRenderColor();
							ensure(RenderColor.Num() == NumRenderVertices);
							ensure(BlendRenderColor.Num() == NumBlendRenderVertices);
							Private::BlendValues(BlendingWeight, RenderColor, BlendRenderColor);
						}
					}
					else
					{
						LogVertexCountMismatch(FText::FromString(TEXT("Render Vertices")), NumRenderVertices, NumBlendRenderVertices);
					}
				}
			}
			else
			{
				Context.Warning(LOCTEXT("InvalidBlendCollectionDetails", "Input Blend Collection is not a valid Cloth Collection."), this, Out);
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
