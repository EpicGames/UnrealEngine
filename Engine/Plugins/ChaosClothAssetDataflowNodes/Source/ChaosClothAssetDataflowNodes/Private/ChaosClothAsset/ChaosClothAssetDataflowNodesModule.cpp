// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Range.h"
#include "ChaosClothAsset/AddStitchNode.h"
#include "ChaosClothAsset/AddWeightMapNode.h"
#include "ChaosClothAsset/ApplyProxyDeformerNode.h"
#include "ChaosClothAsset/ApplyResizingNode.h"
#include "ChaosClothAsset/AttributeNode.h"
#include "ChaosClothAsset/BindToRootBoneNode.h"
#include "ChaosClothAsset/BlendVerticesNode.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothCollectionAttribute.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothCollectionToDynamicMeshNode.h"
#include "ChaosClothAsset/ClothCollectionQueryNode.h"
#include "ChaosClothAsset/ClothRenderableTypes.h"
#include "ChaosClothAsset/ColorScheme.h"
#include "ChaosClothAsset/CopySimulationToRenderMeshNode.h"
#include "ChaosClothAsset/CustomRegionResizingNode.h"
#include "ChaosClothAsset/DatasmithImportNode.h"
#include "ChaosClothAsset/DeleteElementNode.h"
#include "ChaosClothAsset/EnableUVResizingNode.h"
#include "ChaosClothAsset/GenerateSimMorphTargetNode.h"
#include "ChaosClothAsset/ImportNode.h"
#include "ChaosClothAsset/ImportSimulationCacheNode.h"
#include "ChaosClothAsset/MakeClothAssetNode.h"
#include "ChaosClothAsset/MergeClothCollectionsNode.h"
#include "ChaosClothAsset/ProceduralSelectionNode.h"
#include "ChaosClothAsset/ProxyDeformerNode.h"
#include "ChaosClothAsset/RecalculateNormalsNode.h"
#include "ChaosClothAsset/ReferenceBoneNode.h"
#include "ChaosClothAsset/RemeshNode.h"
#include "ChaosClothAsset/ReverseNormalsNode.h"
#include "ChaosClothAsset/SelectionNode.h"
#include "ChaosClothAsset/SelectionToIntMapNode.h"
#include "ChaosClothAsset/SelectionToWeightMapNode.h"
#include "ChaosClothAsset/SetPhysicsAssetNode.h"
#include "ChaosClothAsset/SimAccessoryMeshNode.h"
#include "ChaosClothAsset/SimulationAerodynamicsConfigNode.h"
#include "ChaosClothAsset/SimulationAnimDriveConfigNode.h"
#include "ChaosClothAsset/SimulationBackstopConfigNode.h"
#include "ChaosClothAsset/SimulationBendingConfigNode.h"
#include "ChaosClothAsset/SimulationBendingOverrideConfigNode.h"
#include "ChaosClothAsset/SimulationClothVertexSpringConfigNode.h"
#include "ChaosClothAsset/SimulationClothVertexFaceSpringConfigNode.h"
#include "ChaosClothAsset/SimulationCollisionConfigNode.h"
#include "ChaosClothAsset/SimulationDampingConfigNode.h"
#include "ChaosClothAsset/SimulationDefaultConfigNode.h"
#include "ChaosClothAsset/SimulationGravityConfigNode.h"
#include "ChaosClothAsset/SimulationLongRangeAttachmentConfigNode.h"
#include "ChaosClothAsset/SimulationMassConfigNode.h"
#include "ChaosClothAsset/SimulationMaxDistanceConfigNode.h"
#include "ChaosClothAsset/SimulationMorphTargetConfigNode.h"
#include "ChaosClothAsset/SimulationPBDAreaSpringConfigNode.h"
#include "ChaosClothAsset/SimulationPBDBendingElementConfigNode.h"
#include "ChaosClothAsset/SimulationPBDBendingSpringConfigNode.h"
#include "ChaosClothAsset/SimulationPBDEdgeSpringConfigNode.h"
#include "ChaosClothAsset/SimulationPressureConfigNode.h"
#include "ChaosClothAsset/SimulationSelfCollisionConfigNode.h"
#include "ChaosClothAsset/SimulationSelfCollisionSpheresConfigNode.h"
#include "ChaosClothAsset/SimulationSolverConfigNode.h"
#include "ChaosClothAsset/SimulationMultiResConfigNode.h"
#include "ChaosClothAsset/SimulationResolveExtremeDeformationConfigNode.h"
#include "ChaosClothAsset/SimulationStretchConfigNode.h"
#include "ChaosClothAsset/SimulationStretchOverrideConfigNode.h"
#include "ChaosClothAsset/SimulationVelocityScaleConfigNode.h"
#include "ChaosClothAsset/SimulationXPBDAreaSpringConfigNode.h"
#include "ChaosClothAsset/SimulationXPBDAnisoBendingConfigNode.h"
#include "ChaosClothAsset/SimulationXPBDAnisoSpringConfigNode.h"
#include "ChaosClothAsset/SimulationXPBDAnisoStretchConfigNode.h"
#include "ChaosClothAsset/SimulationXPBDBendingElementConfigNode.h"
#include "ChaosClothAsset/SimulationXPBDBendingSpringConfigNode.h"
#include "ChaosClothAsset/SimulationXPBDEdgeSpringConfigNode.h"
#include "ChaosClothAsset/SkeletalMeshImportNode.h"
#include "ChaosClothAsset/SkinningBlendNode.h"
#include "ChaosClothAsset/StaticMeshImportNode.h"
#include "ChaosClothAsset/StripUserAttributesNode.h"
#include "ChaosClothAsset/TerminalNode.h"
#include "ChaosClothAsset/TransferSkinWeightsNode.h"
#include "ChaosClothAsset/TransformPositionsNode.h"
#include "ChaosClothAsset/TransformUVsNode.h"
#include "ChaosClothAsset/USDImportNode.h"
#include "ChaosClothAsset/USDImportNode_v2.h"
#include "ChaosClothAsset/WeightMapNode.h"
#include "ChaosClothAsset/WeightedValueCustomization.h"

#include "ChaosClothAsset/ClothDataflowViewModes.h"
#include "ChaosClothAsset/ConnectableValueCustomization.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "ChaosClothAsset/ImportedValueCustomization.h"
#include "ChaosClothAsset/SelectionGroupCustomization.h"
#include "ChaosClothAsset/WeightMapToSelectionNode.h"
#include "Dataflow/DataflowCollectionAddScalarVertexPropertyNode.h"
#include "Dataflow/DataflowCategoryRegistry.h"
#include "Dataflow/DataflowNodeColorsRegistry.h"
#include "Dataflow/DataflowNodeFactory.h"
#include "Dataflow/DataflowRenderingFactory.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/Facades/CollectionRenderingFacade.h"
#include "Modules/ModuleManager.h"
#include "ReferenceSkeleton.h"

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		static void RegisterDataflowNodes()
		{
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY_NODE_COLORS_BY_CATEGORY("Cloth", FColorScheme::NodeHeader, FColorScheme::NodeBody);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetAddStitchNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetWeightMapNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetAttributeNode_v2);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetApplyProxyDeformerNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetApplyResizingNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetBindToRootBoneNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetBlendVerticesNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetCollectionToDynamicMeshNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetCollectionQueryNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetCopySimulationToRenderMeshNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetCustomRegionResizingNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetDatasmithImportNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetDeleteElementNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetEnableUVResizingNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetExtractWeightMapNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetExtractSelectionSetNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetGenerateSimMorphTargetNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetImportNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetImportSimulationCacheNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetMakeClothAssetNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetMergeClothCollectionsNode_v2);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetProceduralSelectionNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetProxyDeformerNode_v3);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetReferenceBoneNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetRemeshNode_v2);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetReverseNormalsNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetRecalculateNormalsNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSelectionNode_v2);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSelectionToIntMapNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSelectionToWeightMapNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSetPhysicsAssetNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimAccessoryMeshNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationAerodynamicsConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationAnimDriveConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationBackstopConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationClothVertexSpringConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationClothVertexFaceSpringConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationCollisionConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationBendingConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationBendingOverrideConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationStretchConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationStretchOverrideConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationDampingConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationDefaultConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationGravityConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationMassConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationMaxDistanceConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationMorphTargetConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationPBDAreaSpringConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationPBDBendingElementConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationPBDBendingSpringConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationPBDEdgeSpringConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationPressureConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationSelfCollisionConfigNode_v2);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationSelfCollisionSpheresConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationSolverConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationVelocityScaleConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationXPBDAnisoBendingConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationXPBDAnisoSpringConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationXPBDAnisoStretchConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationXPBDAreaSpringConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationXPBDBendingElementConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationXPBDBendingSpringConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationXPBDEdgeSpringConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationResolveExtremeDeformationConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSkeletalMeshImportNode_v2);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSkinningBlendNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetStaticMeshImportNode_v2);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothStripUserAttributesNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetTerminalNode_v2);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetTransferSkinWeightsNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetTransformPositionsNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetTransformUVsNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetUpdateClothFromDynamicMeshNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetWeightMapToSelectionNode);
			// Deprecated nodes
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetAttributeNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetAddWeightMapNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetMergeClothCollectionsNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetProxyDeformerNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetProxyDeformerNode_v2);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetRemeshNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSelectionNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationLongRangeAttachmentConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationMultiResConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSimulationSelfCollisionConfigNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetSkeletalMeshImportNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetStaticMeshImportNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetTerminalNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetUSDImportNode);
			DATAFLOW_NODE_REGISTER_CREATION_FACTORY(FChaosClothAssetUSDImportNode_v2);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		static TArray<FName> GetReferenceBoneNamesFromCollection(const FManagedArrayCollection& Collection)
		{
			TSharedRef<FManagedArrayCollection> CollectionPtr = MakeShared<FManagedArrayCollection>(Collection);
			FCollectionClothConstFacade ClothFacade(CollectionPtr);
			const FSoftObjectPath& SkeletalMeshPathName = ClothFacade.GetSkeletalMeshSoftObjectPathName();
			TArray<FName> BoneNames;
			if (const USkeletalMesh* const SkeletalMesh = Cast<USkeletalMesh>(SkeletalMeshPathName.TryLoad()))
			{
				const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
				return RefSkeleton.GetRawRefBoneNames();
			}
			return TArray<FName>();
		}

		class FClothSurfaceRenderCallbacks : public UE::Dataflow::FRenderingFactory::ICallbackInterface
		{
		public:

			static UE::Dataflow::FRenderKey RenderKey;

		private:

			virtual UE::Dataflow::FRenderKey GetRenderKey() const override
			{
				return RenderKey;
			}

			virtual bool CanRender(const UE::Dataflow::IDataflowConstructionViewMode& ViewMode) const override
			{
				const FName& ViewModeName = ViewMode.GetName();
				return (ViewModeName == FCloth2DSimViewMode::Name ||
						ViewModeName == FCloth3DSimViewMode::Name ||
						ViewModeName == FClothRenderViewMode::Name ||
						ViewModeName == Dataflow::FDataflowConstruction3DViewMode::Name ||
						ViewModeName == Dataflow::FDataflowConstruction2DViewMode::Name);
			}

			virtual bool CanRenderFromState(const UE::Dataflow::FGraphRenderingState& State) const override
			{
				if (State.GetRenderOutputs().Num())
				{
					const FManagedArrayCollection Default;
					const FName PrimaryOutput = State.GetRenderOutputs()[0];
					FManagedArrayCollection Collection = State.GetValue<FManagedArrayCollection>(PrimaryOutput, Default);
					const TSharedRef<const FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(Collection));
					const FCollectionClothConstFacade ClothFacade(ClothCollection);
					if (ClothFacade.IsValid())
					{
						const FName& ViewModeName = State.GetViewMode().GetName();
						const bool bHasSimData = ClothFacade.HasValidSimulationData();
						const bool bHasRenderData = ClothFacade.HasValidRenderData();

						if (bHasSimData && (ViewModeName == FCloth2DSimViewMode::Name || ViewModeName == Dataflow::FDataflowConstruction2DViewMode::Name))
						{
							return true;
						}
						if (bHasSimData && (ViewModeName == FCloth3DSimViewMode::Name || ViewModeName == Dataflow::FDataflowConstruction3DViewMode::Name))
						{
							return true;
						}
						if (bHasRenderData && (ViewModeName == FClothRenderViewMode::Name || ViewModeName == Dataflow::FDataflowConstruction3DViewMode::Name))
						{
							return true;
						}

						return false;
					}
				}
				return CanRender(State.GetViewMode());
			}

			virtual void Render(GeometryCollection::Facades::FRenderingFacade& RenderCollection, const UE::Dataflow::FGraphRenderingState& State) override
			{
				if (State.GetRenderOutputs().Num())
				{
					const FManagedArrayCollection Default;
					checkf(State.GetRenderOutputs().Num() == 1, TEXT("Expected FGraphRenderingState object to have one render output"));
					const FName PrimaryOutput = State.GetRenderOutputs()[0];
					FManagedArrayCollection Collection = State.GetValue<FManagedArrayCollection>(PrimaryOutput, Default);

					const TSharedRef<const FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(Collection));
					const FCollectionClothConstFacade ClothFacade(ClothCollection);
					if (!ClothFacade.IsValid())
					{
						// Cloth collection may not be valid for all nodes
						return;
					}

					TArray<FVector3f> Vertices;
					TArray<FVector3f> Normals;
					TArray<FIntVector> Indices;
					TArray<TArray<FVector2f>> UVs;
					TArray<int32> RenderMaterialIDs;
					TArray<TArray<float>> BoneWeights;
					TArray<TArray<int32>> BoneIndices;


					const FName& ViewModeName = State.GetViewMode().GetName();
					const bool bHasSimData = ClothFacade.HasValidSimulationData();
					const bool bHasRenderData = ClothFacade.HasValidRenderData();

					if (bHasSimData && (ViewModeName == FCloth3DSimViewMode::Name || ViewModeName == Dataflow::FDataflowConstruction3DViewMode::Name))
					{
						Vertices = ClothFacade.GetSimPosition3D();
						Indices = ClothFacade.GetSimIndices3D();
						Normals = ClothFacade.GetSimNormal();

						BoneWeights = TArray<TArray<float>>(ClothFacade.GetSimBoneWeights());
						BoneIndices = TArray<TArray<int32>>(ClothFacade.GetSimBoneIndices());
					}
					else if (bHasSimData && (ViewModeName == FCloth2DSimViewMode::Name || ViewModeName == Dataflow::FDataflowConstruction2DViewMode::Name))
					{
						//
						// Flip the Y coordinate to get the desired visualization with our LVT_OrthoXY viewport
						//
						float MinY = UE_BIG_NUMBER;
						float MaxY = -UE_BIG_NUMBER;
						for (const FVector2f& Vertex2D : ClothFacade.GetSimPosition2D())
						{
							MinY = FMath::Min(MinY, Vertex2D[1]);
							MaxY = FMath::Max(MaxY, Vertex2D[1]);
						}
						for (const FVector2f& Vertex2D : ClothFacade.GetSimPosition2D())
						{
							Vertices.Add({ Vertex2D[0], MinY + (MaxY - Vertex2D[1]), 0.0 });
						}

						Indices = ClothFacade.GetSimIndices2D();

						FVector3f Normal2D(0.0f, 0.0f, 0.0f);
						for (const FIntVector3& Tri : ClothFacade.GetSimIndices2D())
						{
							const FVector2f AC = ClothFacade.GetSimPosition2D()[Tri[2]] - ClothFacade.GetSimPosition2D()[Tri[0]];
							const FVector2f AB = ClothFacade.GetSimPosition2D()[Tri[1]] - ClothFacade.GetSimPosition2D()[Tri[0]];
							const float Det = AB[0] * AC[1] - AC[0] * AB[1];
							if (FMath::Abs(Det) < UE_SMALL_NUMBER)
							{
								continue;
							}
							Normal2D = Det > 0 ? FVector3f(0.0f, 0.0f, 1.0f) : FVector3f(0.0f, 0.0f, -1.0f);
							break;
						}
						Normals.Init(Normal2D, ClothFacade.GetNumSimVertices2D());
					}
					else if (bHasRenderData && (ViewModeName == FClothRenderViewMode::Name || ViewModeName == Dataflow::FDataflowConstruction3DViewMode::Name))
					{
						Vertices = ClothFacade.GetRenderPosition();
						Indices = ClothFacade.GetRenderIndices();
						Normals = ClothFacade.GetRenderNormal();

						BoneWeights = TArray<TArray<float>>(ClothFacade.GetRenderBoneWeights());
						BoneIndices = TArray<TArray<int32>>(ClothFacade.GetRenderBoneIndices());
						
						const TConstArrayView<TArray<FVector2f>> AllClothUVs = ClothFacade.GetRenderUVs();
						check(AllClothUVs.Num() == Vertices.Num());
						UVs = TArray<TArray<FVector2f>>(AllClothUVs);

						RenderMaterialIDs.SetNumUninitialized(Indices.Num());

						for (int32 PatternIndex = 0; PatternIndex < ClothFacade.GetNumRenderPatterns(); ++PatternIndex)
						{
							const FCollectionClothRenderPatternConstFacade& Pattern = ClothFacade.GetRenderPattern(PatternIndex);
							const int32 PatternFaceIndexOffset = Pattern.GetRenderFacesOffset();
							for (int32 PatternFaceIndex = 0; PatternFaceIndex < Pattern.GetNumRenderFaces(); ++PatternFaceIndex)
							{
								RenderMaterialIDs[PatternFaceIndexOffset + PatternFaceIndex] = PatternIndex;
							}
						}
					}

					TArray<FLinearColor> Colors;
					Colors.Init(FLinearColor::Gray, Vertices.Num());	// TODO: Choose a vertex color

					const FString GeometryName = FString::Printf(TEXT("%s_%s"), *State.GetNodeName().ToString(), *PrimaryOutput.ToString());
					const int32 GeometryIndex = RenderCollection.StartGeometryGroup(GeometryName);

					if (RenderMaterialIDs.Num() > 0)
					{
						TArray<FString> MaterialArray;
						MaterialArray.Reserve(ClothFacade.GetRenderMaterialSoftObjectPathName().Num());
						for (const FSoftObjectPath& MaterialName : ClothFacade.GetRenderMaterialSoftObjectPathName())
						{
							MaterialArray.Emplace(MaterialName.ToString());
						}
						RenderCollection.AddSurface(MoveTemp(Vertices), MoveTemp(Indices), MoveTemp(Normals), MoveTemp(Colors), MoveTemp(UVs), MoveTemp(RenderMaterialIDs), MoveTemp(MaterialArray));
					}
					else
					{
						RenderCollection.AddSurface(MoveTemp(Vertices), MoveTemp(Indices), MoveTemp(Normals), MoveTemp(Colors));
					}

					if (!BoneWeights.IsEmpty() && !BoneIndices.IsEmpty())
					{
						RenderCollection.AddSurfaceBoneWeightsAndIndices(MoveTemp(BoneWeights), MoveTemp(BoneIndices));
					}

					RenderCollection.EndGeometryGroup(GeometryIndex);
				}
			}
		};

		UE::Dataflow::FRenderKey FClothSurfaceRenderCallbacks::RenderKey = { TEXT("SurfaceRender"), FName("FClothCollection") };

		static void RegisterRenderingCallbacks()
		{
			UE::Dataflow::FRenderingViewModeFactory::GetInstance().RegisterViewMode(MakeUnique<FCloth2DSimViewMode>());
			UE::Dataflow::FRenderingViewModeFactory::GetInstance().RegisterViewMode(MakeUnique<FCloth3DSimViewMode>());
			UE::Dataflow::FRenderingViewModeFactory::GetInstance().RegisterViewMode(MakeUnique<FClothRenderViewMode>());

			UE::Dataflow::FRenderingFactory::GetInstance()->RegisterCallbacks(MakeUnique<FClothSurfaceRenderCallbacks>());

			UE::Chaos::ClothAsset::RegisterClothRenderableTypes();
		}

		static void DeregisterRenderingCallbacks()
		{
			UE::Dataflow::FRenderingFactory::GetInstance()->DeregisterCallbacks(FClothSurfaceRenderCallbacks::RenderKey);

			UE::Dataflow::FRenderingViewModeFactory::GetInstance().DeregisterViewMode(FCloth2DSimViewMode::Name);
			UE::Dataflow::FRenderingViewModeFactory::GetInstance().DeregisterViewMode(FCloth3DSimViewMode::Name);
			UE::Dataflow::FRenderingViewModeFactory::GetInstance().DeregisterViewMode(FClothRenderViewMode::Name);
		}

		////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

		class FClothCollectionAddScalarVertexPropertyCallbacks : public IDataflowAddScalarVertexPropertyCallbacks
		{
		public:

			const static FName Name;

			virtual ~FClothCollectionAddScalarVertexPropertyCallbacks() override = default;

			virtual FName GetName() const override
			{
				return Name;
			}

			virtual TArray<FName> GetTargetGroupNames() const override
			{
				return { ClothCollectionGroup::SimVertices2D, ClothCollectionGroup::SimVertices3D, ClothCollectionGroup::RenderVertices };
			}

			virtual void GetTargetGroupInfos(TArray<FTargetGroupInfo>& OutInfos) const override
			{
				OutInfos.Add({
					.TargetGroup = ClothCollectionGroup::SimVertices2D,
					.PositionAttributeKey = { ClothCollectionAttribute::SimPosition3D, ClothCollectionGroup::SimVertices3D },
					.IndicesAttributeKey = { ClothCollectionAttribute::SimIndices3D, ClothCollectionGroup::SimFaces },
					.MappingFrom2DTo3DAttributeKey = { ClothCollectionAttribute::SimVertex3DLookup, ClothCollectionGroup::SimVertices2D },
					});
				OutInfos.Add({
					.TargetGroup = ClothCollectionGroup::SimVertices3D,
					.PositionAttributeKey = { ClothCollectionAttribute::SimPosition3D, ClothCollectionGroup::SimVertices3D },
					.IndicesAttributeKey = { ClothCollectionAttribute::SimIndices3D, ClothCollectionGroup::SimFaces },
					.MappingFrom2DTo3DAttributeKey = {},
					});
				OutInfos.Add({
					.TargetGroup = ClothCollectionGroup::RenderVertices,
					.PositionAttributeKey = { ClothCollectionAttribute::RenderPosition, ClothCollectionGroup::RenderVertices },
					.IndicesAttributeKey = { ClothCollectionAttribute::RenderIndices, ClothCollectionGroup::RenderFaces },
					.MappingFrom2DTo3DAttributeKey = {},
					});

			}

			virtual TArray<UE::Dataflow::FRenderingParameter> GetRenderingParameters() const override
			{
				return { {TEXT("SurfaceRender"), FName("FClothCollection"), {TEXT("Collection")}, FCloth2DSimViewMode::Name},
						 {TEXT("SurfaceRender"), FName("FClothCollection"), {TEXT("Collection")}, FCloth3DSimViewMode::Name},
						 {TEXT("SurfaceRender"), FName("FClothCollection"), {TEXT("Collection")}, FClothRenderViewMode::Name} };
			}
		};

		const FName FClothCollectionAddScalarVertexPropertyCallbacks::Name = FName(TEXT("FClothCollectionAddScalarVertexPropertyCallbacks"));

	}  // End namespace Private

	class FChaosClothAssetDataflowNodesModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
			using namespace UE::Chaos::ClothAsset;

			Private::RegisterDataflowNodes();

			UE_DATAFLOW_REGISTER_CATEGORY_FORASSET_TYPE("Cloth", UChaosClothAsset);

			// Register type customizations
			if (FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
			{
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetWeightedValue::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWeightedValueCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetWeightedValueNonAnimatable::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWeightedValueCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetWeightedValueNonAnimatableNoLowHighRange::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWeightedValueCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetWeightedValueOverride::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWeightedValueCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetConnectableStringValue::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FConnectableValueCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetConnectableIStringValue::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FConnectableValueCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetConnectableOStringValue::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FConnectableValueCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetConnectableIOStringValue::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FConnectableValueCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetNodeSelectionGroup::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSelectionGroupCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetNodeAttributeGroup::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FSelectionGroupCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetReferenceBoneSelection::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateLambda([]()
					{
						return MakeShareable(new UE::Dataflow::FPropertyListCustomization(UE::Dataflow::FPropertyListCustomization::FOnGetListNames::CreateStatic(&Private::GetReferenceBoneNamesFromCollection)));
					}));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetImportedVectorValue::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FImportedValueCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetImportedFloatValue::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FImportedValueCustomization::MakeInstance));
				PropertyModule->RegisterCustomPropertyTypeLayout(FChaosClothAssetImportedIntValue::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FImportedValueCustomization::MakeInstance));
			}

			Private::RegisterRenderingCallbacks();

			FDataflowAddScalarVertexPropertyCallbackRegistry::Get().RegisterCallbacks(MakeUnique<Private::FClothCollectionAddScalarVertexPropertyCallbacks>());
		}

		virtual void ShutdownModule() override
		{
			Private::DeregisterRenderingCallbacks();

			// Unregister type customizations
			if (UObjectInitialized() && !IsEngineExitRequested())
			{
				if (FPropertyEditorModule* const PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
				{
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetWeightedValue::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetWeightedValueNonAnimatable::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetWeightedValueNonAnimatableNoLowHighRange::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetWeightedValueOverride::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetConnectableStringValue::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetConnectableIStringValue::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetConnectableOStringValue::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetConnectableIOStringValue::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetNodeSelectionGroup::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetNodeAttributeGroup::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetReferenceBoneSelection::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetImportedVectorValue::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetImportedFloatValue::StaticStruct()->GetFName());
					PropertyModule->UnregisterCustomPropertyTypeLayout(FChaosClothAssetImportedIntValue::StaticStruct()->GetFName());
				}
			}

			FDataflowAddScalarVertexPropertyCallbackRegistry::Get().DeregisterCallbacks(Private::FClothCollectionAddScalarVertexPropertyCallbacks::Name);
		}
	};
}  // End namespace UE::Chaos::ClothAsset

IMPLEMENT_MODULE(UE::Chaos::ClothAsset::FChaosClothAssetDataflowNodesModule, ChaosClothAssetDataflowNodes)
