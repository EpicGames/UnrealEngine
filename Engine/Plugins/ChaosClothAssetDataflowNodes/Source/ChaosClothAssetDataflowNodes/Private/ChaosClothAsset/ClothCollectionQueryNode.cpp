// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothCollectionQueryNode.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothCollectionQueryNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetCollectionQueryNode"

FChaosClothAssetCollectionQueryNode::FChaosClothAssetCollectionQueryNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&bIsClothCollection);
	RegisterOutputConnection(&bHasClothSimMesh);
	RegisterOutputConnection(&bHasClothRenderMesh);
	RegisterOutputConnection(&bHasClothProxyDeformer);
	RegisterOutputConnection(&bBooleanPropertyValue);
}

void FChaosClothAssetCollectionQueryNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA<bool>(&bIsClothCollection)
		|| Out->IsA<bool>(&bHasClothSimMesh)
		|| Out->IsA<bool>(&bHasClothRenderMesh)
		|| Out->IsA<bool>(&bHasClothProxyDeformer)
		|| Out->IsA<bool>(&bBooleanPropertyValue))
	{
		// Setting all of these outputs once since it involves making a temporary copy of the whole collection.
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothConstFacade ClothFacade(ClothCollection);

		SetValue(Context, ClothFacade.IsValid(), &bIsClothCollection);
		SetValue(Context, FClothGeometryTools::HasSimMesh(ClothCollection), &bHasClothSimMesh);
		SetValue(Context, FClothGeometryTools::HasRenderMesh(ClothCollection), &bHasClothRenderMesh);
		SetValue(Context, ClothFacade.IsValid(EClothCollectionExtendedSchemas::RenderDeformer), &bHasClothProxyDeformer);

		Chaos::Softs::FCollectionPropertyConstFacade PropertyFacade(ClothCollection);
		bool bBooleanPropertyResult = bBooleanPropertyValue;
		if (PropertyFacade.IsValid())
		{
			bBooleanPropertyResult = PropertyFacade.GetValue(FName(BooleanPropertyName), bBooleanPropertyValue);
		}
		SetValue(Context, bBooleanPropertyResult, &bBooleanPropertyValue);
	}
}

#undef LOCTEXT_NAMESPACE
