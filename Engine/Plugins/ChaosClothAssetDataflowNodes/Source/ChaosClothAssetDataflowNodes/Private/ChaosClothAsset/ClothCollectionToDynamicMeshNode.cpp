// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothCollectionToDynamicMeshNode.h"
#include "ChaosClothAsset/ClothPatternToDynamicMesh.h"
#include "ChaosClothAsset/ClothPatternToDynamicMeshMappingSupport.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "Dataflow/DataflowInputOutput.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Materials/MaterialInterface.h"
#include "UDynamicMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothCollectionToDynamicMeshNode)

#define LOCTEXT_NAMESPACE "FChaosClothAssetCollectionToDynamicMeshNode"

FChaosClothAssetCollectionToDynamicMeshNode::FChaosClothAssetCollectionToDynamicMeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&SimDynamicMesh);
	RegisterOutputConnection(&RenderDynamicMesh);
	RegisterOutputConnection(&RenderMaterials);
}

void FChaosClothAssetCollectionToDynamicMeshNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Geometry;
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA(&SimDynamicMesh))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid())
		{
			TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
			NewMesh->Reset();
			FDynamicMesh3& DynamicMesh = NewMesh->GetMeshRef();
			FClothPatternToDynamicMesh Converter;
			Converter.Convert(ClothCollection, INDEX_NONE, EClothPatternVertexType::Sim3D, DynamicMesh);
			SetValue(Context, NewMesh, &SimDynamicMesh);
			return;
		}

		SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &SimDynamicMesh);
	}
	if (Out->IsA(&RenderDynamicMesh) || Out->IsA(&RenderMaterials))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid())
		{
			TObjectPtr<UDynamicMesh> NewMesh = NewObject<UDynamicMesh>();
			NewMesh->Reset();
			FDynamicMesh3& DynamicMesh = NewMesh->GetMeshRef();
			FClothPatternToDynamicMesh Converter;
			Converter.Convert(ClothCollection, INDEX_NONE, EClothPatternVertexType::Render, DynamicMesh);
			SetValue(Context, NewMesh, &RenderDynamicMesh);

			TArray<TObjectPtr<UMaterialInterface>> OutMaterials;
			const TConstArrayView<FSoftObjectPath> RenderMaterialPathNames = ClothFacade.GetRenderMaterialSoftObjectPathName();
			OutMaterials.Reserve(RenderMaterialPathNames.Num());
			for (int32 RenderPatternIndex = 0; RenderPatternIndex < RenderMaterialPathNames.Num(); ++RenderPatternIndex)
			{
				const FSoftObjectPath& RenderMaterialPathName = RenderMaterialPathNames[RenderPatternIndex];
				OutMaterials.Add(Cast<UMaterialInterface>(RenderMaterialPathName.TryLoad()));
			}
			SetValue(Context, OutMaterials, &RenderMaterials);
			return;
		}

		SetValue(Context, TObjectPtr<UDynamicMesh>(NewObject<UDynamicMesh>()), &RenderDynamicMesh);
		SetValue(Context, TArray<TObjectPtr<UMaterialInterface>>(), &RenderMaterials);
	}
}

FChaosClothAssetUpdateClothFromDynamicMeshNode::FChaosClothAssetUpdateClothFromDynamicMeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&DynamicMesh);
	RegisterInputConnection(&Materials);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetUpdateClothFromDynamicMeshNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid())
		{
			if (bCopyToRenderMaterials)
			{
				const TArray<TObjectPtr<const UMaterialInterface>> InMaterials = GetValue(Context, &Materials);
				TArrayView<FSoftObjectPath> RenderMaterialPathNames = ClothFacade.GetRenderMaterialSoftObjectPathName();
				for (int32 MaterialIndex = 0; MaterialIndex < FMath::Min(RenderMaterialPathNames.Num(), InMaterials.Num()); ++MaterialIndex)
				{
					RenderMaterialPathNames[MaterialIndex] = InMaterials[MaterialIndex]->GetPathName();
				}
			}
			if (const TObjectPtr<const UDynamicMesh> InMesh = GetValue(Context, &DynamicMesh))
			{
				const UE::Geometry::FDynamicMesh3& DynMesh = InMesh->GetMeshRef();
				FClothPatternToDynamicMeshMappingSupport ClothMapping(DynMesh);

				auto CopyNormals = [&DynMesh, &ClothMapping](const UE::Geometry::FDynamicMeshNormalOverlay* const NormalOverlay, const TConstArrayView<FIntVector>& TriangleIndices, TArrayView<FVector3f> Normals)
					{
						for (int32 VertexID : DynMesh.VertexIndicesItr())
						{
							const int32 ClothVertexID = ClothMapping.GetOriginalVertexID(VertexID);
							if (Normals.IsValidIndex(ClothVertexID))
							{
								// Get the UV values corresponding to VertexID in the dynamic mesh
								NormalOverlay->EnumerateVertexElements(VertexID,
									[&Normals, &ClothMapping, &TriangleIndices, ClothVertexID](int32 TriangleID, int32 ElementID, const FVector3f& NormalValue)->bool
									{
										const int32 ClothMeshTriID = ClothMapping.GetOriginalTriangleID(TriangleID);
										const FIntVector& ClothTri = TriangleIndices[ClothMeshTriID];
										for (int32 LocalIndex = 0; LocalIndex < 3; ++LocalIndex)
										{
											if (ClothTri[LocalIndex] == ClothVertexID)
											{
												Normals[ClothTri[LocalIndex]] = NormalValue;
											}
										}
										return true;
									});
							}
						}
					};

				if (bCopyToRenderPositions)
				{
					TArrayView<FVector3f> RenderPositions = ClothFacade.GetRenderPosition();
					for (int32 VertexID : DynMesh.VertexIndicesItr())
					{
						const int32 RenderMeshID = ClothMapping.GetOriginalVertexID(VertexID);
						if (RenderPositions.IsValidIndex(RenderMeshID))
						{
							RenderPositions[RenderMeshID] = FVector3f(DynMesh.GetVertexRef(VertexID));
						}
					}
				}
				if (bCopyToRendeNormalsAndTangents)
				{
					if (const UE::Geometry::FDynamicMeshAttributeSet* const AttributeSet = DynMesh.Attributes())
					{
						TConstArrayView<FIntVector> RenderTriangles = ClothFacade.GetRenderIndices();

						if (const UE::Geometry::FDynamicMeshNormalOverlay* const NormalOverlay = AttributeSet->PrimaryNormals())
						{
							CopyNormals(NormalOverlay, RenderTriangles, ClothFacade.GetRenderNormal());
						}
						if (const UE::Geometry::FDynamicMeshNormalOverlay* const TangentOverlay = AttributeSet->PrimaryTangents())
						{
							CopyNormals(TangentOverlay, RenderTriangles, ClothFacade.GetRenderTangentU());
						}
						if (const UE::Geometry::FDynamicMeshNormalOverlay* const TangentOverlay = AttributeSet->PrimaryBiTangents())
						{
							CopyNormals(TangentOverlay, RenderTriangles, ClothFacade.GetRenderTangentV());
						}
					}
				}
				if (bCopyUVsToRenderUVs)
				{
					if (const UE::Geometry::FDynamicMeshAttributeSet* const AttributeSet = DynMesh.Attributes())
					{
						const int32 UVStart = (UVChannelIndex == -1) ? 0: UVChannelIndex;
						const int32 UVCount = (UVChannelIndex == -1) ? AttributeSet->NumUVLayers() : 1;

						for (int32 UVIndex = UVStart; UVIndex < UVStart + UVCount; ++UVIndex)
						{
							if (const UE::Geometry::FDynamicMeshUVOverlay* const UVOverlay = AttributeSet->GetUVLayer(UVIndex))
							{
								TArrayView<TArray<FVector2f>> RenderUVs = ClothFacade.GetRenderUVs();
								TConstArrayView<FIntVector> RenderTriangles = ClothFacade.GetRenderIndices();
								for (int32 VertexID : DynMesh.VertexIndicesItr())
								{
									const int32 RenderMeshVertexID = ClothMapping.GetOriginalVertexID(VertexID);
									if (RenderUVs.IsValidIndex(RenderMeshVertexID))
									{
										while (UVIndex >= RenderUVs[RenderMeshVertexID].Num())
										{
											RenderUVs[RenderMeshVertexID].AddZeroed();
										}
										// Get the UV values corresponding to VertexID in the dynamic mesh
										UVOverlay->EnumerateVertexElements(VertexID,
											[&RenderUVs, &ClothMapping, &RenderTriangles, RenderMeshVertexID, UVIndex](int32 TriangleID, int32 ElementID, const FVector2f& UVValue)->bool
											{
												const int32 RenderMeshTriID = ClothMapping.GetOriginalTriangleID(TriangleID);
												const FIntVector& RenderTri = RenderTriangles[RenderMeshTriID];
												for (int32 LocalIndex = 0; LocalIndex < 3; ++LocalIndex)
												{
													if (RenderTri[LocalIndex] == RenderMeshVertexID)
													{ 
														RenderUVs[RenderTri[LocalIndex]][UVIndex] = UVValue;
													}
												}
												return true;
											});
									}
								}
							}
						}
					}
				}
				if (bCopyToSim3DPositions)
				{
					TArrayView<FVector3f> SimPositions = ClothFacade.GetSimPosition3D();
					for (int32 VertexID : DynMesh.VertexIndicesItr())
					{
						const int32 SimMeshID = ClothMapping.GetOriginalVertexID(VertexID);
						if (SimPositions.IsValidIndex(SimMeshID))
						{
							SimPositions[SimMeshID] = FVector3f(DynMesh.GetVertexRef(VertexID));
						}
					}
				}
				if (bCopyToSimNormals)
				{
					if (const UE::Geometry::FDynamicMeshAttributeSet* const AttributeSet = DynMesh.Attributes())
					{
						if (const UE::Geometry::FDynamicMeshNormalOverlay* const NormalOverlay = AttributeSet->PrimaryNormals())
						{
							CopyNormals(NormalOverlay, ClothFacade.GetSimIndices3D(), ClothFacade.GetSimNormal());
						}
					}
				}
				if (bCopyUVsToSim2DPositions)
				{
					if (const UE::Geometry::FDynamicMeshAttributeSet* const AttributeSet = DynMesh.Attributes())
					{
						if (const UE::Geometry::FDynamicMeshUVOverlay* const UVOverlay = AttributeSet->GetUVLayer(UVChannelIndex))
						{
							TArrayView<FVector2f> SimPositions = ClothFacade.GetSimPosition2D();
							const int32 NumSimVertices3D = ClothFacade.GetNumSimVertices3D();
							TConstArrayView<FIntVector> SimIndices2D = ClothFacade.GetSimIndices2D();
							TConstArrayView<FIntVector> SimIndices3D = ClothFacade.GetSimIndices3D();
							for (int32 VertexID : DynMesh.VertexIndicesItr())
							{
								const int32 SimMeshVertID = ClothMapping.GetOriginalVertexID(VertexID);
								if (SimMeshVertID >= 0 && SimMeshVertID < NumSimVertices3D)
								{
									UVOverlay->EnumerateVertexElements(VertexID,
										[&SimPositions, &SimIndices2D, &SimIndices3D, &ClothMapping, SimMeshVertID](int32 TriangleID, int32 ElementID, const FVector2f& UVValue)->bool
										{
											const int32 SimMeshTriID = ClothMapping.GetOriginalTriangleID(TriangleID);
											if (SimIndices3D.IsValidIndex(SimMeshTriID))
											{
												const FIntVector& Index3D = SimIndices3D[SimMeshTriID];
												const FIntVector& Index2D = SimIndices2D[SimMeshTriID];
												for (int32 LocalIndex = 0; LocalIndex < 3; ++LocalIndex)
												{
													if (Index3D[LocalIndex] == SimMeshVertID)
													{
														if (SimPositions.IsValidIndex(Index2D[LocalIndex]))
														{
															SimPositions[Index2D[LocalIndex]] = UVValue;
														}
														break;
													}
												}
											}
											return true;
										});
								}
							}
						}
					}
				}
			}
		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

FChaosClothAssetExtractWeightMapNode::FChaosClothAssetExtractWeightMapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&WeightMap.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterInputConnection(&DynamicMesh)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&ExtractedWeightMap);
}

void FChaosClothAssetExtractWeightMapNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&ExtractedWeightMap))
	{
		using namespace UE::Chaos::ClothAsset;

		TArray<float> Result;

		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid())
		{
			const FName WeightMapName(*GetValue<FString>(Context, &WeightMap.StringValue));
			switch (MeshTarget)
			{
			default:
			case EChaosClothAssetWeightMapMeshTarget::Simulation:
				if (ClothFacade.HasWeightMap(WeightMapName))
				{
					Result = ClothFacade.GetWeightMap(WeightMapName);
				}
				break;
			case EChaosClothAssetWeightMapMeshTarget::Render:
				if (ClothFacade.HasUserDefinedAttribute<float>(WeightMapName, ClothCollectionGroup::RenderVertices))
				{
					Result = ClothFacade.GetUserDefinedAttribute<float>(WeightMapName, ClothCollectionGroup::RenderVertices);
				}
				break;
			}

			if (bReorderForDynamicMesh && !Result.IsEmpty())
			{
				if (const TObjectPtr<const UDynamicMesh> InMesh = GetValue(Context, &DynamicMesh))
				{
					const UE::Geometry::FDynamicMesh3& DynMesh = InMesh->GetMeshRef();
					FClothPatternToDynamicMeshMappingSupport ClothMapping(DynMesh);

					TArray<float> ReorderedResult;
					ReorderedResult.Init(0.f, DynMesh.MaxVertexID());
					for (int32 VertexID : DynMesh.VertexIndicesItr())
					{
						const int32 ClothMeshID = ClothMapping.GetOriginalVertexID(VertexID);
						if (Result.IsValidIndex(ClothMeshID))
						{
							ReorderedResult[VertexID] = Result[ClothMeshID];
						}
					}

					Result = MoveTemp(ReorderedResult);
				}
			}
		}
		SetValue(Context, MoveTemp(Result), &ExtractedWeightMap);
	}
}



FChaosClothAssetExtractSelectionSetNode::FChaosClothAssetExtractSelectionSetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&Selection.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterInputConnection(&DynamicMesh)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&ExtractedSelectionSet);
	RegisterOutputConnection(&ExtractedSelectionArray);
}

void FChaosClothAssetExtractSelectionSetNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&ExtractedSelectionSet) || Out->IsA(&ExtractedSelectionArray))
	{
		using namespace UE::Chaos::ClothAsset;

		TSet<int32> Result;

		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);
		if (ClothFacade.IsValid() && SelectionFacade.IsValid())
		{
			const FName SelectionName(*GetValue<FString>(Context, &Selection.StringValue));
			if (SelectionFacade.HasSelection(SelectionName))
			{
				const FName SelectionGroup = SelectionFacade.GetSelectionGroup(SelectionName);
				if (SelectionGroup == ClothCollectionGroup::SimVertices3D || SelectionGroup == ClothCollectionGroup::RenderVertices)
				{
					Result = SelectionFacade.GetSelectionSet(SelectionName);
				}
			}

			if (bReorderForDynamicMesh && !Result.IsEmpty())
			{
				if (const TObjectPtr<const UDynamicMesh> InMesh = GetValue(Context, &DynamicMesh))
				{
					const UE::Geometry::FDynamicMesh3& DynMesh = InMesh->GetMeshRef();
					FClothPatternToDynamicMeshMappingSupport ClothMapping(DynMesh);
					TSet<int32> ReorderedResult;
					ReorderedResult.Reserve(Result.Num());
					for (int32 VertexID : DynMesh.VertexIndicesItr())
					{
						const int32 ClothMeshID = ClothMapping.GetOriginalVertexID(VertexID);
						if (Result.Contains(ClothMeshID))
						{
							ReorderedResult.Add(VertexID);
						}
					}

					Result = MoveTemp(ReorderedResult);
				}
			}
		}
		SetValue(Context, Result.Array(), &ExtractedSelectionArray);
		SetValue(Context, MoveTemp(Result), &ExtractedSelectionSet);
	}
}
#undef LOCTEXT_NAMESPACE
