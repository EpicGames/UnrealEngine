// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "Dataflow/DataflowNode.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Modules/ModuleManager.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Util/ColorConstants.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "MeshUtilities.h"
#include "MeshResizing/CustomRegionResizing.h"
#include "MeshResizing/RBFInterpolation.h"
#include "PropertyHandle.h"
#include "ReferenceSkeleton.h"
#include "SkeletalMeshAttributes.h"
#include "ToDynamicMesh.h"

DEFINE_LOG_CATEGORY(LogChaosClothAssetDataflowNodes);

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		//
		// Wrapper for accessing a SkelMeshSection. Implements the interface expected by TToDynamicMesh<>.
		// This will weld all vertices which are the same.
		//
		template<bool bHasTangents = false, bool bHasBiTangents = false, bool bHasColors = false>
		struct FSkelMeshSectionWrapper
		{
			typedef int32 TriIDType;
			typedef int32 VertIDType;
			typedef int32 WedgeIDType;
			typedef int32 UVIDType;
			typedef int32 NormalIDType;
			typedef int32 ColorIDType;

			FSkelMeshSectionWrapper(const FSkeletalMeshLODModel& SkeletalMeshModel, const int32 SectionIndex, bool bInHasNormals)
				: bHasNormals(bInHasNormals)
				, SourceSection(SkeletalMeshModel.Sections[SectionIndex])
				, IndexBuffer(SkeletalMeshModel.IndexBuffer.GetData() + SourceSection.BaseIndex, SourceSection.NumTriangles * 3)
			{
				const int32 NumVerts = SourceSection.SoftVertices.Num();
				const int32 NumTriangles = SourceSection.NumTriangles;

				// We need to weld the mesh verts to get rid of duplicates (happens for smoothing groups)
				TArray<FVector> UniqueVerts;
				OriginalToMerged.AddDefaulted(NumVerts);
				constexpr float ThreshSq = UE_THRESH_POINTS_ARE_SAME * UE_THRESH_POINTS_ARE_SAME;
				for (int32 VertIndex = 0; VertIndex < NumVerts; ++VertIndex)
				{
					const FSoftSkinVertex& SourceVert = SourceSection.SoftVertices[VertIndex];

					bool bUnique = true;
					int32 RemapIndex = INDEX_NONE;

					const int32 NumUniqueVerts = UniqueVerts.Num();
					for (int32 UniqueVertIndex = 0; UniqueVertIndex < NumUniqueVerts; ++UniqueVertIndex)
					{
						FVector& UniqueVert = UniqueVerts[UniqueVertIndex];

						if ((UniqueVert - (FVector)SourceVert.Position).SizeSquared() <= ThreshSq)
						{
							// Not unique
							bUnique = false;
							RemapIndex = UniqueVertIndex;

							break;
						}
					}

					if (bUnique)
					{
						// Unique
						UniqueVerts.Add((FVector)SourceVert.Position);
						OriginalIndexes.Add(VertIndex);
						OriginalToMerged[VertIndex] = VertIndex;
					}
					else
					{
						OriginalToMerged[VertIndex] = OriginalIndexes[RemapIndex];
					}
				}

				TriIDs.SetNum(NumTriangles);
				for (int32 Tri = 0; Tri < NumTriangles; ++Tri)
				{
					TriIDs[Tri] = Tri;
				}
			}

			int32 NumTris() const
			{
				return TriIDs.Num();
			}

			int32 NumVerts() const
			{
				return OriginalIndexes.Num();
			}

			int32 NumUVLayers() const
			{
				return MAX_TEXCOORDS;
			}

			// --"Vertex Buffer" info 
			const TArray<int32>& GetVertIDs() const
			{
				return OriginalIndexes;
			}

			const FVector3d GetPosition(const VertIDType VtxID) const
			{
				return (FVector3d)SourceSection.SoftVertices[VtxID].Position;
			}

			// --"Index Buffer" info
			const TArray<int32>& GetTriIDs() const
			{
				return TriIDs;
			}

			// return false if this TriID is not contained in mesh.
			bool GetTri(const TriIDType TriID, VertIDType& VID0, VertIDType& VID1, VertIDType& VID2) const
			{
				if (TriID >= 0 && TriID < (int32)SourceSection.NumTriangles)
				{
					VID0 = OriginalToMerged[IndexBuffer[3 * TriID + 0] - SourceSection.BaseVertexIndex];
					VID1 = OriginalToMerged[IndexBuffer[3 * TriID + 1] - SourceSection.BaseVertexIndex];
					VID2 = OriginalToMerged[IndexBuffer[3 * TriID + 2] - SourceSection.BaseVertexIndex];
					return true;
				}
				return false;
			}

			bool HasNormals() const
			{
				return bHasNormals;
			}

			bool HasTangents() const
			{
				return bHasTangents;
			}

			bool HasBiTangents() const
			{
				return bHasBiTangents;
			}

			bool HasColors() const
			{
				return bHasColors;
			}

			// Each triangle corner is a wedge. This will lookup into original unwelded soft verts
			void GetWedgeIDs(const TriIDType& TriID, WedgeIDType& WID0, WedgeIDType& WID1, WedgeIDType& WID2) const
			{
				WID0 = IndexBuffer[3 * TriID + 0] - SourceSection.BaseVertexIndex;
				WID1 = IndexBuffer[3 * TriID + 1] - SourceSection.BaseVertexIndex;
				WID2 = IndexBuffer[3 * TriID + 2] - SourceSection.BaseVertexIndex;
			}

			// attribute access per-wedge
			// NB:  ToDynamicMesh will attempt to weld identical attributes that are associated with the same vertex
			FVector2f GetWedgeUV(int32 UVLayerIndex, WedgeIDType WID) const
			{
				check(UVLayerIndex < MAX_TEXCOORDS);
				return SourceSection.SoftVertices[WID].UVs[UVLayerIndex];
			}

			FVector3f GetWedgeNormal(WedgeIDType WID) const
			{
				return SourceSection.SoftVertices[WID].TangentZ;
			}

			FVector3f GetWedgeTangent(WedgeIDType WID) const
			{
				return SourceSection.SoftVertices[WID].TangentX;
			}

			FVector3f GetWedgeBiTangent(WedgeIDType WID) const
			{
				return SourceSection.SoftVertices[WID].TangentY;
			}

			FVector4f GetWedgeColor(WedgeIDType WID) const
			{
				return FLinearColor(SourceSection.SoftVertices[WID].Color);
			}

			// attribute access that exploits shared attributes. 
			// each group of shared attributes presents itself as a mesh with its own attribute vertex buffer.
			// NB:  If the mesh has no shared Attr attributes, then Get{Attr}IDs() should return an empty array.
			// NB:  Get{Attr}Tri() functions should return false if the triangle is not set in the attribute mesh. 
			const TArray<UVIDType>& GetUVIDs(int32 LayerID) const
			{
				return EmptyArray;
			}

			FVector2f GetUV(int32 LayerID, UVIDType UVID) const
			{
				check(false);
				return FVector2f();
			}

			bool GetUVTri(int32 LayerID, const TriIDType& TID, UVIDType& ID0, UVIDType& ID1, UVIDType& ID2) const
			{
				return false;
			}

			const TArray<NormalIDType>& GetNormalIDs() const
			{
				if (bHasNormals)
				{
					return OriginalIndexes;
				}
				else
				{
					return EmptyArray;
				}
			}

			FVector3f GetNormal(NormalIDType ID) const
			{
				check(bHasNormals);
				return SourceSection.SoftVertices[ID].TangentZ;
			}

			bool GetNormalTri(const TriIDType& TriID, NormalIDType& NID0, NormalIDType& NID1, NormalIDType& NID2) const
			{
				if (bHasNormals)
				{
					return GetTri(TriID, NID0, NID1, NID2);
				}
				return false;
			}

			const TArray<NormalIDType>& GetTangentIDs() const
			{
				return EmptyArray;
			}

			FVector3f GetTangent(NormalIDType ID) const
			{
				check(false);
				return FVector3f();
			}

			bool GetTangentTri(const TriIDType& TID, NormalIDType& NID0, NormalIDType& NID1, NormalIDType& NID2) const
			{
				return false;
			}

			const TArray<NormalIDType>& GetBiTangentIDs() const
			{
				return EmptyArray;
			}

			FVector3f GetBiTangent(NormalIDType ID) const
			{
				check(false);
				return FVector3f();
			}

			bool GetBiTangentTri(const TriIDType& TID, NormalIDType& NID0, NormalIDType& NID1, NormalIDType& NID2) const
			{
				return false;
			}

			const TArray<ColorIDType>& GetColorIDs() const
			{
				return EmptyArray;
			}

			FVector4f GetColor(ColorIDType ID) const
			{
				check(false);
				return FVector4f();
			}

			bool GetColorTri(const TriIDType& TID, ColorIDType& NID0, ColorIDType& NID1, ColorIDType& NID2) const
			{
				return false;
			}

			// weight maps information
			int32 NumWeightMapLayers() const
			{
				return 0;
			}

			float GetVertexWeight(int32 WeightMapIndex, int32 SrcVertID) const
			{
				check(false);
				return 0.f;
			}

			FName GetWeightMapName(int32 WeightMapIndex) const
			{
				check(false);
				return FName();
			}

			// skin weight attributes information
			int32 NumSkinWeightAttributes() const
			{
				return 1;
			}

			UE::AnimationCore::FBoneWeights GetVertexSkinWeight(int32 SkinWeightAttributeIndex, VertIDType VtxID) const
			{
				using namespace UE::AnimationCore;
				check(SkinWeightAttributeIndex == 0);
				const int32 NumInfluences = SourceSection.MaxBoneInfluences;
				const FSoftSkinVertex& SoftVertex = SourceSection.SoftVertices[VtxID];
				TArray<FBoneWeight> BoneWeightArray;
				BoneWeightArray.SetNumUninitialized(NumInfluences);
				for (int32 Idx = 0; Idx < NumInfluences; ++Idx)
				{
					BoneWeightArray[Idx] = FBoneWeight(SourceSection.BoneMap[SoftVertex.InfluenceBones[Idx]], (float)SoftVertex.InfluenceWeights[Idx] * UE::AnimationCore::InvMaxRawBoneWeightFloat);
				}
				return FBoneWeights::Create(BoneWeightArray, FBoneWeightsSettings());
			}

			FName GetSkinWeightAttributeName(int32 SkinWeightAttributeIndex) const
			{
				checkfSlow(SkinWeightAttributeIndex == 0, TEXT("Cloth assets should only have one skin weight profile"));
				return FSkeletalMeshAttributes::DefaultSkinWeightProfileName;
			}

			// bone attributes information
			int32 GetNumBones() const
			{
				return 0;
			}

			FName GetBoneName(int32 BoneIdx) const
			{
				check(false);
				return FName();
			}

			int32 GetBoneParentIndex(int32 BoneIdx) const
			{
				check(false);
				return INDEX_NONE;
			}

			FTransform GetBonePose(int32 BoneIdx) const
			{
				check(false);
				return FTransform();
			}

			FVector4f GetBoneColor(int32 BoneIdx) const
			{
				check(false);
				return FLinearColor::White;
			}

			const bool bHasNormals;
			const FSkelMeshSection& SourceSection;
			const TConstArrayView<uint32> IndexBuffer;
			TArray<int32> OriginalIndexes; // UniqueIndex -> OrigIndex
			TArray<int32> OriginalToMerged; // OriginalIndex -> OriginalIndexes[UniqueVertIndex] 
			TArray<int32> TriIDs;
			TArray<int32> EmptyArray;

		};

		namespace Resizing
		{
			static void ApplyGroupResizing(FCollectionClothFacade& ClothFacade, const FMeshDescription& TargetMeshDescription, const FMeshResizingRBFInterpolationData& InterpolationData, const TArray<FMeshResizingCustomRegion>& ResizingGroupData, TArrayView<FVector3f> Positions, TConstArrayView<float> ResizingBlend)
			{
				if (ClothFacade.IsValid(EClothCollectionExtendedSchemas::Resizing))
				{
					TConstArrayView<int32> ClothSetTypes = ClothFacade.GetCustomResizingRegionType();
					check(ClothSetTypes.Num() == ResizingGroupData.Num());

					// Gather trilinear interpolation data.
					TArray<int32> TrilinearInterpolationGroups;
					TArray<FVector3d> BoundCorners;
					int32 BoundCornersIndex = 0;

					for (int32 GroupIndex = 0; GroupIndex < ClothSetTypes.Num(); ++GroupIndex)
					{
						if ((EMeshResizingCustomRegionType)ClothSetTypes[GroupIndex] == EMeshResizingCustomRegionType::TrilinearInterpolation)
						{
							const FMeshResizingCustomRegion& ResizingData = ResizingGroupData[GroupIndex];

							if (ResizingData.IsValid())
							{
								TrilinearInterpolationGroups.Add(GroupIndex);

								const FMatrix TriangleMatrix(FVector3d(ResizingData.SourceAxis0), FVector3d(ResizingData.SourceAxis1), FVector3d(ResizingData.SourceAxis2), ResizingData.SourceOrigin);
								FOrientedBox BoundBox;
								BoundBox.Center = TriangleMatrix.TransformPosition(FVector3d(ResizingData.RegionBoundsCentroid));
								BoundBox.AxisX = FVector3d(ResizingData.SourceAxis0);
								BoundBox.AxisY = FVector3d(ResizingData.SourceAxis1);
								BoundBox.AxisZ = FVector3d(ResizingData.SourceAxis2);
								BoundBox.ExtentX = ResizingData.RegionBoundsExtents.X;
								BoundBox.ExtentY = ResizingData.RegionBoundsExtents.Y;
								BoundBox.ExtentZ = ResizingData.RegionBoundsExtents.Z;
								BoundCorners.SetNum((BoundCornersIndex + 1) * 8);
								BoundBox.CalcVertices(&BoundCorners[BoundCornersIndex]);
								BoundCornersIndex += 8;
							}
						}
					}
					if (!TrilinearInterpolationGroups.IsEmpty())
					{
						UE::MeshResizing::FRBFInterpolation::DeformPoints(TargetMeshDescription, InterpolationData, BoundCorners);
						BoundCornersIndex = 0;
						check(BoundCorners.Num() == TrilinearInterpolationGroups.Num() * 8);

						const TArray<FVector3f> OrigPositions(Positions.GetData(), Positions.Num());
						for (const int32 GroupIndex : TrilinearInterpolationGroups)
						{
							const FMeshResizingCustomRegion& ResizingData = ResizingGroupData[GroupIndex];

							UE::MeshResizing::FCustomRegionResizing::InterpolateCustomRegionPoints(ResizingData, TConstArrayView<FVector3d>(BoundCorners.GetData() + 8 * BoundCornersIndex, 8), Positions);
							BoundCornersIndex += 8;
						}

						for (int32 VertexIndex = 0; VertexIndex < ResizingBlend.Num(); ++VertexIndex)
						{
							Positions[VertexIndex] = FMath::Lerp(OrigPositions[VertexIndex], Positions[VertexIndex], ResizingBlend[VertexIndex]);
						}
					}
				}
			}
		} // namespace Private::Resizing
	} // Private

	void FClothDataflowTools::AddRenderPatternFromSkeletalMeshSection(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FSkeletalMeshLODModel& SkeletalMeshModel, const int32 SectionIndex, const FSoftObjectPath& RenderMaterialPathName)
	{
		check(SectionIndex < SkeletalMeshModel.Sections.Num());

		FCollectionClothFacade Cloth(ClothCollection);
		check(Cloth.IsValid());

		FCollectionClothRenderPatternFacade ClothPatternFacade = Cloth.AddGetRenderPattern();

		const FSkelMeshSection& Section = SkeletalMeshModel.Sections[SectionIndex];
		ClothPatternFacade.SetNumRenderVertices(Section.NumVertices);
		ClothPatternFacade.SetNumRenderFaces(Section.NumTriangles);

		TArrayView<FVector3f> RenderPosition = ClothPatternFacade.GetRenderPosition();
		TArrayView<FVector3f> RenderNormal = ClothPatternFacade.GetRenderNormal();
		TArrayView<FVector3f> RenderTangentU = ClothPatternFacade.GetRenderTangentU();
		TArrayView<FVector3f> RenderTangentV = ClothPatternFacade.GetRenderTangentV();
		TArrayView<TArray<FVector2f>> RenderUVs = ClothPatternFacade.GetRenderUVs();
		TArrayView<FLinearColor> RenderColor = ClothPatternFacade.GetRenderColor();
		TArrayView<TArray<int32>> RenderBoneIndices = ClothPatternFacade.GetRenderBoneIndices();
		TArrayView<TArray<float>> RenderBoneWeights = ClothPatternFacade.GetRenderBoneWeights();
		const uint32 NumTexCoords = FMath::Min((uint32)MAX_TEXCOORDS, SkeletalMeshModel.NumTexCoords);
		for (int32 VertexIndex = 0; VertexIndex < Section.NumVertices; ++VertexIndex)
		{
			const FSoftSkinVertex& SoftVertex = Section.SoftVertices[VertexIndex];

			RenderPosition[VertexIndex] = SoftVertex.Position;
			RenderNormal[VertexIndex] = SoftVertex.TangentZ;
			RenderTangentU[VertexIndex] = SoftVertex.TangentX;
			RenderTangentV[VertexIndex] = SoftVertex.TangentY;
			RenderUVs[VertexIndex].SetNum(NumTexCoords);
			for (uint32 TexCoordIndex = 0; TexCoordIndex < NumTexCoords; ++TexCoordIndex)
			{
				RenderUVs[VertexIndex][TexCoordIndex] = SoftVertex.UVs[TexCoordIndex];
			}

			RenderColor[VertexIndex] = SoftVertex.Color.ReinterpretAsLinear();

			const int32 NumBones = Section.MaxBoneInfluences;
			RenderBoneIndices[VertexIndex].SetNum(NumBones);
			RenderBoneWeights[VertexIndex].SetNum(NumBones);
			for (int32 BoneIndex = 0; BoneIndex < NumBones; ++BoneIndex)
			{
				RenderBoneIndices[VertexIndex][BoneIndex] = (int32)Section.BoneMap[(int32)SoftVertex.InfluenceBones[BoneIndex]];
				RenderBoneWeights[VertexIndex][BoneIndex] = (float)SoftVertex.InfluenceWeights[BoneIndex] * UE::AnimationCore::InvMaxRawBoneWeightFloat;
			}
		}

		const int32 VertexOffset = ClothPatternFacade.GetRenderVerticesOffset();
		TArrayView<FIntVector3> RenderIndices = ClothPatternFacade.GetRenderIndices();
		for (uint32 FaceIndex = 0; FaceIndex < Section.NumTriangles; ++FaceIndex)
		{
			const uint32 IndexOffset = Section.BaseIndex + FaceIndex * 3;
			RenderIndices[FaceIndex] = FIntVector3(
				SkeletalMeshModel.IndexBuffer[IndexOffset + 0] - Section.BaseVertexIndex + VertexOffset,
				SkeletalMeshModel.IndexBuffer[IndexOffset + 1] - Section.BaseVertexIndex + VertexOffset,
				SkeletalMeshModel.IndexBuffer[IndexOffset + 2] - Section.BaseVertexIndex + VertexOffset
			);
		}
		ClothPatternFacade.SetRenderMaterialSoftObjectPathName(RenderMaterialPathName);
	}
	
	void FClothDataflowTools::AddSimPatternsFromSkeletalMeshSection(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FSkeletalMeshLODModel& SkeletalMeshModel, const int32 SectionIndex, const int32 UVChannelIndex, const FVector2f& UVScale, bool bImportNormals, TArray<int32>* OutSim2DToSourceVertex)
	{
		check(SectionIndex < SkeletalMeshModel.Sections.Num());

		// Convert to DynamicMesh and then use that to create patterns.
		UE::Geometry::TToDynamicMesh<Private::FSkelMeshSectionWrapper<>> SkelMeshSectionToDynamicMesh;
		Private::FSkelMeshSectionWrapper<> SectionWrapper(SkeletalMeshModel, SectionIndex, bImportNormals);

		UE::Geometry::FDynamicMesh3 DynamicMesh;
		DynamicMesh.EnableAttributes();
		constexpr bool bCopyTangents = false;
		SkelMeshSectionToDynamicMesh.Convert(DynamicMesh, SectionWrapper, [](int32) { return 0; }, [](int32) { return INDEX_NONE; }, bCopyTangents);

		// Set ToSrcVertIDMap as an overlay that the build sim mesh code expects.
		UE::Geometry::FNonManifoldMappingSupport::AttachNonManifoldVertexMappingData(SkelMeshSectionToDynamicMesh.ToSrcVertIDMap, DynamicMesh);

		constexpr bool bAppend = true;
		const int32 OrigNumSimVertices2D = FCollectionClothConstFacade(ClothCollection).GetNumSimVertices2D();
		check(!OutSim2DToSourceVertex || OutSim2DToSourceVertex->Num() == OrigNumSimVertices2D);
		FClothGeometryTools::BuildSimMeshFromDynamicMesh(ClothCollection, DynamicMesh, UVChannelIndex, UVScale, bAppend, bImportNormals, OutSim2DToSourceVertex);
		if (OutSim2DToSourceVertex)
		{
			// SrcVertID doesn't include SourceSection.BaseVertexIndex.
			for (int32 Index = OrigNumSimVertices2D; Index < OutSim2DToSourceVertex->Num(); ++Index)
			{
				(*OutSim2DToSourceVertex)[Index] += SkeletalMeshModel.Sections[SectionIndex].BaseVertexIndex;
			}
		}
	}

	void FClothDataflowTools::LogAndToastWarning(const FDataflowNode& DataflowNode, const FText& Headline, const FText& Details)
	{
		static const FTextFormat TextFormat = FTextFormat::FromString(TEXT("{0}: {1}\n{2}"));
		const FText NodeName = FText::FromName(DataflowNode.GetName());
		const FText Text = FText::Format(TextFormat, NodeName, Headline, Details);

		FNotificationInfo NotificationInfo(Text);
		NotificationInfo.ExpireDuration = 5.0f;
		FSlateNotificationManager::Get().AddNotification(NotificationInfo);

		UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "%ls", *Text.ToString());
	}

	bool FClothDataflowTools::MakeCollectionName(FString& InOutString)
	{
		const FString SourceString = InOutString;
		InOutString = SlugStringForValidName(InOutString, TEXT("_")).Replace(TEXT("\\"), TEXT("_"));
		bool bCharsWereRemoved;
		do { InOutString.TrimCharInline(TEXT('_'), &bCharsWereRemoved); } while (bCharsWereRemoved);
		return InOutString.Equals(SourceString);
	}

	static void CopyBuildSettings(const FMeshBuildSettings& InStaticMeshBuildSettings, FSkeletalMeshBuildSettings& OutSkeletalMeshBuildSettings)
	{
		OutSkeletalMeshBuildSettings.bRecomputeNormals = InStaticMeshBuildSettings.bRecomputeNormals;
		OutSkeletalMeshBuildSettings.bRecomputeTangents = InStaticMeshBuildSettings.bRecomputeTangents;
		OutSkeletalMeshBuildSettings.bUseMikkTSpace = InStaticMeshBuildSettings.bUseMikkTSpace;
		OutSkeletalMeshBuildSettings.bComputeWeightedNormals = InStaticMeshBuildSettings.bComputeWeightedNormals;
		OutSkeletalMeshBuildSettings.bRemoveDegenerates = InStaticMeshBuildSettings.bRemoveDegenerates;
		OutSkeletalMeshBuildSettings.bUseHighPrecisionTangentBasis = InStaticMeshBuildSettings.bUseHighPrecisionTangentBasis;
		OutSkeletalMeshBuildSettings.bUseFullPrecisionUVs = InStaticMeshBuildSettings.bUseFullPrecisionUVs;
		OutSkeletalMeshBuildSettings.bUseBackwardsCompatibleF16TruncUVs = InStaticMeshBuildSettings.bUseBackwardsCompatibleF16TruncUVs;
		// The rest we leave at defaults.
	}

	bool FClothDataflowTools::BuildSkeletalMeshModelFromMeshDescription(const FMeshDescription* const InMeshDescription, const FMeshBuildSettings& InBuildSettings, FSkeletalMeshLODModel& SkeletalMeshModel)
	{
		// This is following StaticToSkeletalMeshConverter.cpp::AddLODFromStaticMeshSourceModel
		FSkeletalMeshBuildSettings BuildSettings;
		CopyBuildSettings(InBuildSettings, BuildSettings);
		FMeshDescription SkeletalMeshGeometry = *InMeshDescription;
		FSkeletalMeshAttributes SkeletalMeshAttributes(SkeletalMeshGeometry);
		SkeletalMeshAttributes.Register();

		// Full binding to the root bone.
		constexpr int32 RootBoneIndex = 0;
		FSkinWeightsVertexAttributesRef SkinWeights = SkeletalMeshAttributes.GetVertexSkinWeights();
		UE::AnimationCore::FBoneWeight RootInfluence(RootBoneIndex, 1.0f);
		UE::AnimationCore::FBoneWeights RootBinding = UE::AnimationCore::FBoneWeights::Create({ RootInfluence });

		for (const FVertexID VertexID : SkeletalMeshGeometry.Vertices().GetElementIDs())
		{
			SkinWeights.Set(VertexID, RootBinding);
		}

		// Convert static mesh linear to sRGB
		TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = SkeletalMeshAttributes.GetVertexInstanceColors();
		for (const FVertexInstanceID VertexInstanceID : SkeletalMeshGeometry.VertexInstances().GetElementIDs())
		{
			FVector4f VertexColor = VertexInstanceColors.Get(VertexInstanceID);
			VertexColor.X = FMath::Clamp(VertexColor.X, 0.f, 1.f);  // The clamp is required to exactly match the behavior in
			VertexColor.Y = FMath::Clamp(VertexColor.Y, 0.f, 1.f);  // FStaticToSkeletalMeshConverter::InitializeSkeletonFromStaticMesh()
			VertexColor.Z = FMath::Clamp(VertexColor.Z, 0.f, 1.f);  // while allowing to reuse LinearToSRGB
			UE::Geometry::LinearColors::LinearToSRGB(VertexColor);
			VertexInstanceColors.Set(VertexInstanceID, VertexColor);
		}

		FSkeletalMeshImportData SkeletalMeshImportGeometry = FSkeletalMeshImportData::CreateFromMeshDescription(SkeletalMeshGeometry);
		// Data needed by BuildSkeletalMesh
		TArray<FVector3f> LODPoints;
		TArray<SkeletalMeshImportData::FMeshWedge> LODWedges;
		TArray<SkeletalMeshImportData::FMeshFace> LODFaces;
		TArray<SkeletalMeshImportData::FVertInfluence> LODInfluences;
		TArray<int32> LODPointToRawMap;
		SkeletalMeshImportGeometry.CopyLODImportData(LODPoints, LODWedges, LODFaces, LODInfluences, LODPointToRawMap);
		IMeshUtilities::MeshBuildOptions BuildOptions;
		BuildOptions.TargetPlatform = GetTargetPlatformManagerRef().GetRunningTargetPlatform();
		BuildOptions.FillOptions(BuildSettings);
		SkeletalMeshModel.NumTexCoords = SkeletalMeshImportGeometry.NumTexCoords;

		static const FString SkeletalMeshName("ClothAssetStaticMeshImportConvert"); // This is only used by warning messages in the mesh builder.
		// Build a RefSkeleton with just a root bone. The BuildSkeletalMesh code expects you have a reference skeleton with at least one bone to work.
		FReferenceSkeleton RootBoneRefSkeleton;
		FReferenceSkeletonModifier SkeletonModifier(RootBoneRefSkeleton, nullptr);
		FMeshBoneInfo RootBoneInfo;
		RootBoneInfo.Name = FName("Root");
		SkeletonModifier.Add(RootBoneInfo, FTransform());
		RootBoneRefSkeleton.RebuildRefSkeleton(nullptr, true);

		IMeshUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshUtilities>("MeshUtilities");
		TArray<FText> WarningMessages;
		if (!MeshUtilities.BuildSkeletalMesh(SkeletalMeshModel, SkeletalMeshName, RootBoneRefSkeleton, LODInfluences, LODWedges, LODFaces, LODPoints, LODPointToRawMap, BuildOptions, &WarningMessages))
		{
			for (const FText& Message : WarningMessages)
			{
				UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "%ls", *Message.ToString());
			}
			return false;
		}
		return true;
	}

	FDataflowNode* FClothDataflowTools::GetPropertyOwnerDataflowNode(const TSharedPtr<IPropertyHandle>& PropertyHandle, const UStruct* DataflowNodeStruct)
	{
		for (TSharedPtr<IPropertyHandle> OwnerHandle = PropertyHandle->GetParentHandle(); OwnerHandle; OwnerHandle = OwnerHandle->GetParentHandle())
		{
			if (const TSharedPtr<IPropertyHandleStruct> OwnerHandleStruct = OwnerHandle->AsStruct())
			{
				if (TSharedPtr<FStructOnScope> StructOnScope = OwnerHandleStruct->GetStructData())
				{
					if (StructOnScope->GetStruct()->IsChildOf(DataflowNodeStruct))
					{
						return reinterpret_cast<FDataflowNode*>(StructOnScope->GetStructMemory());
					}
				}
			}
		}
		return nullptr;
	}

	FClothDataflowTools::FSimMeshCleanup::FSimMeshCleanup(const TArray<FIntVector3>& InTriangleToVertexIndex, const TArray<FVector2f>& InRestPositions2D, const TArray<FVector3f>& InDrapedPositions3D)
		: TriangleToVertexIndex(InTriangleToVertexIndex)
		, RestPositions2D(InRestPositions2D)
		, DrapedPositions3D(InDrapedPositions3D)
	{
		check(RestPositions2D.Num() == DrapedPositions3D.Num());

		OriginalTriangles.SetNum(TriangleToVertexIndex.Num());
		for (int32 Index = 0; Index < OriginalTriangles.Num(); ++Index)
		{
			OriginalTriangles[Index].Add(Index);
		}
		OriginalVertices.SetNum(DrapedPositions3D.Num());
		for (int32 Index = 0; Index < OriginalVertices.Num(); ++Index)
		{
			OriginalVertices[Index].Add(Index);
		}
	}

	bool FClothDataflowTools::FSimMeshCleanup::RemoveDegenerateTriangles()
	{
		check(RestPositions2D.Num() == DrapedPositions3D.Num());

		bool bHasDegenerateTriangles = false;

		const int32 VertexCount = RestPositions2D.Num();

		// Remap[Index] is the index of the first vertex in a group of degenerated triangles to be callapsed.
		// When two groups of collapsed vertices are merged, the group with the greatest Remap[index] value must adopt the one from the other group.
		// For Example:
		// 1. For all i, Remap[i] = i
		// 2. Finds one degenerated triangle (7, 9, 4) with collapsed edges (7, 9), (9, 4), and (7, 4) -> Remap[4] = 4, Remap[7] = 4, and Remap[9] = 4
		// 3. Finds another degenerated triangle (2, 3, 4) with collapsed edges (2, 4) -> Remap[2] = 2, Remap[4] = 2, Remap[7] = 2, and Remap[9] = 2
		TArray<int32> Remap;
		Remap.SetNumUninitialized(VertexCount);

		for (int32 Index = 0; Index < VertexCount; ++Index)
		{
			Remap[Index] = Index;
		}

		int32 OutVertexCount = VertexCount;

		auto RemapAndPropagateIndex = [&Remap, &OutVertexCount](int32 Index0, int32 Index1)
			{
				if (Remap[Index0] != Remap[Index1])
				{
					if (Remap[Index0] > Remap[Index1])  // Always remap from the lowest index to ensure the earlier index is always kept
					{
						Swap(Index0, Index1);
					}
					// Merge groups with this new first index Remap[Index0]
					const int32 PrevRemapIndex = Remap[Index1];
					for (int32 Index = PrevRemapIndex; Index < Remap.Num(); ++Index)  // Only need to start from the first index of the group to merge
					{
						if (Remap[Index] == PrevRemapIndex)
						{
							Remap[Index] = Remap[Index0];
						}
					}
					--OutVertexCount;
				}
			};

		const int32 TriangleCount = TriangleToVertexIndex.Num();
		TArray<FIntVector3> OutTriangleToVertexIndex;
		OutTriangleToVertexIndex.Reserve(TriangleCount);
		TArray<TSet<int32>> OutOriginalTriangles;
		OutOriginalTriangles.Reserve(TriangleCount);

		for (int32 TriangleIndex = 0; TriangleIndex < TriangleCount; ++TriangleIndex)
		{
			const int32 Index0 = TriangleToVertexIndex[TriangleIndex][0];
			const int32 Index1 = TriangleToVertexIndex[TriangleIndex][1];
			const int32 Index2 = TriangleToVertexIndex[TriangleIndex][2];

			const FVector3f& P0 = DrapedPositions3D[Index0];
			const FVector3f& P1 = DrapedPositions3D[Index1];
			const FVector3f& P2 = DrapedPositions3D[Index2];
			const FVector3f P0P1 = P1 - P0;
			const FVector3f P0P2 = P2 - P0;

			const float TriNormSizeSquared = (P0P1 ^ P0P2).SizeSquared();
			if (TriNormSizeSquared <= UE_SMALL_NUMBER)
			{
				const FVector3f P1P2 = P2 - P1;

				if (P0P1.SquaredLength() <= UE_SMALL_NUMBER)
				{
					RemapAndPropagateIndex(Index0, Index1);
				}
				if (P0P2.SquaredLength() <= UE_SMALL_NUMBER)
				{
					RemapAndPropagateIndex(Index0, Index2);
				}
				if (P1P2.SquaredLength() <= UE_SMALL_NUMBER)
				{
					RemapAndPropagateIndex(Index1, Index2);
				}
			}
			else
			{
				OutTriangleToVertexIndex.Emplace(TriangleToVertexIndex[TriangleIndex]);
				OutOriginalTriangles.Emplace(OriginalTriangles[TriangleIndex]);
			}
		}

		TriangleToVertexIndex = MoveTemp(OutTriangleToVertexIndex);
		OriginalTriangles = MoveTemp(OutOriginalTriangles);

		const int32 OutTriangleCount = TriangleToVertexIndex.Num();
		bHasDegenerateTriangles = (TriangleCount != OutTriangleCount);

		UE_CLOGF(bHasDegenerateTriangles, LogChaosClothAssetDataflowNodes, Display,
			"USD import found and removed %d degenerated triangles out of %d source triangles.", TriangleCount - OutTriangleCount, TriangleCount);

		// Reconstruct vertices
		TArray<FVector2f> OutRestPositions2D;
		OutRestPositions2D.Reserve(OutVertexCount);
		TArray<FVector3f> OutDrapedPositions3D;
		OutDrapedPositions3D.Reserve(OutVertexCount);
		TArray<TSet<int32>> OutOriginalVertices;
		OutOriginalVertices.Reserve(OutVertexCount);
		TArray<int32> OutIndices;
		OutIndices.Reserve(VertexCount);
		int32 OutIndex = -1;

		for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			if (Remap[VertexIndex] == VertexIndex)
			{
				OutRestPositions2D.Emplace(RestPositions2D[VertexIndex]);
				OutDrapedPositions3D.Emplace(DrapedPositions3D[VertexIndex]);
				OutOriginalVertices.Emplace(OriginalVertices[VertexIndex]);
				OutIndices.Emplace(++OutIndex);
			}
			else
			{
				const int32 OutRemappedIndex = OutIndices[Remap[VertexIndex]];
				OutOriginalVertices[OutRemappedIndex].Append(OriginalVertices[VertexIndex]);
				OutIndices.Emplace(OutRemappedIndex);
			}
		}
		ensure(OutIndex + 1 == OutVertexCount);

		RestPositions2D = MoveTemp(OutRestPositions2D);
		DrapedPositions3D = MoveTemp(OutDrapedPositions3D);
		OriginalVertices = MoveTemp(OutOriginalVertices);

		// Remap final triangles
		for (int32 TriangleIndex = 0; TriangleIndex < OutTriangleCount; ++TriangleIndex)
		{
			int32& Index0 = TriangleToVertexIndex[TriangleIndex][0];
			int32& Index1 = TriangleToVertexIndex[TriangleIndex][1];
			int32& Index2 = TriangleToVertexIndex[TriangleIndex][2];

			Index0 = OutIndices[Index0];
			Index1 = OutIndices[Index1];
			Index2 = OutIndices[Index2];

			checkSlow(Index0 != Index1);
			checkSlow(Index0 != Index2);
			checkSlow(Index1 != Index2);
			checkSlow((OutDrapedPositions3D[Index0] - OutDrapedPositions3D[Index1]).SquaredLength() > UE_SMALL_NUMBER);
			checkSlow((OutDrapedPositions3D[Index0] - OutDrapedPositions3D[Index2]).SquaredLength() > UE_SMALL_NUMBER);
			checkSlow((OutDrapedPositions3D[Index1] - OutDrapedPositions3D[Index2]).SquaredLength() > UE_SMALL_NUMBER);
		}

		return bHasDegenerateTriangles;
	}

	bool FClothDataflowTools::FSimMeshCleanup::RemoveDuplicateTriangles()
	{
		bool bHasDuplicatedTriangles = false;

		const int32 TriangleCount = TriangleToVertexIndex.Num();

		TMap<FIntVector3, int32> Triangles;
		Triangles.Reserve(TriangleCount);

		TArray<FIntVector3> OutTriangleToVertexIndex;
		OutTriangleToVertexIndex.Reserve(TriangleCount);
		TArray<TSet<int32>> OutOriginalTriangles;
		OutOriginalTriangles.Reserve(TriangleCount);

		auto GetSortedIndices = [](const FIntVector3& TriangleIndices)->FIntVector3
			{
				const int32 Index0 = TriangleIndices[0];
				const int32 Index1 = TriangleIndices[1];
				const int32 Index2 = TriangleIndices[2];

				return (Index0 < Index1) ?
					(Index1 < Index2) ? FIntVector3(Index0, Index1, Index2) : (Index0 < Index2) ? FIntVector3(Index0, Index2, Index1) : FIntVector3(Index2, Index0, Index1) :
					(Index0 < Index2) ? FIntVector3(Index1, Index0, Index2) : (Index1 < Index2) ? FIntVector3(Index1, Index2, Index0) : FIntVector3(Index2, Index1, Index0);
			};

		for (int32 Index = 0; Index < TriangleCount; ++Index)
		{
			const FIntVector3& TriangleIndices = TriangleToVertexIndex[Index];
			const FIntVector3 TriangleSortedIndices = GetSortedIndices(TriangleIndices);

			if (int32* NewTriangle = Triangles.Find(TriangleSortedIndices))
			{
				bHasDuplicatedTriangles = true;
				OutOriginalTriangles[*NewTriangle].Append(OriginalTriangles[Index]);
			}
			else
			{
				NewTriangle = &Triangles.Emplace(TriangleSortedIndices);
				*NewTriangle = OutTriangleToVertexIndex.Emplace(TriangleIndices);
				OutOriginalTriangles.Emplace(OriginalTriangles[Index]);
			}
		}

		TriangleToVertexIndex = MoveTemp(OutTriangleToVertexIndex);
		OriginalTriangles = MoveTemp(OutOriginalTriangles);

		UE_CLOGF(bHasDuplicatedTriangles, LogChaosClothAssetDataflowNodes, Display,
			"USD import found and removed %d duplicated triangles out of %d source triangles.", TriangleCount - TriangleToVertexIndex.Num(), TriangleCount);

		return bHasDuplicatedTriangles;
	}

	template<typename T UE_REQUIRES_DEFINITION(std::is_same_v<T, TArray<int32>> || std::is_same_v<T, TSet<int32>>)>
	TArray<int32> FClothDataflowTools::GetOriginalToNewIndices(const TConstArrayView<T>& NewToOriginals, int32 NumOriginalIndices)
	{
		TArray<int32> OriginalToNewIndices;
		OriginalToNewIndices.Init(INDEX_NONE, NumOriginalIndices);

		for (int32 NewIndex = 0; NewIndex < NewToOriginals.Num(); ++NewIndex)
		{
			for (const int32 OriginalIndex : NewToOriginals[NewIndex])
			{
				check(OriginalToNewIndices.IsValidIndex(OriginalIndex));
				check(OriginalToNewIndices[OriginalIndex] == INDEX_NONE);
				OriginalToNewIndices[OriginalIndex] = NewIndex;
			}
		}
		return OriginalToNewIndices;
	}
	template CHAOSCLOTHASSETDATAFLOWNODES_API TArray<int32> FClothDataflowTools::GetOriginalToNewIndices<TArray<int32>>(const TConstArrayView<TArray<int32>>& NewToOriginals, int32 NumOriginalIndices);
	template CHAOSCLOTHASSETDATAFLOWNODES_API TArray<int32> FClothDataflowTools::GetOriginalToNewIndices<TSet<int32>>(const TConstArrayView<TSet<int32>>& NewToOriginals, int32 NumOriginalIndices);

	bool FClothDataflowTools::RemoveDegenerateTriangles(
		const TArray<FIntVector3>& TriangleToVertexIndex,
		const TArray<FVector2f>& RestPositions2D,
		const TArray<FVector3f>& DrapedPositions3D,
		TArray<FIntVector3>& OutTriangleToVertexIndex,
		TArray<FVector2f>& OutRestPositions2D,
		TArray<FVector3f>& OutDrapedPositions3D,
		TArray<int32>& OutIndices)
	{
		FSimMeshCleanup SimMeshCleanup(TriangleToVertexIndex, RestPositions2D, DrapedPositions3D);
		const bool bHasDegenerateTriangles = SimMeshCleanup.RemoveDegenerateTriangles();
		OutIndices = GetOriginalToNewIndices<TSet<int32>>(SimMeshCleanup.OriginalVertices, DrapedPositions3D.Num());
		OutTriangleToVertexIndex = MoveTemp(SimMeshCleanup.TriangleToVertexIndex);
		OutRestPositions2D = MoveTemp(SimMeshCleanup.RestPositions2D);
		OutDrapedPositions3D = MoveTemp(SimMeshCleanup.DrapedPositions3D);
		return bHasDegenerateTriangles;
	}

	bool FClothDataflowTools::RemoveDuplicateStitches(TArray<TArray<FIntVector2>>& SeamStitches)
	{
		bool bHasDuplicateStitches = false;

		const int32 NumSeamStitches = SeamStitches.Num();

		// Calculate the total number of stitches
		int32 NumStitches = 0;
		for (const TArray<FIntVector2>& Stitches : SeamStitches)
		{
			NumStitches += Stitches.Num();
		}

		TSet<FIntVector2> StichSet;
		StichSet.Reserve(NumStitches);

		int32 OutNumStitches = 0;
		TArray<TArray<FIntVector2>> OutSeamStitches;
		OutSeamStitches.Reserve(NumSeamStitches);

		for (const TArray<FIntVector2>& Stitches : SeamStitches)
		{
			TArray<FIntVector2> OutStitches;
			OutStitches.Reserve(Stitches.Num());

			for (const FIntVector2& Stitch : Stitches)
			{
				const FIntVector2 SortedStitch = Stitch[0] < Stitch[1] ?
					FIntVector2(Stitch[0], Stitch[1]) :
					FIntVector2(Stitch[1], Stitch[0]);

				bool bIsAlreadyInSet;
				StichSet.FindOrAdd(SortedStitch, &bIsAlreadyInSet);

				if (bIsAlreadyInSet)
				{
					bHasDuplicateStitches = true;
				}
				else
				{
					OutStitches.Emplace(Stitch);
				}
			}

			if (OutStitches.Num())
			{
				OutSeamStitches.Emplace(OutStitches);
				OutNumStitches += OutStitches.Num();
			}
		}

		UE_CLOGF(bHasDuplicateStitches, LogChaosClothAssetDataflowNodes, Display,
			"USD import found and removed %d duplicated stitches out of %d source stitches.", NumStitches - OutNumStitches, NumStitches);

		SeamStitches = MoveTemp(OutSeamStitches);

		return bHasDuplicateStitches;
	}

	void FClothDataflowTools::SetGroupResizingData(const TSharedRef<FManagedArrayCollection>& ClothCollection, const TConstArrayView<FName>& SetNames, const TConstArrayView<int32>& SetTypes)
	{
		check(SetNames.Num() == SetTypes.Num());
		FCollectionClothFacade ClothFacade(ClothCollection);

		ClothFacade.DefineSchema(EClothCollectionExtendedSchemas::Resizing);

		// Clear the sim and render blends
		TArrayView<float> SimBlend = ClothFacade.GetSimCustomResizingBlend();
		TArrayView<float> RenderBlend = ClothFacade.GetRenderCustomResizingBlend();
		for (int32 Index = 0; Index < SimBlend.Num(); ++Index)
		{
			SimBlend[Index] = 0.f;
		}
		for (int32 Index = 0; Index < RenderBlend.Num(); ++Index)
		{
			RenderBlend[Index] = 0.f;
		}

		// Filter out invalid sets
		TArray<FName> FilteredSetNames;
		TArray<int32> FilteredSetTypes;
		FilteredSetNames.Reserve(SetNames.Num());
		FilteredSetTypes.Reserve(SetNames.Num());
		TSet<int32> SetValues;
		for (int32 Index = 0; Index < SetNames.Num(); ++Index)
		{
			// Try to get the set as a SimVertex3D set
			if (FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, SetNames[Index], ClothCollectionGroup::SimVertices3D, SetValues))
			{
				FilteredSetNames.Add(SetNames[Index]);
				FilteredSetTypes.Add(SetTypes[Index]);

				for (const int32 Vertex : SetValues)
				{
					if (SimBlend.IsValidIndex(Vertex))
					{
						SimBlend[Vertex] = 1.f;
					}
				}
			}
			else if (FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, SetNames[Index], ClothCollectionGroup::RenderVertices, SetValues))
			{
				FilteredSetNames.Add(SetNames[Index]);
				FilteredSetTypes.Add(SetTypes[Index]);

				for (const int32 Vertex : SetValues)
				{
					if (RenderBlend.IsValidIndex(Vertex))
					{
						RenderBlend[Vertex] = 1.f;
					}
				}
			}
		}

		check(FilteredSetNames.Num() == FilteredSetTypes.Num());
		ClothFacade.SetNumCustomResizingRegions(FilteredSetNames.Num());
		TArrayView<FString> ClothSetNames = ClothFacade.GetCustomResizingRegionSet();
		TArrayView<int32> ClothSetTypes = ClothFacade.GetCustomResizingRegionType();
		for (int32 Index = 0; Index < FilteredSetNames.Num(); ++Index)
		{
			ClothSetNames[Index] = FilteredSetNames[Index].ToString();
			ClothSetTypes[Index] = FilteredSetTypes[Index];
		}
	}

	void FClothDataflowTools::GenerateSimMeshResizingGroupData(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FMeshDescription& SourceMeshDescription, TArray<FMeshResizingCustomRegion>& OutResizingGroupData)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid(EClothCollectionExtendedSchemas::Resizing))
		{
			UE::Geometry::FDynamicMesh3 SourceMesh;
			FMeshDescriptionToDynamicMesh Converter;
			Converter.Convert(&SourceMeshDescription, SourceMesh, true);

			TConstArrayView<FString> ClothSetNames = ClothFacade.GetCustomResizingRegionSet();
			TConstArrayView<int32> ClothSetTypes = ClothFacade.GetCustomResizingRegionType();
			OutResizingGroupData.SetNum(ClothSetNames.Num());
			for (int32 GroupIndex = 0; GroupIndex < ClothSetNames.Num(); ++GroupIndex)
			{
				TSet<int32> SetValues;
				if (FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, FName(ClothSetNames[GroupIndex]), ClothCollectionGroup::SimVertices3D, SetValues))
				{
					UE::MeshResizing::FCustomRegionResizing::GenerateCustomRegion(ClothFacade.GetSimPosition3D(), SourceMesh, SetValues, OutResizingGroupData[GroupIndex]);
				}
			}
		}
	}

	void FClothDataflowTools::GenerateRenderMeshResizingGroupData(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FMeshDescription& SourceMeshDescription, TArray<FMeshResizingCustomRegion>& OutResizingGroupData)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid(EClothCollectionExtendedSchemas::Resizing))
		{
			UE::Geometry::FDynamicMesh3 SourceMesh;
			bool bIsSourceMeshValid = false;

			TConstArrayView<FString> ClothSetNames = ClothFacade.GetCustomResizingRegionSet();
			TConstArrayView<int32> ClothSetTypes = ClothFacade.GetCustomResizingRegionType();
			OutResizingGroupData.SetNum(ClothSetNames.Num());
			for (int32 GroupIndex = 0; GroupIndex < ClothSetNames.Num(); ++GroupIndex)
			{
				TSet<int32> SetValues;
				if (FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, FName(ClothSetNames[GroupIndex]), ClothCollectionGroup::RenderVertices, SetValues))
				{
					if (!bIsSourceMeshValid)
					{
						FMeshDescriptionToDynamicMesh Converter;
						Converter.Convert(&SourceMeshDescription, SourceMesh, true);
						bIsSourceMeshValid = true;
					}

					UE::MeshResizing::FCustomRegionResizing::GenerateCustomRegion(ClothFacade.GetRenderPosition(), SourceMesh, SetValues, OutResizingGroupData[GroupIndex]);
				}
			}
		}
	}

	void FClothDataflowTools::ApplySimGroupResizing(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FMeshDescription& TargetMeshDescription, const FMeshResizingRBFInterpolationData& InterpolationData, const TArray<FMeshResizingCustomRegion>& ResizingGroupData)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);
		Private::Resizing::ApplyGroupResizing(ClothFacade, TargetMeshDescription, InterpolationData, ResizingGroupData, ClothFacade.GetSimPosition3D(), ClothFacade.GetSimCustomResizingBlend());
	}

	void FClothDataflowTools::ApplyRenderGroupResizing(const TSharedRef<FManagedArrayCollection>& ClothCollection, const FMeshDescription& TargetMeshDescription, const FMeshResizingRBFInterpolationData& InterpolationData, const TArray<FMeshResizingCustomRegion>& ResizingGroupData)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);
		Private::Resizing::ApplyGroupResizing(ClothFacade, TargetMeshDescription, InterpolationData, ResizingGroupData, ClothFacade.GetRenderPosition(), ClothFacade.GetRenderCustomResizingBlend());
	}
}  // End namespace UE::Chaos::ClothAsset
