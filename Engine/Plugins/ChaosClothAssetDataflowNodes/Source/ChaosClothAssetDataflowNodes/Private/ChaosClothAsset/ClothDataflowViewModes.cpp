// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothDataflowViewModes.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetDataflowViewModes"

namespace UE::Chaos::ClothAsset
{
	const FName FCloth2DSimViewMode::Name = FName("Cloth2DSimView");

	FName FCloth2DSimViewMode::GetName() const
	{
		return Name;
	}

	FText FCloth2DSimViewMode::GetButtonText() const
	{
		return FText(LOCTEXT("Cloth2DSimViewButtonText", "2DSim"));
	}

	FText FCloth2DSimViewMode::GetTooltipText() const
	{
		return FText(LOCTEXT("Cloth2DSimViewTooltipText", "2D Simulation Mesh"));
	}

	FRotator FCloth2DSimViewMode::GetViewportOrientation() const
	{
		return FRotator(90, -90, 0);
	}

	const FName FCloth3DSimViewMode::Name = FName("Cloth3DSimView");

	FName FCloth3DSimViewMode::GetName() const
	{
		return Name;
	}

	FText FCloth3DSimViewMode::GetButtonText() const
	{
		return FText(LOCTEXT("Cloth3DSimViewButtonText", "3DSim"));
	}

	FText FCloth3DSimViewMode::GetTooltipText() const
	{
		return FText(LOCTEXT("Cloth3DSimViewTooltipText", "3D Simulation Mesh"));
	}

	const FName FClothRenderViewMode::Name = FName("ClothRenderView");

	FName FClothRenderViewMode::GetName() const
	{
		return Name;
	}

	FText FClothRenderViewMode::GetButtonText() const
	{
		return FText(LOCTEXT("ClothRenderViewButtonText", "Render"));
	}

	FText FClothRenderViewMode::GetTooltipText() const
	{
		return FText(LOCTEXT("ClothRenderViewTooltipText", "3D Render Mesh"));
	}

}  // End namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE 
