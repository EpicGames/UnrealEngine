// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowRenderingViewMode.h"

namespace UE::Chaos::ClothAsset
{
	class FCloth2DSimViewMode : public UE::Dataflow::FDataflowConstruction2DViewModeBase
	{
	public:
		static const FName Name;
		virtual ~FCloth2DSimViewMode() = default;
	private:
		virtual FName GetName() const override;
		virtual FText GetButtonText() const override;
		virtual FText GetTooltipText() const override;
		virtual FRotator GetViewportOrientation() const override;
	};

	class FCloth3DSimViewMode : public UE::Dataflow::FDataflowConstruction3DViewModeBase
	{
	public:
		static const FName Name;
		virtual ~FCloth3DSimViewMode() = default;
	private:
		virtual FName GetName() const override;
		virtual FText GetButtonText() const override;
		virtual FText GetTooltipText() const override;
		virtual bool CanDisplayGizmo() const { return true; }
	};

	class FClothRenderViewMode : public UE::Dataflow::FDataflowConstruction3DViewMode
	{
	public:
		static const FName Name;
		virtual ~FClothRenderViewMode() = default;
	private:
		virtual FName GetName() const override;
		virtual FText GetButtonText() const override;
		virtual FText GetTooltipText() const override;
		virtual bool CanDisplayGizmo() const { return true; }
	};

}  // End namespace UE::Chaos::ClothAsset


