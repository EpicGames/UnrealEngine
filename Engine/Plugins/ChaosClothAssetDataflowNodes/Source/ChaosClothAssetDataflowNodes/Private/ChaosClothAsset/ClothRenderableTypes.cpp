// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothRenderableTypes.h"

#include "Components/DynamicMeshComponent.h"
#include "Drawing/LineSetComponent.h"
#include "Drawing/PointSetComponent.h"

#include "Dataflow/DataflowRenderingViewMode.h"
#include "DataflowRendering/DataflowRenderableComponents.h"
#include "DataflowRendering/DataflowRenderableType.h"
#include "DataflowRendering/DataflowRenderableTypeInstance.h"
#include "DataflowRendering/DataflowRenderableTypeRegistry.h"
#include "DataflowRendering/DataflowRenderableTypeUtils.h"

#include "GeometryCollection/ManagedArrayCollection.h"

#include "UObject/ObjectPtr.h"

#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowViewModes.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/ClothPatternToDynamicMesh.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"

#include "Materials/Material.h"

#define LOCTEXT_NAMESPACE "ClothRenderableTypes"

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		static UMaterialInterface* GetTwoSidedMaterialForRenderingTypes()
		{
			return Cast<UMaterial>(StaticLoadObject(UMaterial::StaticClass(), NULL, TEXT("/Engine/EditorMaterials/Dataflow/DataflowTwoSidedVertexMaterial")));
		}

		static UMaterialInterface* GetTwoSidedMaterialForColoredPatterns()
		{
			return Cast<UMaterial>(StaticLoadObject(UMaterial::StaticClass(), NULL, TEXT("/MeshModelingToolsetExp/Materials/MeshVertexColorMaterialTwoSided")));
		}

		static FLinearColor MakePseudoRandomColor(int32 NumColorRotations)
		{
			constexpr uint8 Spread = 157;  // Prime number that gives a good spread of colors without getting too similar as a rand might do.
			uint8 Seed = Spread;
			for (int32 Rotation = 0; Rotation < NumColorRotations; ++Rotation)
			{
				Seed += Spread;
			}
			return FLinearColor::MakeFromHSV8(Seed, 180, 140);
		}

		static void AddPatternsColorsToMesh(const FCollectionClothConstFacade& ClothFacade, FDynamicMesh3& InOutMesh)
		{
			using namespace UE::Geometry;

			checkf(InOutMesh.TriangleCount() == ClothFacade.GetNumSimFaces(), TEXT("Expected to have the same number of faces in LodMesh and ClothCollection sim mesh"));

			if (!InOutMesh.Attributes()->HasPrimaryColors())
			{
				InOutMesh.Attributes()->EnablePrimaryColors();
				InOutMesh.Attributes()->PrimaryColors()->CreateFromPredicate([](int ParentVID, int TriIDA, int TriIDB)
					{
						return false; // make sure we have a value per corner
					}
				, 0.0f);
			}


			if (FDynamicMeshColorOverlay* const ColorAttributeLayer = InOutMesh.Attributes()->PrimaryColors())
			{
				for (int32 PatternID = 0; PatternID < ClothFacade.GetNumSimPatterns(); ++PatternID)
				{
					const FCollectionClothSimPatternConstFacade Pattern = ClothFacade.GetSimPattern(PatternID);
					const FLinearColor PatternColor = MakePseudoRandomColor(PatternID);

					for (int32 TriID = 0; TriID < Pattern.GetNumSimFaces(); ++TriID)
					{
						const int32 GlobalTriID = Pattern.GetSimFacesOffset() + TriID;
						FIndex3i AttrTri;
						if (ColorAttributeLayer->GetTriangleIfValid(GlobalTriID, AttrTri))
						{
							ColorAttributeLayer->SetElement(AttrTri[0], (FVector4f)PatternColor);
							ColorAttributeLayer->SetElement(AttrTri[1], (FVector4f)PatternColor);
							ColorAttributeLayer->SetElement(AttrTri[2], (FVector4f)PatternColor);
						}
					}
				}
			}
		}

		static void AddAttributeAsColorToMesh(TConstArrayView<FIntVector> Triangles, FDynamicMesh3& InOutMesh, TConstArrayView<float> AttributeValue)
		{
			using namespace UE::Geometry;

			checkf(InOutMesh.VertexCount() == AttributeValue.Num(), TEXT("Expected to have the same number of vertices in LodMesh and ClothCollection mesh"));

			if (!InOutMesh.Attributes()->HasPrimaryColors())
			{
				InOutMesh.Attributes()->EnablePrimaryColors();
				InOutMesh.Attributes()->PrimaryColors()->CreateFromPredicate([](int ParentVID, int TriIDA, int TriIDB)
					{
						return false; // make sure we have a value per corner
					}
				, 0.0f);
			}


			if (FDynamicMeshColorOverlay* const ColorAttributeLayer = InOutMesh.Attributes()->PrimaryColors())
			{
				auto GetVertexColor = [&AttributeValue](int32 VtxIdx)
					{
						if (AttributeValue.IsValidIndex(VtxIdx))
						{
							const float Value = AttributeValue[VtxIdx];
							return FVector4f{ Value , Value , Value, 1.0f };
						}
						return FVector4f{ 1, 0, 0, 1.0f };
					};

				for (int32 TriID = 0; TriID < Triangles.Num(); ++TriID)
				{
					FIndex3i AttrTri;
					if (ColorAttributeLayer->GetTriangleIfValid(TriID, AttrTri))
					{
						const int32 VtxIdx0 = ColorAttributeLayer->GetParentVertex(AttrTri[0]);
						const int32 VtxIdx1 = ColorAttributeLayer->GetParentVertex(AttrTri[1]);
						const int32 VtxIdx2 = ColorAttributeLayer->GetParentVertex(AttrTri[2]);

						ColorAttributeLayer->SetElement(AttrTri[0], GetVertexColor(VtxIdx0));
						ColorAttributeLayer->SetElement(AttrTri[1], GetVertexColor(VtxIdx1));
						ColorAttributeLayer->SetElement(AttrTri[2], GetVertexColor(VtxIdx2));
					}
				}
			}
		}

		static bool AddFaceSelectionToMesh(const TSharedRef<const FManagedArrayCollection>& Collection, FName SelectionName, FDynamicMesh3& InOutMesh)
		{
			using namespace UE::Geometry;

			if (SelectionName.IsNone())
			{
				return false;
			}

			const FCollectionClothSelectionConstFacade SelectionFacade(Collection);
			if (!SelectionFacade.IsValid())
			{
				return false;
			}

			const TSet<int32>* SelectionSet = SelectionFacade.FindSelectionSet(SelectionName);
			if (!SelectionSet)
			{
				return false;
			}
			
			const FName& SelectionGroup = SelectionFacade.GetSelectionGroup(SelectionName);
			if (SelectionGroup != ClothCollectionGroup::SimFaces)
			{
				return false;
			}

			if (!InOutMesh.Attributes()->HasPrimaryColors())
			{
				InOutMesh.Attributes()->EnablePrimaryColors();
				InOutMesh.Attributes()->PrimaryColors()-> CreateFromPredicate([](int ParentVID, int TriIDA, int TriIDB)
					{
						return true; // make sure we have a value per corner
					}
				, 0.0f);
			}

			const FLinearColor SelectionColor = FLinearColor::Yellow;

			if (FDynamicMeshColorOverlay* const ColorAttributeLayer = InOutMesh.Attributes()->PrimaryColors())
			{
				for (const int32 FaceID: *SelectionSet)
				{
					FIndex3i AttrTri;
					if (ColorAttributeLayer->GetTriangleIfValid(FaceID, AttrTri))
					{
						ColorAttributeLayer->SetElement(AttrTri[0], (FVector4f)SelectionColor);
						ColorAttributeLayer->SetElement(AttrTri[1], (FVector4f)SelectionColor);
						ColorAttributeLayer->SetElement(AttrTri[2], (FVector4f)SelectionColor);
					}
				}
			}

			return true;
		}

		static void GetVertexSelectionForSim3D(const TSharedRef<const FManagedArrayCollection>& Collection, const FCollectionClothConstFacade& ClothFacade, FName SelectionName, TArray<FVector>& OutPoints)
		{
			using namespace UE::Geometry;

			OutPoints.Reset();

			if (SelectionName.IsNone())
			{
				return;
			}

			const FCollectionClothSelectionConstFacade SelectionFacade(Collection);
			if (!SelectionFacade.IsValid())
			{
				return;
			}

			const TSet<int32>* SelectionSet = SelectionFacade.FindSelectionSet(SelectionName);
			if (!SelectionSet)
			{
				return;
			}

			OutPoints.Reserve(SelectionSet->Num());

			const FName& SelectionGroup = SelectionFacade.GetSelectionGroup(SelectionName);
			if (SelectionGroup == ClothCollectionGroup::SimVertices3D)
			{
				TConstArrayView<FVector3f> Sim3DPositions = ClothFacade.GetSimPosition3D();
				for (const int32 VertexIndex : *SelectionSet)
				{
					if (Sim3DPositions.IsValidIndex(VertexIndex))
					{
						OutPoints.Add(FVector(Sim3DPositions[VertexIndex]));
					}
				}
			}
			else if (SelectionGroup == ClothCollectionGroup::SimVertices2D)
			{
				TConstArrayView<FVector3f> Sim3DPositions = ClothFacade.GetSimPosition3D();
				const TConstArrayView<int32> Vertex2DTo3D = ClothFacade.GetSimVertex3DLookup();
				for (const int32 VertexIndex : *SelectionSet)
				{
					if (Vertex2DTo3D.IsValidIndex(VertexIndex))
					{
						const int32 VertexIndexIn3D = Vertex2DTo3D[VertexIndex];
						if (Sim3DPositions.IsValidIndex(VertexIndexIn3D))
						{
							OutPoints.Add(FVector(Sim3DPositions[VertexIndexIn3D]));
						}
					}
				}
			}
		}

		static void GetVertexSelectionForSim2D(const TSharedRef<const FManagedArrayCollection>& Collection, const FCollectionClothConstFacade& ClothFacade, FName SelectionName, TArray<FVector>& OutPoints)
		{
			using namespace UE::Geometry;

			OutPoints.Reset();

			if (SelectionName.IsNone())
			{
				return;
			}

			const FCollectionClothSelectionConstFacade SelectionFacade(Collection);
			if (!SelectionFacade.IsValid())
			{
				return;
			}

			const TSet<int32>* SelectionSet = SelectionFacade.FindSelectionSet(SelectionName);
			if (!SelectionSet)
			{
				return;
			}

			OutPoints.Reserve(SelectionSet->Num());

			const FName& SelectionGroup = SelectionFacade.GetSelectionGroup(SelectionName);
			if (SelectionGroup == ClothCollectionGroup::SimVertices3D)
			{
				TConstArrayView<FVector2f> Sim2DPositions = ClothFacade.GetSimPosition2D();
				const TConstArrayView<TArray<int32>> Vertex3DTo2D = ClothFacade.GetSimVertex2DLookup();
				for (const int32 VertexIndex : *SelectionSet)
				{
					if (Vertex3DTo2D.IsValidIndex(VertexIndex))
					{
						for (const int32 VertexIndexIn2D : Vertex3DTo2D[VertexIndex])
						{
							if (Sim2DPositions.IsValidIndex(VertexIndexIn2D))
							{
								const FVector2f PositionIn2D = Sim2DPositions[VertexIndexIn2D];
								OutPoints.Add(FVector(PositionIn2D.X, PositionIn2D.Y, 0.f));
							}
						}
					}
				}
			}
			else if (SelectionGroup == ClothCollectionGroup::SimVertices2D)
			{
				TConstArrayView<FVector2f> Sim2DPositions = ClothFacade.GetSimPosition2D();
				for (const int32 VertexIndex : *SelectionSet)
				{
					if (Sim2DPositions.IsValidIndex(VertexIndex))
					{
						const FVector2f PositionIn2D = Sim2DPositions[VertexIndex];
						OutPoints.Add(FVector(PositionIn2D.X, PositionIn2D.Y, 0.f));
					}
				}
			}
		}

		static void GenerateSeamsForSim2D(
			const TSharedRef<const FManagedArrayCollection>& Collection,
			const FCollectionClothConstFacade& ClothFacade,
			const UE::Geometry::FDynamicMesh3& Mesh,
			UPointSetComponent* PointComponent, const float PointSize,
			ULineSetComponent* LineComponent, const float LineThickness,
			bool bSeamsCollapse
		)
		{
			if (PointComponent == nullptr && LineComponent == nullptr)
			{
				return;
			}

			if (LineComponent)
			{
				LineComponent->SetLineMaterial(LoadObject<UMaterialInterface>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/LineSetOverlaidComponentMaterial")));
			}
			if (PointComponent)
			{
				PointComponent->SetPointMaterial(LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial")));
			}

			int32 ConnectedSeamIndex = 0;		// Used to generate different colors for each connected seam, if multiple connected seams are found per input seam

			for (int32 SeamIndex = 0; SeamIndex < ClothFacade.GetNumSeams(); ++SeamIndex)
			{
				// Stitches are given in random order, so first construct paths of connected stitches
				// Note one SeamFacade can contain multiple disjoint paths
				TArray<TArray<FIntVector2>> ConnectedSeams;
				UE::Chaos::ClothAsset::FClothGeometryTools::BuildConnectedSeams2D(Collection, SeamIndex, Mesh, ConnectedSeams);

				for (const TArray<FIntVector2>& ConnectedSeam : ConnectedSeams)
				{
					const FColor SeamColor = UE::Chaos::ClothAsset::Private::MakePseudoRandomColor(ConnectedSeamIndex++).ToFColor(true);

					// draw connected edge on each side of the seam
					if (LineComponent)
					{
						for (int32 StitchID = 0; StitchID < ConnectedSeam.Num() - 1; ++StitchID)
						{
							const FVector3d PointA = Mesh.GetVertex(ConnectedSeam[StitchID][0]);
							const FVector3d PointB = Mesh.GetVertex(ConnectedSeam[StitchID][1]);
							const FVector3d PointC = Mesh.GetVertex(ConnectedSeam[StitchID + 1][0]);
							const FVector3d PointD = Mesh.GetVertex(ConnectedSeam[StitchID + 1][1]);
							LineComponent->AddLine(PointA, PointC, SeamColor, LineThickness);
							LineComponent->AddLine(PointB, PointD, SeamColor, LineThickness);
						}
					}

					// draw connection between stitch points
					if (bSeamsCollapse)
					{
						const int32 StitchID = ConnectedSeam.Num() / 2;
						const int32 VertexA = ConnectedSeam[StitchID][0];
						const int32 VertexB = ConnectedSeam[StitchID][1];
						const FVector PtA = Mesh.GetVertex(VertexA);
						const FVector PtB = Mesh.GetVertex(VertexB);
						if (LineComponent)
						{
							LineComponent->AddLine(PtA, PtB, SeamColor, LineThickness);
						}
						if (PointComponent)
						{
							PointComponent->AddPoint(PtA, SeamColor, PointSize);
							PointComponent->AddPoint(PtB, SeamColor, PointSize);
						}
					}
					else
					{
						for (int32 StitchID = 0; StitchID < ConnectedSeam.Num(); ++StitchID)
						{
							const int32 VertexA = ConnectedSeam[StitchID][0];
							const int32 VertexB = ConnectedSeam[StitchID][1];
							const FVector PtA = Mesh.GetVertex(VertexA);
							const FVector PtB = Mesh.GetVertex(VertexB);
							if (LineComponent)
							{
								LineComponent->AddLine(PtA, PtB, SeamColor, 2.0f);
							}
							if (PointComponent)
							{
								PointComponent->AddPoint(PtA, SeamColor, PointSize);
								PointComponent->AddPoint(PtB, SeamColor, PointSize);
							}
						}
					}
				}
			}
		}

		static void GenerateSeamsForSim3D(
			const TSharedRef<const FManagedArrayCollection>& Collection, 
			const FCollectionClothConstFacade& ClothFacade, 
			const UE::Geometry::FDynamicMesh3& Mesh, 
			UPointSetComponent* PointComponent, const float PointSize,
			ULineSetComponent* LineComponent, const float LineThickness,
			bool bSeamsCollapse
		)
		{
			if (PointComponent == nullptr && LineComponent == nullptr)
			{
				return;
			}

			if (LineComponent)
			{
				LineComponent->SetLineMaterial(LoadObject<UMaterialInterface>(nullptr, TEXT("/Engine/EngineMaterials/LineSetComponentMaterial")));
			}
			if (PointComponent)
			{
				PointComponent->SetPointMaterial(LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetComponentMaterial")));
			}

			int32 ConnectedSeamIndex = 0;		// Used to generate different colors for each connected seam, if multiple connected seams are found per input seam

			for (int32 SeamIndex = 0; SeamIndex < ClothFacade.GetNumSeams(); ++SeamIndex)
			{
				const UE::Chaos::ClothAsset::FCollectionClothSeamConstFacade SeamFacade = ClothFacade.GetSeam(SeamIndex);

				const TArray<int32> SeamStitches(SeamFacade.GetSeamStitch3DIndex());
				const FColor SeamColor = UE::Chaos::ClothAsset::Private::MakePseudoRandomColor(SeamIndex).ToFColor(true);

				// In 3D we should be able to draw the seam edges in any order, doesn't need to be in connected paths
				for (int32 StitchIndexI = 0; StitchIndexI < SeamStitches.Num(); ++StitchIndexI)
				{
					const int32 VertexIndexI = SeamStitches[StitchIndexI];
					for (int32 StitchIndexJ = StitchIndexI + 1; StitchIndexJ < SeamStitches.Num(); ++StitchIndexJ)
					{
						const int32 VertexIndexJ = SeamStitches[StitchIndexJ];

						if (Mesh.FindEdge(VertexIndexI, VertexIndexJ) != UE::Geometry::FDynamicMesh3::InvalidID)
						{
							const FVector PointI = Mesh.GetVertex(VertexIndexI);
							const FVector PointJ = Mesh.GetVertex(VertexIndexJ);
							if (LineComponent)
							{
								LineComponent->AddLine(PointI, PointJ, SeamColor, LineThickness);
							}
							if (PointComponent)
							{
								PointComponent->AddPoint(PointI, SeamColor, PointSize);
								PointComponent->AddPoint(PointJ, SeamColor, PointSize);
							}
						}
					}
				}
			}
		}
	}
	////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FClothSimSurfaceRenderableType : public UE::Dataflow::IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(SimSurface);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowClothSimRenderSettings);

		virtual bool IsViewModeSupported(const UE::Dataflow::IDataflowConstructionViewMode& InViewMode) const override
		{
			return InViewMode.GetName() == UE::Chaos::ClothAsset::FCloth3DSimViewMode::Name;
		}

		virtual bool CanRender(const UE::Dataflow::FRenderableTypeInstance& Instance) const override
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);
			return (Collection.NumElements(ClothCollectionGroup::SimVertices3D) > 0);
		}

		virtual void GetPrimitiveComponents(const UE::Dataflow::FRenderableTypeInstance& Instance, UE::Dataflow::FRenderableComponents& OutComponents) const override
		{
			const UDataflowClothSimRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowClothSimRenderSettings>();

			FManagedArrayCollection Collection = GetCollection(Instance);

			const TSharedRef<const FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(Collection));
			const FCollectionClothConstFacade ClothFacade(ClothCollection);

			if (ClothFacade.HasValidSimulationData())
			{
				FDynamicMesh3 DynamicMesh;
				FClothPatternToDynamicMesh Converter;
				Converter.Convert(ClothCollection, INDEX_NONE, EClothPatternVertexType::Sim3D, DynamicMesh);

				bool bNeedVertexColorMaterial = false;
				const bool bShowPatternColors = (Settings && Settings->bShowPatternColors);
				if (bShowPatternColors)
				{
					Private::AddPatternsColorsToMesh(ClothFacade, DynamicMesh);
					bNeedVertexColorMaterial = true;
				}
				else 
				{
					const FDataflowNode::FAttributeKey AttributeKey = Instance.GetVertexAttributeToVisualize();
					if (AttributeKey.GroupName == ClothCollectionGroup::SimVertices3D)
					{
						TConstArrayView<float> MapValues = ClothFacade.GetWeightMap(AttributeKey.AttributeName);
						if (MapValues.Num() > 0)
						{
							Private::AddAttributeAsColorToMesh(ClothFacade.GetSimIndices3D(), DynamicMesh, MapValues);
							bNeedVertexColorMaterial = true;
						}
					}
				}
				const FString SelectionName = Instance.GetOutputValueByType<FString>({}, TEXT("DataflowSelection"));
				const bool bHasFaceSelection = Private::AddFaceSelectionToMesh(ClothCollection, FName(SelectionName), DynamicMesh);

				TArray<FVector> SelectedPoints;
				Private::GetVertexSelectionForSim3D(ClothCollection, ClothFacade, FName(SelectionName), SelectedPoints);

				// only create components for the one that have vertices
				if (DynamicMesh.VertexCount() > 0)
				{
					static const FName ComponentName = TEXT("Cloth_Sim3D_Mesh");
					if (UDynamicMeshComponent* Component = OutComponents.AddNewComponent<UDynamicMeshComponent>(ComponentName))
					{
						Component->SetCastShadow(false);
						Component->SetMesh(MoveTemp(DynamicMesh));

						UMaterialInterface* TwoSidedMaterial = bNeedVertexColorMaterial || bHasFaceSelection
							? Private::GetTwoSidedMaterialForColoredPatterns()
							: Private::GetTwoSidedMaterialForRenderingTypes();
						Component->SetOverrideRenderMaterial(TwoSidedMaterial);

						const FDynamicMesh3* Mesh = Component->GetMesh();
						if (Mesh && Settings && Settings->bShowSeams)
						{
							static const FName PointComponentName = TEXT("Cloth_Sim3D_SeamPoints");
							UPointSetComponent* PointComponent = (Settings && Settings->bShowSeamPoints)
								? OutComponents.AddNewComponent<UPointSetComponent>(PointComponentName)
								: nullptr;

							static const FName LineComponentName = TEXT("Cloth_Sim3D_SeamLines");
							ULineSetComponent* LineComponent = (Settings && Settings->bShowSeamLines)
								? OutComponents.AddNewComponent<ULineSetComponent>(LineComponentName)
								: nullptr;

							const float SeamPointsSize = Settings ? Settings->SeamPointSize : 4.0f;
							const float SeamLinesThickness = Settings ? Settings->SeamLineThickness : 2.0f;
							const bool bCollapseSeamLines = Settings && Settings->bCollapseSeamLines;

							Private::GenerateSeamsForSim3D(
								ClothCollection, ClothFacade, *Mesh,
								PointComponent, SeamPointsSize,
								LineComponent, SeamLinesThickness,
								bCollapseSeamLines
							);
						}

						if (SelectedPoints.Num() > 0)
						{
							static const FName PointComponentName = TEXT("Cloth_Sim3D_SelectedPoints");
							if (UPointSetComponent* PointComponent = OutComponents.AddNewComponent<UPointSetComponent>(PointComponentName))
							{
								PointComponent->AddPoints(SelectedPoints, FColor::Yellow, 10.f);
								PointComponent->SetPointMaterial(LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetComponentMaterial")));
							}
						}
					}
				}
			}
		}
	};

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FClothRenderSurfaceRenderableType : public UE::Dataflow::IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(RenderSurface);

		virtual bool IsViewModeSupported(const UE::Dataflow::IDataflowConstructionViewMode& InViewMode) const override
		{
			return InViewMode.GetName() == UE::Dataflow::FDataflowConstruction3DViewMode::Name
				|| InViewMode.GetName() == UE::Chaos::ClothAsset::FClothRenderViewMode::Name
				;
		}

		virtual bool CanRender(const UE::Dataflow::FRenderableTypeInstance& Instance) const override
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);
			return (Collection.NumElements(ClothCollectionGroup::RenderVertices) > 0);
		}

		virtual void GetPrimitiveComponents(const UE::Dataflow::FRenderableTypeInstance& Instance, UE::Dataflow::FRenderableComponents& OutComponents) const override
		{
			FManagedArrayCollection Collection = GetCollection(Instance);

			const TSharedRef<const FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(Collection));
			const FCollectionClothConstFacade ClothFacade(ClothCollection);
			if (ClothFacade.HasValidRenderData())
			{
				FDynamicMesh3 DynamicMesh;
				FClothPatternToDynamicMesh Converter;
				Converter.Convert(ClothCollection, INDEX_NONE, EClothPatternVertexType::Render, DynamicMesh);

				// only create components for the one that have vertices
				if (DynamicMesh.VertexCount() > 0)
				{
					bool bNeedVertexColorMaterial = false;
					const FDataflowNode::FAttributeKey AttributeKey = Instance.GetVertexAttributeToVisualize();
					if (AttributeKey.GroupName == ClothCollectionGroup::RenderVertices)
					{
						TConstArrayView<float> MapValues = ClothFacade.GetUserDefinedAttribute<float>(AttributeKey.AttributeName, ClothCollectionGroup::RenderVertices);
						if (MapValues.Num() > 0)
						{
							Private::AddAttributeAsColorToMesh(ClothFacade.GetRenderIndices(), DynamicMesh, MapValues);
							bNeedVertexColorMaterial = true;
						}
					}

					static const FName ComponentName = TEXT("Cloth_Render_Mesh");
					if (UDynamicMeshComponent* Component = OutComponents.AddNewComponent<UDynamicMeshComponent>(ComponentName))
					{
						Component->SetCastShadow(false);
						Component->SetMesh(MoveTemp(DynamicMesh));

						const TConstArrayView<FSoftObjectPath> MaterialPaths = ClothFacade.GetRenderMaterialSoftObjectPathName();
						const int32 NumMaterials = MaterialPaths.Num();

						UMaterialInterface* VertexColorMaterial = Private::GetTwoSidedMaterialForColoredPatterns();

						for (int32 Index = 0; Index < NumMaterials; ++Index)
						{
							UMaterialInterface* const Material = bNeedVertexColorMaterial
								? VertexColorMaterial
								: Cast<UMaterialInterface>(MaterialPaths[Index].TryLoad());
							Component->SetMaterial(Index, Material);
						}
					}
				}
			}
		}
	};


	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	class FClothSimPatternRenderableType : public UE::Dataflow::IRenderableType
	{
		UE_DATAFLOW_IRENDERABLE_PRIMARY_TYPE(FManagedArrayCollection, Collection);
		UE_DATAFLOW_IRENDERABLE_RENDER_GROUP(Pattern);
		UE_DATAFLOW_IRENDERABLE_SETTINGS(UDataflowClothSimRenderSettings);

		virtual bool IsViewModeSupported(const UE::Dataflow::IDataflowConstructionViewMode& InViewMode) const override
		{
			return InViewMode.GetName() == UE::Dataflow::FDataflowConstruction2DViewMode::Name
				|| InViewMode.GetName() == UE::Chaos::ClothAsset::FCloth2DSimViewMode::Name
				;
		}

		virtual bool CanRender(const UE::Dataflow::FRenderableTypeInstance& Instance) const override
		{
			const FManagedArrayCollection& Collection = GetCollection(Instance);
			return (Collection.NumElements(ClothCollectionGroup::SimVertices2D) > 0);
		}

		virtual void GetPrimitiveComponents(const UE::Dataflow::FRenderableTypeInstance& Instance, UE::Dataflow::FRenderableComponents& OutComponents) const override
		{
			const UDataflowClothSimRenderSettings* Settings = Instance.GetTypedRenderSettings<UDataflowClothSimRenderSettings>();

			FManagedArrayCollection Collection = GetCollection(Instance);

			const TSharedRef<const FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(Collection));
			const FCollectionClothConstFacade ClothFacade(ClothCollection);
			if (ClothFacade.HasValidSimulationData())
			{
				FDynamicMesh3 DynamicMesh;
				FClothPatternToDynamicMesh Converter;
				Converter.Convert(ClothCollection, INDEX_NONE, EClothPatternVertexType::Sim2D, DynamicMesh, /*bDisableAttributes*/false, INDEX_NONE, /*bFlip2DSimFaces*/true);

				const bool bShowPatternColors = (Settings && Settings->bShowPatternColors);
				if (bShowPatternColors)
				{
					Private::AddPatternsColorsToMesh(ClothFacade, DynamicMesh);
				}
				const FString SelectionName = Instance.GetOutputValueByType<FString>({}, TEXT("DataflowSelection"));
				const bool bHasFaceSelection = Private::AddFaceSelectionToMesh(ClothCollection, FName(SelectionName), DynamicMesh);

				TArray<FVector> SelectedPoints;
				Private::GetVertexSelectionForSim2D(ClothCollection, ClothFacade, FName(SelectionName), SelectedPoints);

				const FVector ScaleFactor(100, 100, 1);

				// only create components for the one that have vertices
				if (DynamicMesh.VertexCount() > 0)
				{
					static const FName ComponentName = TEXT("Cloth_Sim2D_Mesh");
					if (UDynamicMeshComponent* Component = OutComponents.AddNewComponent<UDynamicMeshComponent>(ComponentName))
					{
						Component->SetCastShadow(false);
						Component->SetMesh(MoveTemp(DynamicMesh));
						UMaterialInterface* TwoSidedMaterial = bShowPatternColors || bHasFaceSelection
							? Private::GetTwoSidedMaterialForColoredPatterns()
							: Private::GetTwoSidedMaterialForRenderingTypes();
						Component->SetOverrideRenderMaterial(TwoSidedMaterial);
						Component->SetWorldScale3D(ScaleFactor);

						const FDynamicMesh3* Mesh = Component->GetMesh();
						if (Mesh && Settings && Settings->bShowSeams)
						{

							static const FName PointComponentName = TEXT("Cloth_Sim2D_SeamPoints");
							UPointSetComponent* PointComponent = (Settings && Settings->bShowSeamPoints)
								? OutComponents.AddNewComponent<UPointSetComponent>(PointComponentName)
								: nullptr;

							static const FName LineComponentName = TEXT("Cloth_Sim2D_SeamLines");
							ULineSetComponent* LineComponent = (Settings && Settings->bShowSeamLines)
								? OutComponents.AddNewComponent<ULineSetComponent>(LineComponentName)
								: nullptr;

							const float SeamPointsSize = Settings ? Settings->SeamPointSize : 4.0f;
							const float SeamLinesThickness = Settings ? Settings->SeamLineThickness : 2.0f;
							const bool bCollapseSeamLines = Settings && Settings->bCollapseSeamLines;

							Private::GenerateSeamsForSim2D(
								ClothCollection, ClothFacade, *Mesh,
								PointComponent, SeamPointsSize,
								LineComponent, SeamLinesThickness,
								bCollapseSeamLines
							);

							if (PointComponent)
							{
								PointComponent->SetWorldScale3D(ScaleFactor);
							}
							if (LineComponent)
							{
								LineComponent->SetWorldScale3D(ScaleFactor);
							}
						}

						if (SelectedPoints.Num() > 0)
						{
							static const FName PointComponentName = TEXT("Cloth_Sim2D_SelectedPoints");
							if (UPointSetComponent* PointComponent = OutComponents.AddNewComponent<UPointSetComponent>(PointComponentName))
							{
								PointComponent->AddPoints(SelectedPoints, FColor::Yellow, 10.f);
								PointComponent->SetPointMaterial(LoadObject<UMaterial>(nullptr, TEXT("/MeshModelingToolsetExp/Materials/PointSetOverlaidComponentMaterial")));
								PointComponent->SetWorldScale3D(ScaleFactor);
							}
						}
					}
				}
			}
		}
	};
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////

	void RegisterClothRenderableTypes()
	{
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FClothSimSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FClothRenderSurfaceRenderableType);
		UE_DATAFLOW_REGISTER_RENDERABLE_TYPE(FClothSimPatternRenderableType);


		static const FName ClothCategoryName(TEXT("Cloth"));

		UDataflowRenderableTypeSettings::RegisterSection(
			UDataflowClothSimRenderSettings::StaticClass(),
			"Cloth", LOCTEXT("Cloth", "Cloth"),
			{ ClothCategoryName }
		);
	}
}

#undef LOCTEXT_NAMESPACE 
