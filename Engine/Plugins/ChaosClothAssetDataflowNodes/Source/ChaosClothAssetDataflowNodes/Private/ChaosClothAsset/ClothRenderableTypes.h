// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataflowRendering/DataflowRenderableTypeSettings.h"

#include "ClothRenderableTypes.generated.h"

UCLASS(MinimalAPI, AutoExpandCategories = "Cloth")
class UDataflowClothSimRenderSettings : public UDataflowRenderableTypeSettings
{
	GENERATED_BODY()

public:
	/** display the seams between patterns */
	UPROPERTY(EditAnywhere, Category = "Cloth")
	bool bShowSeams = false;

	/** when seams are displayed, collapse the multiple seams lines into one */
	UPROPERTY(EditAnywhere, Category = "Cloth", meta = (EditCondition = bShowSeams))
	bool bCollapseSeamLines = false;

	/** Whether to show the seam points or not */
	UPROPERTY(EditAnywhere, Category = "Cloth", meta = (EditCondition = bShowSeams))
	bool bShowSeamPoints = true;

	/** Size of the seam points */
	UPROPERTY(EditAnywhere, Category = "Cloth", meta = (UIMin = 0.01, ClampMin = 0.01, EditCondition = "bShowSeams && bShowSeamPoints"))
	float SeamPointSize = 4.0f;

	/** Whether to show the seam lines or not */
	UPROPERTY(EditAnywhere, Category = "Cloth", meta = (EditCondition = bShowSeams))
	bool bShowSeamLines = true;

	/** Thickness of the seam lines */
	UPROPERTY(EditAnywhere, Category = "Cloth", meta = (UIMin = 0.01, ClampMin = 0.01, EditCondition = "bShowSeams && bShowSeamLines"))
	float SeamLineThickness = 2.0f;

	/** display the patterns colors */
	UPROPERTY(EditAnywhere, Category = "Cloth")
	bool bShowPatternColors = false;
};

namespace UE::Chaos::ClothAsset
{
	void RegisterClothRenderableTypes();
}  // End namespace UE::Chaos::ClothAsset


