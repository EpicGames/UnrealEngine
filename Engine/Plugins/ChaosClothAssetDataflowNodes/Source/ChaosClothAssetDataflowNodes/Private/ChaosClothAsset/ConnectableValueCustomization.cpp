// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ConnectableValueCustomization.h"
#include "ChaosClothAsset/ClothAssetEditorStyle.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ImportedValueCustomization.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowNodeParameters.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetWeightedValueCustomization"

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		static const FString OverridePrefix = TEXT("_Override");  // UE_DEPRECATED(5.5, "Override properties are no longer used.")
		static const FString BuildFabricMaps = TEXT("BuildFabricMaps");
		static const FString CouldUseFabrics = TEXT("CouldUseFabrics");
	}
	
	// UE_DEPRECATED(5.5, "Override properties are no longer used.")
	bool FConnectableValueCustomization::IsOverrideProperty(const TSharedPtr<IPropertyHandle>& Property)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FStringView PropertyPath = Property ? Property->GetPropertyPath() : FStringView();
		return PropertyPath.EndsWith(Private::OverridePrefix, ESearchCase::CaseSensitive);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	// UE_DEPRECATED(5.5, "Override properties are no longer used.")
	bool FConnectableValueCustomization::IsOverridePropertyOf(const TSharedPtr<IPropertyHandle>& OverrideProperty, const TSharedPtr<IPropertyHandle>& Property)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FStringView OverridePropertyPath = OverrideProperty ? OverrideProperty->GetPropertyPath() : FStringView();
		const FStringView PropertyPath = Property ? Property->GetPropertyPath() : FStringView();
		return OverridePropertyPath == FString(PropertyPath) + Private::OverridePrefix;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	bool FConnectableValueCustomization::BuildFabricMapsProperty(const TSharedPtr<IPropertyHandle>& Property)
	{
		const FStringView PropertyPath = Property ? Property->GetPropertyPath() : FStringView();
		return PropertyPath.EndsWith(Private::BuildFabricMaps, ESearchCase::CaseSensitive);
	}
	bool FConnectableValueCustomization::CouldUseFabricsProperty(const TSharedPtr<IPropertyHandle>& Property)
	{
		const FStringView PropertyPath = Property ? Property->GetPropertyPath() : FStringView();
		return PropertyPath.EndsWith(Private::CouldUseFabrics, ESearchCase::CaseSensitive);
	}
	
	TSharedRef<IPropertyTypeCustomization> FConnectableValueCustomization::MakeInstance()
	{
		return MakeShareable(new FConnectableValueCustomization);
	}

	FConnectableValueCustomization::FConnectableValueCustomization() = default;
	
	FConnectableValueCustomization::~FConnectableValueCustomization() = default;

	void FConnectableValueCustomization::CustomizeChildren(
		TSharedRef<IPropertyHandle> PropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& CustomizationUtils)
	{
		for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];

			if (CouldUseFabricsProperty(ChildHandle))
			{
				bool bCouldUseFabrics = false;
				ChildHandle->GetValue(bCouldUseFabrics);
				if (bCouldUseFabrics)
				{
					FImportedValueCustomization::CustomizeChildren(PropertyHandle, ChildBuilder, CustomizationUtils);
				}
				return;
			}
		}
	}

	void FConnectableValueCustomization::MakeHeaderRow(TSharedRef<class IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row)
	{
		const TWeakPtr<IPropertyHandle> StructWeakHandlePtr = StructPropertyHandle;

		TSharedPtr<SHorizontalBox> ValueHorizontalBox;
		TSharedPtr<SHorizontalBox> NameHorizontalBox;

		Row.NameContent()
			[
				SAssignNew(NameHorizontalBox, SHorizontalBox)
				.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, StructWeakHandlePtr)
			]
		.ValueContent()
			// Make enough space for each child handle
			.MinDesiredWidth(125.f * SortedChildHandles.Num())
			.MaxDesiredWidth(125.f * SortedChildHandles.Num())
			[
				SAssignNew(ValueHorizontalBox, SHorizontalBox)
				.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, StructWeakHandlePtr)
			];
		
		for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];
			
			if (CouldUseFabricsProperty(ChildHandle))
			{
				bool bValue = false;
				ChildHandle->GetValue(bValue);
				if(!bValue)
				{
					break;
				}
			}
			else if (BuildFabricMapsProperty(ChildHandle))
			{
				AddToggledCheckBox(ChildHandle, NameHorizontalBox, UE::Chaos::ClothAsset::FClothAssetEditorStyle::Get().GetBrush("ClassIcon.ChaosClothPreset"));
			}
		}
		
		NameHorizontalBox->AddSlot().VAlign(VAlign_Center)
				.Padding(FMargin(4.f, 2.f, 4.0f, 2.f))
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					StructPropertyHandle->CreatePropertyNameWidget()
				];

		for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (IsOverrideProperty(ChildHandle))
			{
				continue;  // Skip overrides
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			const bool bLastChild = SortedChildHandles.Num() - 1 == ChildIndex;

			TSharedRef<SWidget> ChildWidget = MakeChildWidget(StructPropertyHandle, ChildHandle);
			if(ChildWidget != SNullWidget::NullWidget)
			{
				ValueHorizontalBox->AddSlot()
					.Padding(FMargin(0.f, 2.f, bLastChild ? 0.f : 3.f, 2.f))
					[
						ChildWidget
					];
			}
		}
	}

	TSharedRef<SWidget> FConnectableValueCustomization::MakeChildWidget(
		TSharedRef<IPropertyHandle>& StructurePropertyHandle,
		TSharedRef<IPropertyHandle>& PropertyHandle)
	{
		const FFieldClass* PropertyClass = PropertyHandle->GetPropertyClass();
		
		if (PropertyClass == FStrProperty::StaticClass())
		{
			TWeakPtr<IPropertyHandle> HandleWeakPtr = PropertyHandle;
			return
				SNew(SEditableTextBox)
				.ToolTipText(PropertyHandle->GetToolTipText())
				.Text_Lambda([HandleWeakPtr, StructurePropertyHandle]() -> FText
					{
						using namespace UE::Chaos::ClothAsset;

						FString Text;
						if (const TSharedPtr<IPropertyHandle> HandlePtr = HandleWeakPtr.Pin())
						{
							const FDataflowNode* const DataflowNode = FClothDataflowTools::GetPropertyOwnerDataflowNode(StructurePropertyHandle);
							if (ensure(DataflowNode))
							{
								void* Data;
								if (HandlePtr->GetValueData(Data) == FPropertyAccess::Success)  // GetValueData could return false with multiple selections 
								{
									Text = *static_cast<FString*>(Data);  // Default value if the property isn't an input, or isn't connected
									if (const FDataflowInput* const DataflowInput = DataflowNode->FindInput(Data))
									{
										UE::Dataflow::FContextThreaded Context;
										Text = DataflowInput->GetValue<FString>(Context, Text);
									}
								}
							}
						}
						return FText::FromString(Text);
					})
				.OnTextCommitted_Lambda([HandleWeakPtr](const FText& Text, ETextCommit::Type)
					{
						if (const TSharedPtr<IPropertyHandle> HandlePtr = HandleWeakPtr.Pin())
						{
							FString TextString = Text.ToString();
							FClothDataflowTools::MakeCollectionName(TextString);
							HandlePtr->SetValue(TextString, EPropertyValueSetFlags::DefaultFlags);
						}
					})
				.OnVerifyTextChanged_Lambda([](const FText& Text, FText& OutErrorMessage) -> bool
					{
						bool bIsValidCollectionName = false;
						FString TextString = Text.ToString();
						bIsValidCollectionName = FClothDataflowTools::MakeCollectionName(TextString);
						if (!bIsValidCollectionName)
						{
							OutErrorMessage =
								LOCTEXT("NotValidCollectioName",
									"To be a valid collection name, this text string musn't start by an underscore,\n"
									"contain whitespaces, or any of the following character: \"',/.:|&!~@#(){}[]=;^%$`");
						}
						return bIsValidCollectionName;
					})
				.IsEnabled_Lambda([HandleWeakPtr, StructurePropertyHandle]() -> bool
					{
						if (const TSharedPtr<IPropertyHandle> HandlePtr = HandleWeakPtr.Pin())
						{
							const FDataflowNode* const DataflowNode = FClothDataflowTools::GetPropertyOwnerDataflowNode(StructurePropertyHandle);
							if (ensure(DataflowNode))
							{
								void* Data;
								if (HandlePtr->GetValueData(Data) == FPropertyAccess::Success)
								{
									if (const FDataflowInput* const DataflowInput = DataflowNode->FindInput(Data))
									{
										return !DataflowInput->HasAnyConnections();
									}
								}
							}
						}
						return true;
					})
				.Font(IPropertyTypeCustomizationUtils::GetRegularFont());
		}
		return SNullWidget::NullWidget;
	}
}  // End namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
