// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosClothAsset/ImportedValueCustomization.h"

namespace UE::Chaos::ClothAsset
{
	/**
	 * Customization for all connectable property that could be imported
	 * Works like a FMathStructCustomization.
	 */
	class FConnectableValueCustomization : public FImportedValueCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		FConnectableValueCustomization();
		virtual ~FConnectableValueCustomization() override;

	protected:
		//~ Begin IPropertyTypeCustomization implementation
		virtual void CustomizeChildren(
			TSharedRef<IPropertyHandle> PropertyHandle,
			IDetailChildrenBuilder& ChildBuilder,
			IPropertyTypeCustomizationUtils& CustomizationUtils) override;
		//~ End IPropertyTypeCustomization implementation

		//~ Begin FMathStructCustomization implementation
		virtual void MakeHeaderRow(TSharedRef<class IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row) override;
		virtual TSharedRef<SWidget> MakeChildWidget(TSharedRef<IPropertyHandle>& StructurePropertyHandle, TSharedRef<IPropertyHandle>& PropertyHandle) override;
		//~ End FMathStructCustomization implementation

		UE_DEPRECATED(5.5, "Override properties are no longer used.")
		static bool IsOverrideProperty(const TSharedPtr<IPropertyHandle>& Property);
		UE_DEPRECATED(5.5, "Override properties are no longer used.")
		static bool IsOverridePropertyOf(const TSharedPtr<IPropertyHandle>& OverrideProperty, const TSharedPtr<IPropertyHandle>& Property);
		static bool BuildFabricMapsProperty(const TSharedPtr<IPropertyHandle>& Property);
		static bool CouldUseFabricsProperty(const TSharedPtr<IPropertyHandle>& Property);
	};
}
