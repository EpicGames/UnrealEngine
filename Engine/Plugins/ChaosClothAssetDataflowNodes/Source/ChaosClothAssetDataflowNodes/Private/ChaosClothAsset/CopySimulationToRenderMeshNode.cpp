// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/CopySimulationToRenderMeshNode.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CopySimulationToRenderMeshNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetCopySimulationToRenderMeshNode"

FChaosClothAssetCopySimulationToRenderMeshNode::FChaosClothAssetCopySimulationToRenderMeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetCopySimulationToRenderMeshNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		if (FCollectionClothFacade(ClothCollection).IsValid())  // Can only act on the collection if it is a valid cloth collection
		{
			// Empty mesh and materials
			FClothGeometryTools::DeleteRenderMesh(ClothCollection);

			// Find the material asset path
			const FString MaterialPathName = Material ? 
				Material->GetPathName() :
				FString(TEXT("/Engine/EditorMaterials/Cloth/CameraLitDoubleSided.CameraLitDoubleSided"));

			// Copy the mesh data
			FClothGeometryTools::CopySimMeshToRenderMesh(ClothCollection, MaterialPathName, bGenerateSingleRenderPattern);
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
