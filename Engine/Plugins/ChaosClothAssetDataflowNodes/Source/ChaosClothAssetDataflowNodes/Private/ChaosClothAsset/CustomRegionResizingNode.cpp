// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/CustomRegionResizingNode.h"
#include "ChaosClothAsset/ClothCollectionAttribute.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomRegionResizingNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetCustomRegionResizingNode"

FChaosClothAssetCustomRegionResizingNode::FChaosClothAssetCustomRegionResizingNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	using namespace UE::Chaos::ClothAsset;
	SimCustomResizingBlendName = ClothCollectionAttribute::SimCustomResizingBlend.ToString();
	RenderCustomResizingBlend = ClothCollectionAttribute::RenderCustomResizingBlend.ToString();

	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&SimCustomResizingBlendName);
	RegisterOutputConnection(&RenderCustomResizingBlend);

	check(GetNumInputs() == NumRequiredInputs);

	// Add a set of pins to start
	for (int32 Index = 0; Index < NumInitialOptionalInputs; ++Index)
	{
		AddPins();
	}
}

void FChaosClothAssetCustomRegionResizingNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;
		using namespace Chaos::Softs;

		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		TArray<FName> InputSets;
		TArray<int32> InputTypes;
		InputSets.SetNum(InputGroupData.Num());
		InputTypes.SetNum(InputGroupData.Num());
		for (int32 Index = 0; Index < InputGroupData.Num(); ++Index)
		{
			InputSets[Index] = FName(*GetValue(Context, GetConnectionReference(Index)));
			InputTypes[Index] = (int32)InputGroupData[Index].ResizingType;
		}

		FClothDataflowTools::SetGroupResizingData(ClothCollection, InputSets, InputTypes);
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
	else if(Out->IsA<FString>(&SimCustomResizingBlendName))
	{
		SetValue(Context, SimCustomResizingBlendName, &SimCustomResizingBlendName);
	}
	else if (Out->IsA<FString>(&RenderCustomResizingBlend))
	{
		SetValue(Context, RenderCustomResizingBlend, &RenderCustomResizingBlend);
	}
}

TArray<UE::Dataflow::FPin> FChaosClothAssetCustomRegionResizingNode::AddPins()
{
	const int32 Index = InputGroupData.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FChaosClothAssetCustomRegionResizingNode::GetPinsToRemove() const
{
	const int32 Index = InputGroupData.Num() - 1;
	check(InputGroupData.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FChaosClothAssetCustomRegionResizingNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = InputGroupData.Num() - 1;
	check(InputGroupData.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	InputGroupData.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

void FChaosClothAssetCustomRegionResizingNode::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		for (int32 Index = 0; Index < NumInitialOptionalInputs; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialOptionalInputs; Index < InputGroupData.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialOptionalInputs);
			const int32 OrigNumOptionalInputs = InputGroupData.Num();
			const int32 OrigNumRegisteredOptionalInputs = OrigNumRegisteredInputs - NumRequiredInputs;
			if (OrigNumRegisteredOptionalInputs > OrigNumOptionalInputs)
			{
				// Inputs have been removed.
				// Temporarily expand Collections so we can get connection references.
				InputGroupData.SetNum(OrigNumRegisteredOptionalInputs);
				for (int32 Index = OrigNumOptionalInputs; Index < InputGroupData.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				InputGroupData.SetNum(OrigNumOptionalInputs);
			}
		}
		else
		{
			ensureAlways(InputGroupData.Num() + NumRequiredInputs == GetNumInputs());
		}
	}
}

UE::Dataflow::TConnectionReference<FString> FChaosClothAssetCustomRegionResizingNode::GetConnectionReference(int32 Index) const
{
	return { &InputGroupData[Index].InputSet.StringValue, Index, &InputGroupData };
}

TArray<FChaosClothAssetCustomRegionResizingNode::FRegionData> FChaosClothAssetCustomRegionResizingNode::GetRegionData(UE::Dataflow::FContext& Context) const
{
	TArray<FChaosClothAssetCustomRegionResizingNode::FRegionData> BindingSetData;
	BindingSetData.SetNumUninitialized(InputGroupData.Num());
	for (int32 Index = 0; Index < InputGroupData.Num(); ++Index)
	{
		BindingSetData[Index].InputSet = *GetValue(Context, GetConnectionReference(Index));
		BindingSetData[Index].ResizingType = InputGroupData[Index].ResizingType;
	}
	return BindingSetData;
}
#undef LOCTEXT_NAMESPACE
