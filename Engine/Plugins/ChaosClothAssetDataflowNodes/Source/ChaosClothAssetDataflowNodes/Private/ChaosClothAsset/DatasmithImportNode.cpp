// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/DatasmithImportNode.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothComponent.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Misc/SecureHash.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DatasmithImportNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetDatasmithImportNode"

FChaosClothAssetDatasmithImportNode::FChaosClothAssetDatasmithImportNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&Collection);
}

void FChaosClothAssetDatasmithImportNode::Serialize(FArchive& Archive)
{
	using namespace UE::Chaos::ClothAsset;

	::Chaos::FChaosArchive ChaosArchive(Archive);
	ImportCache.Serialize(ChaosArchive);
	if (Archive.IsLoading())
	{
		// Make sure to always have a valid cloth collection on reload, some new attributes could be missing from the cached collection
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(ImportCache));
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (!ClothFacade.IsValid())
		{
			ClothFacade.DefineSchema();
		}
		ImportCache = MoveTemp(*ClothCollection);
	}
	FMD5Hash ImportHash;
	Archive << ImportHash;
}

void FChaosClothAssetDatasmithImportNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SetValue(Context, ImportCache, &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
