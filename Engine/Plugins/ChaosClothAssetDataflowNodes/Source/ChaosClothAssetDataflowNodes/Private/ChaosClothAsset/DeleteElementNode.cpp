// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/DeleteElementNode.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DeleteElementNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetDeleteElementNode"

FChaosClothAssetDeleteElementNode::FChaosClothAssetDeleteElementNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterInputConnection(&SelectionName.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
}

void FChaosClothAssetDeleteElementNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		FCollectionClothFacade Cloth(ClothCollection);
		if (Cloth.IsValid())
		{
			auto DeleteElementsByType = [&ClothCollection, &Cloth](const FName& GroupName, const TSet<int32>& Set)
			{
				if(GroupName == ClothCollectionGroup::SimPatterns)
				{
					if (Set.IsEmpty())
					{
						// Remove all sim patterns
						Cloth.SetNumSimPatterns(0);
					}
					else
					{
						TArray<int32> PatternsToRemove;
						PatternsToRemove.Reserve(Set.Num());
						for (int32 ElementIndex = 0; ElementIndex < Cloth.GetNumSimPatterns(); ++ElementIndex)
						{
							if (Set.Contains(ElementIndex))
							{
								PatternsToRemove.Add(ElementIndex);
							}
						}
						if (PatternsToRemove.Num())
						{
							Cloth.RemoveSimPatterns(PatternsToRemove);
						}
					}
				}
				else if(GroupName == ClothCollectionGroup::RenderPatterns)
				{
					if (Set.IsEmpty())
					{
						// Remove all render patterns
						Cloth.SetNumRenderPatterns(0);
					}
					else
					{
						TArray<int32> PatternsToRemove;
						PatternsToRemove.Reserve(Set.Num());
						for (int32 ElementIndex = 0; ElementIndex < Cloth.GetNumRenderPatterns(); ++ElementIndex)
						{
							if (Set.Contains(ElementIndex))
							{
								PatternsToRemove.Add(ElementIndex);
							}
						}
						if (PatternsToRemove.Num())
						{
							Cloth.RemoveRenderPatterns(PatternsToRemove);
						}
					}
				}
				else if(GroupName == ClothCollectionGroup::SimVertices2D)
				{
					if (Set.IsEmpty())
					{
						for (int32 ElementIndex = 0; ElementIndex < Cloth.GetNumSimPatterns(); ++ElementIndex)
						{
							FCollectionClothSimPatternFacade Pattern = Cloth.GetSimPattern(ElementIndex);
							Pattern.RemoveAllSimVertices2D();
						}
					}
					else
					{
						TArray<int32> SortedToDeleteList;
						SortedToDeleteList.Reserve(Set.Num());
						for (int32 PatternIndex = Cloth.GetNumSimPatterns() - 1; PatternIndex >= 0; --PatternIndex)
						{
							FCollectionClothSimPatternFacade Pattern = Cloth.GetSimPattern(PatternIndex);
							const int32 VertexOffset = Pattern.GetSimVertices2DOffset();
							SortedToDeleteList.Reset();
							for (int32 VertexIndex = 0; VertexIndex < Pattern.GetNumSimVertices2D(); ++VertexIndex)
							{
								if (Set.Contains(VertexIndex + VertexOffset))
								{
									SortedToDeleteList.Add(VertexIndex);
								}
							}
							if (SortedToDeleteList.Num())
							{
								Pattern.RemoveSimVertices2D(SortedToDeleteList);
							}
						}
					}
				}
				else if(GroupName == ClothCollectionGroup::SimVertices3D)
				{
					if (Set.IsEmpty())
					{
						Cloth.RemoveAllSimVertices3D();
					}
					else
					{
						TArray<int32> SortedToDeleteList;
						SortedToDeleteList.Reserve(Set.Num());
						for (int32 VertexIndex = 0; VertexIndex < Cloth.GetNumSimVertices3D(); ++VertexIndex)
						{
							if (Set.Contains(VertexIndex))
							{
								SortedToDeleteList.Add(VertexIndex);
							}
						}
						Cloth.RemoveSimVertices3D(SortedToDeleteList);
					}
				}
				else if(GroupName == ClothCollectionGroup::RenderVertices)
				{
					if (Set.IsEmpty())
					{
						for (int32 ElementIndex = 0; ElementIndex < Cloth.GetNumRenderPatterns(); ++ElementIndex)
						{
							FCollectionClothRenderPatternFacade Pattern = Cloth.GetRenderPattern(ElementIndex);
							Pattern.SetNumRenderVertices(0);
						}
					}
					else
					{
						TArray<int32> SortedToDeleteList;
						SortedToDeleteList.Reserve(Set.Num());
						for (int32 PatternIndex = Cloth.GetNumRenderPatterns() - 1; PatternIndex >= 0; --PatternIndex)
						{
							FCollectionClothRenderPatternFacade Pattern = Cloth.GetRenderPattern(PatternIndex);
							const int32 VertexOffset = Pattern.GetRenderVerticesOffset();
							SortedToDeleteList.Reset();
							for (int32 VertexIndex = 0; VertexIndex < Pattern.GetNumRenderVertices(); ++VertexIndex)
							{
								if (Set.Contains(VertexIndex + VertexOffset))
								{
									SortedToDeleteList.Add(VertexIndex);
								}
							}
							if (SortedToDeleteList.Num())
							{
								Pattern.RemoveRenderVertices(SortedToDeleteList);
							}
						}
					}
				}
				else if(GroupName == ClothCollectionGroup::SimFaces)
				{
					if (Set.IsEmpty())
					{
						for (int32 ElementIndex = 0; ElementIndex < Cloth.GetNumSimPatterns(); ++ElementIndex)
						{
							FCollectionClothSimPatternFacade Pattern = Cloth.GetSimPattern(ElementIndex);
							Pattern.SetNumSimFaces(0);
						}
					}
					else
					{
						TArray<int32> SortedToDeleteList;
						SortedToDeleteList.Reserve(Set.Num());
						for (int32 PatternIndex = Cloth.GetNumSimPatterns() - 1; PatternIndex >= 0; --PatternIndex)
						{
							FCollectionClothSimPatternFacade Pattern = Cloth.GetSimPattern(PatternIndex);
							const int32 FaceOffset = Pattern.GetSimFacesOffset();
							SortedToDeleteList.Reset();
							for (int32 FaceIndex = 0; FaceIndex < Pattern.GetNumSimFaces(); ++FaceIndex)
							{
								if (Set.Contains(FaceIndex + FaceOffset))
								{
									SortedToDeleteList.Add(FaceIndex);
								}
							}
							if (SortedToDeleteList.Num())
							{
								Pattern.RemoveSimFaces(SortedToDeleteList);
							}
						}
					}
				}
				else if(GroupName == ClothCollectionGroup::RenderFaces)
				{
					if (Set.IsEmpty())
					{
						for (int32 PatternIndex = 0; PatternIndex < Cloth.GetNumRenderPatterns(); ++PatternIndex)
						{
							FCollectionClothRenderPatternFacade Pattern = Cloth.GetRenderPattern(PatternIndex);
							Pattern.SetNumRenderFaces(0);
						}
					}
					else
					{
						TArray<int32> SortedToDeleteList;
						SortedToDeleteList.Reserve(Set.Num());
						for (int32 PatternIndex = Cloth.GetNumRenderPatterns() - 1; PatternIndex >= 0; --PatternIndex)
						{
							FCollectionClothRenderPatternFacade Pattern = Cloth.GetRenderPattern(PatternIndex);
							const int32 FaceOffset = Pattern.GetRenderFacesOffset();
							SortedToDeleteList.Reset();
							for (int32 FaceIndex = 0; FaceIndex < Pattern.GetNumRenderFaces(); ++FaceIndex)
							{
								if (Set.Contains(FaceIndex + FaceOffset))
								{
									SortedToDeleteList.Add(FaceIndex);
								}
							}
							if (SortedToDeleteList.Num())
							{
								Pattern.RemoveRenderFaces(SortedToDeleteList);
							}
						}
					}
				}
				else if(GroupName == ClothCollectionGroup::Seams)
				{
					if (Set.IsEmpty())
					{
						// Remove all seams
						Cloth.SetNumSeams(0);
					}
					else
					{
						TArray<int32> SeamsToRemove;
						SeamsToRemove.Reserve(Set.Num());
						for (int32 SeamIndex = 0; SeamIndex < Cloth.GetNumSeams(); ++SeamIndex)
						{
							if (Set.Contains(SeamIndex))
							{
								SeamsToRemove.Add(SeamIndex);
							}
						}
						if (SeamsToRemove.Num())
						{
							Cloth.RemoveSeams(SeamsToRemove);
						}
					}
				}
				else if(GroupName == ClothCollectionGroup::Fabrics)
				{
					if (Set.IsEmpty())
					{
						// Remove all fabrics
						Cloth.SetNumFabrics(0);
					}
					else
					{
						TArray<int32> FabricsToRemove;
						FabricsToRemove.Reserve(Set.Num());
						for (int32 FabricIndex = 0; FabricIndex < Cloth.GetNumFabrics(); ++FabricIndex)
						{
							if (Set.Contains(FabricIndex))
							{
								FabricsToRemove.Add(FabricIndex);
							}
						}
						if (FabricsToRemove.Num())
						{
							Cloth.RemoveFabrics(FabricsToRemove);
						}
					}
				}
			};


			// Selection set
			FCollectionClothSelectionFacade SelectionFacade(ClothCollection);
			const FName InSelectionName(*GetValue<FString>(Context, &SelectionName.StringValue));

			FName SelectionGroup;
			TSet<int32> SelectedElements;
			bool bValidSelectionGroup = false;

			if (SelectionFacade.IsValid() && InSelectionName != NAME_None)
			{
				SelectionGroup = SelectionFacade.GetSelectionGroup(InSelectionName);
				if (const TSet<int32>* const SelectionSet = SelectionFacade.FindSelectionSet(InSelectionName))
				{
					if (SelectionGroup == ClothCollectionGroup::Seams ||
						SelectionGroup == ClothCollectionGroup::SimPatterns ||
						SelectionGroup == ClothCollectionGroup::RenderPatterns ||
						SelectionGroup == ClothCollectionGroup::SimFaces ||
						SelectionGroup == ClothCollectionGroup::SimVertices2D ||
						SelectionGroup == ClothCollectionGroup::SimVertices3D ||
						SelectionGroup == ClothCollectionGroup::RenderFaces ||
						SelectionGroup == ClothCollectionGroup::RenderVertices ||
						SelectionGroup == ClothCollectionGroup::Fabrics)
					{
						SelectedElements = *SelectionSet;
						bValidSelectionGroup = true;
					}
					else
					{
						Context.Warning(
							FText::Format(LOCTEXT("SelectionTypeNotCorrectDetails", "Selection \"{0}\" does not have its index dependency group set to a type known to the DeleteElement node."),
								FText::FromName(InSelectionName))
							,this, Out
						);

					}

					if (bValidSelectionGroup && !SelectedElements.IsEmpty())
					{
						// Remove the selection since all of its elements will be deleted.
						SelectionFacade.RemoveSelectionSet(InSelectionName);
					}
				}
			}

			const FName GroupName(*Group.Name);
			if (bValidSelectionGroup && SelectionGroup == GroupName)
			{
				// Merge selections
				if (Elements.IsEmpty())
				{
					// Empty elements means delete everything.
					SelectedElements.Reset();
				}
				else
				{
					SelectedElements.Append(Elements);
				}

				DeleteElementsByType(SelectionGroup, SelectedElements);
			}
			else
			{
				if (!SelectedElements.IsEmpty())
				{
					DeleteElementsByType(SelectionGroup, SelectedElements);
				}

				DeleteElementsByType(GroupName, TSet<int32>(Elements));
			}

			if (bDeleteSimMesh)
			{
				FClothGeometryTools::DeleteSimMesh(ClothCollection);
			}

			if (bDeleteRenderMesh)
			{
				FClothGeometryTools::DeleteRenderMesh(ClothCollection);
			}

			FClothGeometryTools::CleanupAndCompactMesh(ClothCollection);
		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

void FChaosClothAssetDeleteElementNode::Serialize(FArchive& Ar)
{
	using namespace UE::Chaos::ClothAsset;

	// This is just for convenience and can be removed post 5.4 once the plugin loses its experimental status
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Ar.IsLoading() && ElementType_DEPRECATED != EChaosClothAssetElementType::Deprecated)
	{
		switch (ElementType_DEPRECATED)
		{
		case EChaosClothAssetElementType::None:
		default:
			break;
		case EChaosClothAssetElementType::SimMesh:
			bDeleteSimMesh = true;
			break;
		case EChaosClothAssetElementType::RenderMesh:
			bDeleteRenderMesh = true;
			break;
		case EChaosClothAssetElementType::SimPattern:
			Group.Name = ClothCollectionGroup::SimPatterns.ToString();
			break;
		case EChaosClothAssetElementType::RenderPattern:
			Group.Name = ClothCollectionGroup::RenderPatterns.ToString();
			break;
		case EChaosClothAssetElementType::SimVertex2D:
			Group.Name = ClothCollectionGroup::SimVertices2D.ToString();
			break;
		case EChaosClothAssetElementType::SimVertex3D:
			Group.Name = ClothCollectionGroup::SimVertices3D.ToString();
			break;
		case EChaosClothAssetElementType::RenderVertex:
			Group.Name = ClothCollectionGroup::RenderVertices.ToString();
			break;
		case EChaosClothAssetElementType::SimFace:
			Group.Name = ClothCollectionGroup::SimFaces.ToString();
			break;
		case EChaosClothAssetElementType::RenderFace:
			Group.Name = ClothCollectionGroup::RenderFaces.ToString();
			break;
		case EChaosClothAssetElementType::Seam:
			Group.Name = ClothCollectionGroup::Seams.ToString();
			break;
		}
		ElementType_DEPRECATED = EChaosClothAssetElementType::Deprecated;  // This is only for clarity since the Type property won't be saved from now on

		FClothDataflowTools::LogAndToastWarning(*this,
			LOCTEXT("DeprecatedDeleteElementTypeHeadline", "Outdated Dataflow asset."),
				LOCTEXT("DeprecatedDeleteElementTypeDetails", "This node is out of data and contain deprecated data. The asset needs to be re-saved before it stops working at the next version update."));
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#undef LOCTEXT_NAMESPACE
