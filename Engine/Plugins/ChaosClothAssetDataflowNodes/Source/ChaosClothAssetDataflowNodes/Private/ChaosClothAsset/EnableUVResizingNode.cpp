// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/EnableUVResizingNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EnableUVResizingNode)

FChaosClothAssetEnableUVResizingNode::FChaosClothAssetEnableUVResizingNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
}

void FChaosClothAssetEnableUVResizingNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyBool(TEXT("EnableUVResizing"), true, {}, ECollectionPropertyFlags::None);
}
