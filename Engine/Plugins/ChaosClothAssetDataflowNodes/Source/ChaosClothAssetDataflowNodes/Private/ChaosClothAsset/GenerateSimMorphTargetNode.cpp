// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/GenerateSimMorphTargetNode.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GenerateSimMorphTargetNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetGenerateSimMorphTargetNode"

FChaosClothAssetGenerateSimMorphTargetNode::FChaosClothAssetGenerateSimMorphTargetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&MorphTargetCollection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&MorphTargetName);
}

void FChaosClothAssetGenerateSimMorphTargetNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		FManagedArrayCollection InMorphTargetCollection = GetValue<FManagedArrayCollection>(Context, &MorphTargetCollection);  // Can't use a const reference here sadly since the facade needs a SharedRef to be created
		const TSharedRef<const FManagedArrayCollection> MorphTargetClothCollection = MakeShared<const FManagedArrayCollection>(MoveTemp(InMorphTargetCollection));

		// Always check for a valid cloth collection/facade to avoid processing non cloth collections
		FCollectionClothFacade ClothFacade(ClothCollection);
		const FCollectionClothConstFacade MorphTargetClothFacade(MorphTargetClothCollection);

		if (ClothFacade.IsValid())
		{
			if (MorphTargetClothFacade.IsValid())
			{
				if (!MorphTargetName.IsEmpty())
				{
					const int32 NumSimVertices = ClothFacade.GetNumSimVertices3D();
					const int32 NumMorphSimVertices = MorphTargetClothFacade.GetNumSimVertices3D();
					if (NumSimVertices == NumMorphSimVertices)
					{
						TConstArrayView<FVector3f> Positions = ClothFacade.GetSimPosition3D();
						TConstArrayView<FVector3f> MorphPositions = MorphTargetClothFacade.GetSimPosition3D();
						TConstArrayView<FVector3f> Normals = ClothFacade.GetSimNormal();
						TConstArrayView<FVector3f> MorphNormals = MorphTargetClothFacade.GetSimNormal();

						TArray<FVector3f> PositionDeltas;
						TArray<FVector3f> NormalDeltas;
						TArray<int32> Indices;
						PositionDeltas.Reserve(NumSimVertices);
						NormalDeltas.Reserve(NumSimVertices);
						Indices.Reserve(NumSimVertices);

						for (int32 Index = 0; Index < NumSimVertices; ++Index)
						{
							const FVector3f PositionDelta = MorphPositions[Index] - Positions[Index];
							const FVector3f NormalDelta = bGenerateNormalDeltas ? MorphNormals[Index] - Normals[Index] : FVector3f::ZeroVector;
							if (PositionDelta.SquaredLength() >= FMath::Square(UE_THRESH_POINTS_ARE_NEAR) ||
								(bGenerateNormalDeltas && NormalDelta.SizeSquared() > FMath::Square(UE_THRESH_VECTORS_ARE_NEAR)))
							{
								PositionDeltas.Add(PositionDelta);
								NormalDeltas.Add(NormalDelta);
								Indices.Add(Index);
							}
						}

						if (PositionDeltas.Num() > 0)
						{
							const int32 ExistingMorphTarget = ClothFacade.FindSimMorphTargetIndexByName(MorphTargetName);
							if (ExistingMorphTarget != INDEX_NONE)
							{
								Context.Warning(
									FText::Format(LOCTEXT("DuplicateMorphTargetDetails", "Existing Sim Morph target with name '{0}' will be replaced."),
										FText::FromString(MorphTargetName))
									,this, Out);
							}

							FCollectionClothSimMorphTargetFacade MorphTargetFacade = ExistingMorphTarget == INDEX_NONE ? ClothFacade.AddGetSimMorphTarget() : ClothFacade.GetSimMorphTarget(ExistingMorphTarget);
							MorphTargetFacade.Initialize(MorphTargetName, PositionDeltas, NormalDeltas, Indices);
						}
						else
						{
							Context.Warning(LOCTEXT("IdenticalCollectionsDetails", "No morph targets generated because the sim collections are identical."), this, Out);
						}
					}
					else
					{
						Context.Warning(
							FText::Format(LOCTEXT("VertexCountMismatchDetail", "Vertex count mismatch {0} != {1}."), NumSimVertices, NumMorphSimVertices)
							, this, Out
						);
					}
				}
			}
			else
			{
				Context.Warning(LOCTEXT("InvalidMorphTargetCollectionDetails", "Input Morph Target Collection is not a valid Cloth Collection."), this, Out);
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
