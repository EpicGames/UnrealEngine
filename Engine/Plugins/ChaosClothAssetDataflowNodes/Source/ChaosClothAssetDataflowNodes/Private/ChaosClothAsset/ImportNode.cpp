// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ImportNode.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowObjectInterface.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Materials/Material.h"
#include "PhysicsEngine/PhysicsAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImportNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetImportNode"

FChaosClothAssetImportNode::FChaosClothAssetImportNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&ClothAsset);
	RegisterOutputConnection(&Collection);
}

void FChaosClothAssetImportNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;
	using namespace Chaos::Softs;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Create a new cloth collection with its LOD 0
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
		FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.DefineSchema();

		if (TObjectPtr<const UChaosClothAsset> InClothAsset = GetValue(Context, &ClothAsset))
		{
			bool bSetValue = true;
			// Copy the main cloth asset details to this dataflow's owner if any
			if (const UE::Dataflow::FEngineContext* EngineContext = Context.AsType<UE::Dataflow::FEngineContext>())
			{
				if (const UChaosClothAsset* const OwnerClothAsset = Cast<UChaosClothAsset>(EngineContext->Owner))
				{
					if (OwnerClothAsset == InClothAsset)
					{
						// Can't create a loop
						bSetValue = false;
						Context.Warning(LOCTEXT("RecursiveAssetLoopDetails", "The source asset cannot be the same as the terminal asset."), this, Out);
					}
				}
			}

			// Copy the cloth asset to this node's output collection
			if (bSetValue)
			{
				const TArray<TSharedRef<const FManagedArrayCollection>>& InClothCollections = InClothAsset->GetClothCollections();
				if (ImportLod >= 0 && InClothCollections.Num() > ImportLod)
				{
					// Copy cloth
					const FCollectionClothConstFacade InClothFacade(InClothCollections[ImportLod]);
					ClothFacade.Initialize(InClothFacade);

					// Copy properties
					const FCollectionPropertyConstFacade InPropertyFacade(InClothCollections[ImportLod]);
					if (InPropertyFacade.IsValid())
					{
						FCollectionPropertyMutableFacade PropertyFacade(ClothCollection);
						PropertyFacade.DefineSchema();
						constexpr bool bUpdateExistingPropertiesFalse = false; // There are no existing properties.
						PropertyFacade.Append(InClothCollections[ImportLod].ToSharedPtr(), bUpdateExistingPropertiesFalse);
					}

					// Copy selections
					const FCollectionClothSelectionConstFacade InSelectionFacade(InClothCollections[ImportLod]);
					if (InSelectionFacade.IsValid())
					{
						FCollectionClothSelectionFacade SelectionFacade(ClothCollection);
						SelectionFacade.DefineSchema();
						constexpr bool bOverwriteExistingIfMismatchedTrue = true;
						SelectionFacade.Append(InSelectionFacade, bOverwriteExistingIfMismatchedTrue);
					}
				}
			}
		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
