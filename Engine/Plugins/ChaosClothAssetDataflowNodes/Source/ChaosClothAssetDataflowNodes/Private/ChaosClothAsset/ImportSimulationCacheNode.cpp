// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ImportSimulationCacheNode.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Chaos/CacheCollection.h"
#include "Chaos/ChaosCache.h"
#include "Dataflow/DataflowInputOutput.h"
#include "TriangleTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ImportSimulationCacheNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetImportSimulationCacheNode"

namespace UE::Chaos::ClothAsset::Private
{
	static void UpdateSimulationPositions(const FCacheEvaluationResult& EvaluatedResult, const int32 ParticleOffset, TArrayView<FVector3f> SimPositions)
	{
		const TArray<float>* const PendingPX = EvaluatedResult.Channels.Find(FName(TEXT("PositionX")));
		const TArray<float>* const PendingPY = EvaluatedResult.Channels.Find(FName(TEXT("PositionY")));
		const TArray<float>* const PendingPZ = EvaluatedResult.Channels.Find(TEXT("PositionZ"));
		check(PendingPX && PendingPY && PendingPZ);
		const int32 NumCachedParticles = EvaluatedResult.ParticleIndices.Num();
		for (int32 CachedIndex = 0; CachedIndex < NumCachedParticles; ++CachedIndex)
		{
			const int32 ParticleIndex = EvaluatedResult.ParticleIndices[CachedIndex] - ParticleOffset;
			if (SimPositions.IsValidIndex(ParticleIndex))
			{
				SimPositions[ParticleIndex].X = (*PendingPX)[CachedIndex];
				SimPositions[ParticleIndex].Y = (*PendingPY)[CachedIndex];
				SimPositions[ParticleIndex].Z = (*PendingPZ)[CachedIndex];
			}
		}
	}

	static void RecalculateSimulationNormals(TConstArrayView<FVector3f> SimPositions, const FCollectionClothConstFacade& ClothFacade, TArrayView<FVector3f> Normals)
	{
		for (FVector3f& Normal : Normals)
		{
			Normal = FVector3f::ZeroVector;
		}

		for (const FIntVector& Element : ClothFacade.GetSimIndices3D())
		{
			UE::Geometry::FTriangle3f Triangle;
			Triangle.V[0] = SimPositions[Element[0]];
			Triangle.V[1] = SimPositions[Element[1]];
			Triangle.V[2] = SimPositions[Element[2]];
			const FVector3f& Normal = Triangle.Normal();
			Normals[Element[0]] += Normal;
			Normals[Element[1]] += Normal;
			Normals[Element[2]] += Normal;
		}

		for (FVector3f& Normal : Normals)
		{
			Normal = Normal.GetSafeNormal(UE_SMALL_NUMBER, FVector3f::XAxisVector);
		}
	}
}

FChaosClothAssetImportSimulationCacheNode::FChaosClothAssetImportSimulationCacheNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&ImportedCache);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetImportSimulationCacheNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		const UChaosCacheCollection* const InCacheCollection = GetValue(Context, &ImportedCache);

		// Always check for a valid cloth collection/facade to avoid processing non cloth collections
		FCollectionClothFacade ClothFacade(ClothCollection);

		if (ClothFacade.IsValid() && InCacheCollection && InCacheCollection->Caches.IsValidIndex(CacheIndex) && InCacheCollection->Caches[CacheIndex])
		{
			const UChaosCache* const InCache = InCacheCollection->Caches[CacheIndex];

			FCacheUserToken Token = InCache->BeginPlayback();

			FPlaybackTickRecord TickRecord;
			TickRecord.SetLastTime(CacheTime);
			TickRecord.SetDt(0.f);
			TickRecord.SetSpaceTransform(Transform);

			FCacheEvaluationContext CacheContext(TickRecord);
			CacheContext.bEvaluateTransform = false;
			CacheContext.bEvaluateCurves = false;
			CacheContext.bEvaluateEvents = false;
			CacheContext.bEvaluateChannels = true;
			CacheContext.bEvaluateNamedTransforms = true;
			const FCacheEvaluationResult EvaluatedResult = InCache->Evaluate(CacheContext, nullptr);

			const TArray<float>* const PendingPX = EvaluatedResult.Channels.Find(FName(TEXT("PositionX")));
			const TArray<float>* const PendingPY = EvaluatedResult.Channels.Find(FName(TEXT("PositionY")));
			const TArray<float>* const PendingPZ = EvaluatedResult.Channels.Find(TEXT("PositionZ"));

			if (PendingPX && PendingPY && PendingPZ)
			{
				if (bUpdateSimulationMesh)
				{
					TArrayView<FVector3f> SimPositions = ClothFacade.GetSimPosition3D();
					Private::UpdateSimulationPositions(EvaluatedResult, ParticleOffset, SimPositions);
					if (bRecalculateNormals)
					{
						Private::RecalculateSimulationNormals(SimPositions, ClothFacade, ClothFacade.GetSimNormal());
					}
				}

				if (bUpdateRenderMesh)
				{
					TArray<FVector3f> CachedSimPositions;
					TArray<FVector3f> CachedSimNormals;
					TArrayView<FVector3f> SimPositions;
					TArrayView<FVector3f> SimNormals;
					if (bUpdateSimulationMesh)
					{
						SimPositions = ClothFacade.GetSimPosition3D();
						if (bRecalculateNormals)
						{
							SimNormals = ClothFacade.GetSimNormal();
						}
						else
						{
							CachedSimNormals.SetNumUninitialized(ClothFacade.GetNumSimVertices3D());
							SimNormals = CachedSimNormals;
							Private::RecalculateSimulationNormals(SimPositions, ClothFacade, SimNormals);
						}
					}
					else
					{
						CachedSimPositions = ClothFacade.GetSimPosition3D();
						SimPositions = CachedSimPositions;
						Private::UpdateSimulationPositions(EvaluatedResult, ParticleOffset, SimPositions);
						CachedSimNormals.SetNumUninitialized(ClothFacade.GetNumSimVertices3D());
						SimNormals = CachedSimNormals;
						Private::RecalculateSimulationNormals(SimPositions, ClothFacade, SimNormals);
					}

					constexpr bool bIgnoreSkinningBlend = false;
					FClothGeometryTools::ApplyProxyDeformer(ClothFacade, bIgnoreSkinningBlend, SimPositions, SimNormals);
				}
			}

			InCache->EndPlayback(Token);
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
