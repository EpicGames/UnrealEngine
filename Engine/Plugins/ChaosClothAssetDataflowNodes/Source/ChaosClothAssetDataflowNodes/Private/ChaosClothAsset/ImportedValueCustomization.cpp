// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ImportedValueCustomization.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetImportedValueCustomization"

namespace UE::Chaos::ClothAsset
{
	
namespace Private
{
	static const FString UseImportedValue = TEXT("UseImportedValue");
	
	static bool UseImportedValueProperty(const TSharedPtr<IPropertyHandle>& Property)
	{
		const FStringView PropertyPath = Property ? Property->GetPropertyPath() : FStringView();
		return PropertyPath.EndsWith(UseImportedValue, ESearchCase::CaseSensitive);
	}
}

void FImportedValueCustomization::AddToggledCheckBox(const TSharedRef<IPropertyHandle>& PropertyHandle, const TSharedPtr<SHorizontalBox>& HorizontalBox, const FSlateBrush* SlateBrush)
{
	TWeakPtr<IPropertyHandle> WeakHandle = PropertyHandle;
	
	HorizontalBox->AddSlot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(FMargin(2.f, 2.f, 2.0f, 2.f))
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Padding(FMargin(0.f, 0.f, 0.0f, 0.f))
			.Style(&FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
			.Type(ESlateCheckBoxType::ToggleButton)
			.ToolTipText(WeakHandle.Pin()->GetToolTipText())
			.IsChecked_Lambda([WeakHandle]()->ECheckBoxState
				{
					bool bValue = false;
					WeakHandle.Pin()->GetValue(bValue);
					return bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([WeakHandle](ECheckBoxState CheckBoxState)
				{
					WeakHandle.Pin()->SetValue(CheckBoxState == ECheckBoxState::Checked, EPropertyValueSetFlags::DefaultFlags);
				})
			[
			SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(SlateBrush)
			]
		];
}

TSharedRef<IPropertyTypeCustomization> FImportedValueCustomization::MakeInstance()
{
	return MakeShareable(new FImportedValueCustomization);
}

FImportedValueCustomization::FImportedValueCustomization() = default;

FImportedValueCustomization::~FImportedValueCustomization() = default;

void FImportedValueCustomization::MakeHeaderRow(TSharedRef<class IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row)
{
	const TWeakPtr<IPropertyHandle> StructWeakHandlePtr = StructPropertyHandle;

	TSharedPtr<SHorizontalBox> ValueHorizontalBox;
	TSharedPtr<SHorizontalBox> NameHorizontalBox;

	Row.NameContent()
		[
			SAssignNew(NameHorizontalBox, SHorizontalBox)
			.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, StructWeakHandlePtr)
		]
	.ValueContent()
		// Make enough space for each child handle
		.MinDesiredWidth(125.f * SortedChildHandles.Num())
		.MaxDesiredWidth(125.f * SortedChildHandles.Num())
		[
			SAssignNew(ValueHorizontalBox, SHorizontalBox)
			.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, StructWeakHandlePtr)
		];
	
	for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];
		
		if (Private::UseImportedValueProperty(ChildHandle))
		{
			AddToggledCheckBox(ChildHandle, NameHorizontalBox, FAppStyle::Get().GetBrush("Icons.Import"));
		}
	}
	
	NameHorizontalBox->AddSlot().VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.Padding(FMargin(4.f, 2.f, 4.0f, 2.f))
			.AutoWidth()
			[
				StructPropertyHandle->CreatePropertyNameWidget()
			];

	for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];
		if (!Private::UseImportedValueProperty(ChildHandle))
		{
			const bool bLastChild = SortedChildHandles.Num() - 1 == ChildIndex;
			
			TSharedRef<SWidget> ChildWidget = MakeChildWidget(StructPropertyHandle, ChildHandle);
			if(ChildWidget != SNullWidget::NullWidget)
			{
				ValueHorizontalBox->AddSlot()
					.Padding(FMargin(0.f, 2.f, bLastChild ? 0.f : 3.f, 2.f))
					[
						ChildWidget
					];
			}
		}
	}
}

TOptional<float> FImportedValueCustomization::GetAxisValue(TSharedRef<IPropertyHandle> InPropertyHandle, EAxis::Type InAxis) const
{
	TOptional<float> Result;

	if (InPropertyHandle->IsValidHandle())
	{
		const uint32 ChildIndex = InAxis == EAxis::X ? 0 : InAxis == EAxis::Y ? 1 : 2;
		float VectorValue = 0.0;
		if (InPropertyHandle->GetChildHandle(ChildIndex)->GetValue(VectorValue) == FPropertyAccess::Success)
		{
			Result = VectorValue;
		}
	}
	return Result;
}

void FImportedValueCustomization::CommitAxisValue(float InNewValue, ETextCommit::Type CommitInfo, TSharedRef<IPropertyHandle> InPropertyHandle, EAxis::Type InAxis) const
{
	if (CommitInfo == ETextCommit::Default)
	{
		return;
	}
	if (InPropertyHandle->IsValidHandle())
	{
		const uint32 ChildIndex = InAxis == EAxis::X ? 0 : InAxis == EAxis::Y ? 1 : 2;
		InPropertyHandle->GetChildHandle(ChildIndex)->SetValue(InNewValue);
	}
}

void FImportedValueCustomization::ChangeAxisValue(float InNewValue, TSharedRef<IPropertyHandle> InPropertyHandle, EAxis::Type InAxis) const
{
	if (InPropertyHandle->IsValidHandle())
	{
		const uint32 ChildIndex = InAxis == EAxis::X ? 0 : InAxis == EAxis::Y ? 1 : 2;
		InPropertyHandle->GetChildHandle(ChildIndex)->SetValue(InNewValue);
	}
}

TSharedRef<SWidget> FImportedValueCustomization::MakeChildWidget(
	TSharedRef<IPropertyHandle>& StructurePropertyHandle,
	TSharedRef<IPropertyHandle>& PropertyHandle)
{
	const FFieldClass* PropertyClass = PropertyHandle->GetPropertyClass();

	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);

	if(NumChildren == 3 && PropertyHandle->GetChildHandle(0)->GetPropertyClass() == FFloatProperty::StaticClass())
	{
		const TWeakPtr<IPropertyHandle> WeakHandlePtr = PropertyHandle;
		
		return SNew(SNumericVectorInputBox<float>)
			.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, WeakHandlePtr)
			.X(this, &FImportedValueCustomization::GetAxisValue, PropertyHandle, EAxis::X)
			.Y(this, &FImportedValueCustomization::GetAxisValue, PropertyHandle, EAxis::Y)
			.Z(this, &FImportedValueCustomization::GetAxisValue, PropertyHandle,  EAxis::Z)
			.OnXChanged(this, &FImportedValueCustomization::ChangeAxisValue, PropertyHandle, EAxis::X)
			.OnYChanged(this, &FImportedValueCustomization::ChangeAxisValue, PropertyHandle, EAxis::Y)
			.OnZChanged(this, &FImportedValueCustomization::ChangeAxisValue, PropertyHandle, EAxis::Z)
			.OnXCommitted(this, &FImportedValueCustomization::CommitAxisValue, PropertyHandle, EAxis::X)
			.OnYCommitted(this, &FImportedValueCustomization::CommitAxisValue, PropertyHandle, EAxis::Y)
			.OnZCommitted(this, &FImportedValueCustomization::CommitAxisValue, PropertyHandle, EAxis::Z)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.bColorAxisLabels(true)
			.AllowSpin(true)
			.SpinDelta(1.0f);
	}
	else if(PropertyClass == FFloatProperty::StaticClass() || PropertyClass == FIntProperty::StaticClass())
	{
		TSharedRef<SWidget> NumericWidget = FMathStructCustomization::MakeChildWidget(StructurePropertyHandle, PropertyHandle);
		NumericEntryBoxWidgetList.Add(NumericWidget);
		return NumericWidget;
	}
	return SNullWidget::NullWidget;
}
	
}  // End namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
