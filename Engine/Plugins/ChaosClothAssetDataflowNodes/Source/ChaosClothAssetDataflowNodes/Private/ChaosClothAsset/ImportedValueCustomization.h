// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Customizations/MathStructCustomizations.h"

template <typename OptionalType> struct TOptional;

namespace UE::Chaos::ClothAsset
{
	/**
	 * Customization for all base property that could be imported
	 * Works like a FMathStructCustomization.
	 */
	class FImportedValueCustomization : public FMathStructCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		FImportedValueCustomization();
		virtual ~FImportedValueCustomization() override;

	protected:
		virtual void MakeHeaderRow(TSharedRef<class IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row) override;
		virtual TSharedRef<SWidget> MakeChildWidget(TSharedRef<IPropertyHandle>& StructurePropertyHandle, TSharedRef<IPropertyHandle>& PropertyHandle) override;

		/** Add a toggled check box to the horizontal one */
		static void AddToggledCheckBox(const TSharedRef<IPropertyHandle>& PropertyHandle, const TSharedPtr<SHorizontalBox>& HorizontalBox, const FSlateBrush* SlateBrush);

	private : 
		TOptional<float> GetAxisValue(TSharedRef<IPropertyHandle> InPropertyHandle, EAxis::Type InAxis) const;
		void CommitAxisValue(float InNewValue, ETextCommit::Type CommitInfo, TSharedRef<IPropertyHandle> InPropertyHandle, EAxis::Type InAxis) const;
		void ChangeAxisValue(float InNewValue, TSharedRef<IPropertyHandle> InPropertyHandle, EAxis::Type InAxis) const;
	};
}
