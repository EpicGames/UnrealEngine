// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/MakeClothAssetNode.h"
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "PhysicsEngine/PhysicsAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MakeClothAssetNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetMakeClothAssetNode"

FChaosClothAssetMakeClothAssetNode::FChaosClothAssetMakeClothAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterOutputConnection(&ClothAsset);
	RegisterInputConnection(&CollectionLodsArray);

	for (int32 Index = 0; Index < NumInitialCollectionLods; ++Index)
	{
		AddPins();
	}
	check(GetNumInputs() == NumRequiredInputs + NumInitialCollectionLods);  // Update NumRequiredInputs when adding inputs (used by Serialize)
}

void FChaosClothAssetMakeClothAssetNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA<TObjectPtr<const UChaosClothAsset>>(&ClothAsset))
	{
		TObjectPtr<UChaosClothAsset> OutClothAsset = NewObject<UChaosClothAsset>();

		const TArray<TSharedRef<const FManagedArrayCollection>> InClothCollections = GetCleanedCollectionLodValues(Context);

		FText ErrorText;
		FText VerboseText;
		OutClothAsset->Build(InClothCollections, nullptr, &ErrorText, &VerboseText);

		if (!ErrorText.IsEmpty())
		{
			// TODO: Use error reporting system
		}

		SetValue<TObjectPtr<const UChaosClothAsset>>(Context, OutClothAsset, &ClothAsset);
	}
}

TArray<UE::Dataflow::FPin> FChaosClothAssetMakeClothAssetNode::AddPins()
{
	const int32 Index = CollectionLods.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FChaosClothAssetMakeClothAssetNode::GetPinsToRemove() const
{
	const int32 Index = CollectionLods.Num() - 1;
	check(CollectionLods.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FChaosClothAssetMakeClothAssetNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = CollectionLods.Num() - 1;
	check(CollectionLods.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	CollectionLods.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

TArray<TSharedRef<const FManagedArrayCollection>> FChaosClothAssetMakeClothAssetNode::GetCleanedCollectionLodValues(UE::Dataflow::FContext& Context) const
{
	using namespace UE::Chaos::ClothAsset;

	TArray<TSharedRef<const FManagedArrayCollection>> CollectionLodValues;
	int32 LastValidLodIndex = INDEX_NONE;
	auto AddCleanedCollectionLod = [this, &Context, &LastValidLodIndex, &CollectionLodValues](int32 LodIndex, TSharedRef<FManagedArrayCollection> CollectionLodValue)
		{
			FCollectionClothFacade ClothFacade(CollectionLodValue);
			if (ClothFacade.HasValidRenderData())  // The cloth collection must at least have a render mesh
			{
				FClothGeometryTools::CleanupAndCompactMesh(CollectionLodValue);
				LastValidLodIndex = LodIndex;
			}
			else if (LastValidLodIndex >= 0)
			{
				ClothFacade.DefineSchema();
				ClothFacade.Initialize(FCollectionClothConstFacade(CollectionLodValues[LastValidLodIndex]));

				Context.Warning(
					FText::Format(
						LOCTEXT("InvalidInputLodNDetails",
							"Invalid or empty input LOD for LOD {0}.\n"
							"Using the previous valid LOD {1} instead."),
						LodIndex,
						LastValidLodIndex)
					, this
				);
			}
			else
			{
				Context.Warning(
					LOCTEXT("InvalidInputLod0Details",
						"Invalid or empty input LOD for LOD 0.\n"
						"LOD 0 cannot be empty in order to construct a valid Cloth Asset.")
					, this
				);
				return;
			}
			CollectionLodValues.Emplace(MoveTemp(CollectionLodValue));
		};


	if (IsConnected(&CollectionLodsArray))
	{
		bool bAlsoHasElementConnections = false;
		for (int32 LodIndex = 0; LodIndex < CollectionLods.Num(); ++LodIndex)
		{
			if (IsConnected(GetConnectionReference(LodIndex)))
			{
				bAlsoHasElementConnections = true;
				break;
			}
		}

		if (bAlsoHasElementConnections)
		{
			Context.Warning(TEXT("Connections found both to CollectionsLodsArray and individual CollectionLods. Only CollectionsLodArray will be used"), this);
		}

		const TArray<FManagedArrayCollection>& InCollectionLodsArray = GetValue(Context, &CollectionLodsArray);

		CollectionLodValues.Reserve(InCollectionLodsArray.Num());

		for (int32 LodIndex = 0; LodIndex < InCollectionLodsArray.Num(); ++LodIndex)
		{
			AddCleanedCollectionLod(LodIndex, MakeShared<FManagedArrayCollection>(InCollectionLodsArray[LodIndex]));
		}
	}
	else
	{
		CollectionLodValues.Reserve(CollectionLods.Num());
		for (int32 LodIndex = 0; LodIndex < CollectionLods.Num(); ++LodIndex)
		{
			AddCleanedCollectionLod(LodIndex, MakeShared<FManagedArrayCollection>(GetValue<FManagedArrayCollection>(Context, GetConnectionReference(LodIndex))));			
		}
	}
	return CollectionLodValues;
}

UE::Dataflow::TConnectionReference<FManagedArrayCollection> FChaosClothAssetMakeClothAssetNode::GetConnectionReference(int32 Index) const
{
	return { &CollectionLods[Index], Index, &CollectionLods };
}

void FChaosClothAssetMakeClothAssetNode::PostSerialize(const FArchive& Ar)
{
	// Added pins need to be restored when loading to make sure they get reconnected
	if (Ar.IsLoading())
	{
		for (int32 Index = 0; Index < NumInitialCollectionLods; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialCollectionLods; Index < CollectionLods.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialCollectionLods);
			const int32 OrigNumCollections = CollectionLods.Num();
			const int32 OrigNumRegisteredCollections = (OrigNumRegisteredInputs - NumRequiredInputs);
			if (OrigNumRegisteredCollections > OrigNumCollections)
			{
				// Inputs have been removed, temporarily expand ClothAssets so we can get connection references
				CollectionLods.SetNum(OrigNumRegisteredCollections);
				for (int32 Index = OrigNumCollections; Index < CollectionLods.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				CollectionLods.SetNum(OrigNumCollections);
			}
		}
		else
		{
			ensureAlways(CollectionLods.Num() + NumRequiredInputs == GetNumInputs());
		}
	}
}

#undef LOCTEXT_NAMESPACE
