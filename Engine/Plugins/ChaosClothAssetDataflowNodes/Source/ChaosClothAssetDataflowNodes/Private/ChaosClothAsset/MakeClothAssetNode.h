// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/ClothLodTransitionDataCache.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "MakeClothAssetNode.generated.h"

class UChaosClothAsset;

/** Cloth terminal node to generate a cloth asset from a cloth collection. */
USTRUCT(Meta = (Experimental, DataflowCloth))
struct FChaosClothAssetMakeClothAssetNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetMakeClothAssetNode, "MakeClothAsset", "Cloth", "Make Cloth Asset ")

public:
	FChaosClothAssetMakeClothAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return true; }
	virtual bool CanRemovePin() const override { return CollectionLods.Num() > NumInitialCollectionLods; }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface

	TArray<TSharedRef<const FManagedArrayCollection>> GetCleanedCollectionLodValues(UE::Dataflow::FContext& Context) const;
	UE::Dataflow::TConnectionReference<FManagedArrayCollection> GetConnectionReference(int32 Index) const;

	/** Input cloth collections for this LOD -- Array connection. Individual CollectionLods will be ignored if there is a CollectionLodsArray connection. */
	UPROPERTY(Meta = (DataflowInput))
	TArray<FManagedArrayCollection> CollectionLodsArray;

	/** Input cloth collection for this LOD. Individual CollectionLods will be ignored if there is a CollectionLodsArray connection. */
	UPROPERTY()
	TArray<FManagedArrayCollection> CollectionLods;

	/** The cloth asset output. */
	UPROPERTY(Meta = (DataflowOutput))
	TObjectPtr<const UChaosClothAsset> ClothAsset;

	static constexpr int32 NumRequiredInputs = 1;
	static constexpr int32 NumInitialCollectionLods = 1;
};
