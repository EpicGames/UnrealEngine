// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/MergeClothCollectionsNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/SoftObjectPath.h"
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MergeClothCollectionsNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetMergeClothCollectionsNode"

namespace UE::Chaos::ClothAsset::Private
{
	static void LogAndToastDifferentWeightMapNames(const FDataflowNode& DataflowNode, const FName& PropertyName, const FString& InWeightMapName, const FString& OutWeightMapName, const FString& WeightMapName)
	{
		using namespace UE::Chaos::ClothAsset;

		static const FText Headline = LOCTEXT("DifferentWeightMapNamesHeadline", "Different weight map names.");

		const FText Details = FText::Format(
			LOCTEXT(
				"DifferentWeightMapNamesDetails",
				"Two identical Cloth Collection properties '{0}' are being merged but have different weight map names '{1}' and '{2}'. The weight map named '{3}' will be used in the resulting merge."),
			FText::FromName(PropertyName),
			FText::FromString(OutWeightMapName),
			FText::FromString(InWeightMapName),
			FText::FromString(WeightMapName));

		FClothDataflowTools::LogAndToastWarning(DataflowNode, Headline, Details);
	}

	struct FMergedProperty
	{
		FString WeightMapName;
		FVector4f PropertyBounds;
	};

	struct FConstraintMergeData
	{
		int32 VertexSpringConstraintIndex = INDEX_NONE;
		int32 VertexFaceSpringConstraintIndex = INDEX_NONE;
		int32 VertexFaceRepulsionConstraintIndex = INDEX_NONE;
		int32 FaceSpringConstraintIndex = INDEX_NONE;

		int32 OtherVertexSpringConstraintIndex = INDEX_NONE;
		int32 OtherVertexFaceSpringConstraintIndex = INDEX_NONE;
		int32 OtherVertexFaceRepulsionConstraintIndex = INDEX_NONE;
		int32 OtherFaceSpringConstraintIndex = INDEX_NONE;

		void ResetOtherData()
		{
			OtherVertexSpringConstraintIndex = INDEX_NONE;
			OtherVertexFaceSpringConstraintIndex = INDEX_NONE; 
			OtherVertexFaceRepulsionConstraintIndex = INDEX_NONE;
			OtherFaceSpringConstraintIndex = INDEX_NONE;
		}
	};

	static void FillWeightMap(const TArrayView<float> WeightMap, const TConstArrayView<float> InWeightMap, const FVector2f& PropertyBounds, const FVector2f& InPropertyBounds)
	{
		const bool bHasAlreadyValues = (InWeightMap.Num() > 0);
		for (int32 VertexIndex = 0; VertexIndex < WeightMap.Num(); ++VertexIndex)
		{
			// If no values in the weight map we are using the low value
			const float WeightMapValue = bHasAlreadyValues ? (InWeightMap[VertexIndex] * (InPropertyBounds[1] - InPropertyBounds[0]) +
				InPropertyBounds[0]) : InPropertyBounds[0];
			WeightMap[VertexIndex] = (WeightMapValue - PropertyBounds[0]) / (PropertyBounds[1] - PropertyBounds[0]);
		}
	}

	/** Build weight maps for each properties if necessary */
	static FString BuildWeightMaps(const FDataflowNode& DataflowNode,
		bool bAppendedCloth,
		const FCollectionClothConstFacade& InClothFacade, FCollectionClothFacade& OutClothFacade,
		const FVector2f& InPropertyBounds, const FVector2f& OutPropertyBounds,
		const FVector2f& PropertyBounds, const FName& PropertyName,
		const FString& InWeightMapName, const FString& OutWeightMapName, TMap<FString,FMergedProperty>& MergedPropertyMaps)
	{
		const FMergedProperty MergedProperty = {OutWeightMapName + FString(TEXT("_")) + InWeightMapName,
			FVector4f(InPropertyBounds[0], InPropertyBounds[1], OutPropertyBounds[0], OutPropertyBounds[1])};

		for(const TPair<FString, FMergedProperty>& MergedPropertyMap : MergedPropertyMaps)
		{
			if((MergedPropertyMap.Value.WeightMapName == MergedProperty.WeightMapName) &&
			   (MergedPropertyMap.Value.PropertyBounds == MergedProperty.PropertyBounds))
			{
				return MergedPropertyMap.Key;
			}
		}
		FString WeightMapName = PropertyName.ToString();
		int32 WeightMapCount = 0;
		
		// the weight map could already been stored on the out collection and linked to different bounds
		// coming from the out collection itself or from previous merge with in collection
		// Since we don't want to break them we need to create a new one on the first available slot
		while(OutClothFacade.GetWeightMap(FName(WeightMapName)).Num() > 0)
		{
			WeightMapName = PropertyName.ToString();
			WeightMapName.AppendInt(++WeightMapCount);
		}
		
		// If the low high values of the merged property are the same we don't need to build a weight map
		if(PropertyBounds[0] != PropertyBounds[1])
		{
			// If names are different we must let the user know
			if ((!InWeightMapName.IsEmpty() && InWeightMapName != WeightMapName) || (!OutWeightMapName.IsEmpty() && OutWeightMapName != WeightMapName))
			{
				Private::LogAndToastDifferentWeightMapNames(DataflowNode, PropertyName, InWeightMapName, OutWeightMapName, WeightMapName);	
			}
			MergedPropertyMaps.Add(WeightMapName,MergedProperty);
			
			// Create if necessary a new weight map
			OutClothFacade.AddWeightMap(FName(WeightMapName));
			TArrayView<float> WeightMap = OutClothFacade.GetWeightMap(FName(WeightMapName));

			const int32 InNumVertices = InClothFacade.GetNumSimVertices3D();
			const int32 OutNumVertices = bAppendedCloth ? OutClothFacade.GetNumSimVertices3D() - InNumVertices : 0;
			check(OutNumVertices >= 0);
			
			FillWeightMap(WeightMap.Left(OutNumVertices), OutClothFacade.GetWeightMap(FName(OutWeightMapName)).Left(OutNumVertices), PropertyBounds, OutPropertyBounds);
			FillWeightMap(WeightMap.Right(InNumVertices), InClothFacade.GetWeightMap(FName(InWeightMapName)), PropertyBounds, InPropertyBounds);
		}
		return WeightMapName;
	}

	// Merge the property bounds of 2 collections
	static FVector2f MergePropertyBounds(const FVector2f& InPropertyBounds, const FVector2f& OutPropertyBounds)
	{
		FVector2f PropertyBounds(0.0f);
		if(InPropertyBounds[0] <= InPropertyBounds[1])
		{
			if(OutPropertyBounds[0] <= OutPropertyBounds[1])
			{
				PropertyBounds[0] = FMath::Min(InPropertyBounds[0], OutPropertyBounds[0]);
				PropertyBounds[1] = FMath::Max(InPropertyBounds[1], OutPropertyBounds[1]);
			}
			else
			{
				PropertyBounds[0] = FMath::Min(InPropertyBounds[0], OutPropertyBounds[1]);
				PropertyBounds[1] = FMath::Max(InPropertyBounds[1], OutPropertyBounds[0]);
			}
		}
		else
		{
			if(OutPropertyBounds[0] <= OutPropertyBounds[1])
			{
				PropertyBounds[0] = FMath::Min(InPropertyBounds[1], OutPropertyBounds[0]);
				PropertyBounds[1] = FMath::Max(InPropertyBounds[0], OutPropertyBounds[1]);
			}
			else
			{
				PropertyBounds[0] = FMath::Min(InPropertyBounds[1], OutPropertyBounds[1]);
				PropertyBounds[1] = FMath::Max(InPropertyBounds[0], OutPropertyBounds[0]);
			}
		}
		if (FMath::IsNearlyEqual(PropertyBounds[0], PropertyBounds[1]))
		{
			PropertyBounds[1] = PropertyBounds[0];
		}
		return PropertyBounds;
	}

	static const TArray<FName> VertexSpringConstraintPropertyNames =
	{
		TEXT("VertexSpringExtensionStiffness"),
		TEXT("VertexSpringCompressionStiffness"),
		TEXT("VertexSpringDamping"),
	};

	static const TArray<FName> VertexFaceSpringConstraintPropertyNames =
	{
		TEXT("VertexFaceSpringExtensionStiffness"),
		TEXT("VertexFaceSpringCompressionStiffness"),
		TEXT("VertexFaceSpringDamping"),
	};

	static const TArray<FName> FaceSpringConstraintPropertyNames =
	{
		TEXT("FaceSpringExtensionStiffness"),
		TEXT("FaceSpringCompressionStiffness"),
		TEXT("FaceSpringDamping"),
	};

	static bool IsSpringConstraintProperty(const FName& PropertyKey)
	{
		return VertexSpringConstraintPropertyNames.Contains(PropertyKey) || VertexFaceSpringConstraintPropertyNames.Contains(PropertyKey) || FaceSpringConstraintPropertyNames.Contains(PropertyKey);
	}

	static void UpdateSpringConstraintWeights(
		bool bAppendedCloth,
		const ::Chaos::Softs::FEmbeddedSpringFacade& InSpringFacade,
		::Chaos::Softs::FEmbeddedSpringFacade& OutSpringFacade,
		const FConstraintMergeData& ConstraintMergeData,
		const FVector2f& InPropertyBounds, const FVector2f& OutPropertyBounds,
		const FVector2f& PropertyBounds, const FName& PropertyName)
	{
		if (PropertyBounds[0] == PropertyBounds[1])
		{
			// If the low high values of the merged property are the same we don't need to build a weight map
			return;
		}
		using namespace ::Chaos::Softs;


		TArrayView<float> WeightMap;
		TConstArrayView<float> InWeightMap;
		if (VertexSpringConstraintPropertyNames.Contains(PropertyName))
		{
			FEmbeddedSpringConstraintFacade OutConstraintFacade = OutSpringFacade.GetSpringConstraint(ConstraintMergeData.VertexSpringConstraintIndex);
			check(ConstraintMergeData.OtherVertexSpringConstraintIndex != INDEX_NONE);
			FEmbeddedSpringConstraintFacade InConstraintFacade = InSpringFacade.GetSpringConstraintConst(ConstraintMergeData.OtherVertexSpringConstraintIndex);

			if (PropertyName == FName(TEXT("VertexSpringExtensionStiffness")))
			{
				WeightMap = OutConstraintFacade.GetExtensionStiffness();
				InWeightMap = InConstraintFacade.GetExtensionStiffnessConst();
			}
			else if (PropertyName == FName(TEXT("VertexSpringCompressionStiffness")))
			{
				WeightMap = OutConstraintFacade.GetCompressionStiffness();
				InWeightMap = InConstraintFacade.GetCompressionStiffnessConst();
			}
			else
			{
				check(PropertyName == FName(TEXT("VertexSpringDamping")));
				WeightMap = OutConstraintFacade.GetDamping();
				InWeightMap = InConstraintFacade.GetDampingConst();
			}
			const int32 InNumSprings = InConstraintFacade.GetNumSprings();
			const int32 OutNumSprings = bAppendedCloth ? OutConstraintFacade.GetNumSprings() - InNumSprings : 0;
			check(OutNumSprings >= 0);
			FillWeightMap(WeightMap.Left(OutNumSprings), WeightMap.Left(OutNumSprings), PropertyBounds, OutPropertyBounds);
			FillWeightMap(WeightMap.Right(InNumSprings), InWeightMap, PropertyBounds, InPropertyBounds);
		}
		else if (VertexFaceSpringConstraintPropertyNames.Contains(PropertyName))
		{
			FEmbeddedSpringConstraintFacade OutConstraintFacade = OutSpringFacade.GetSpringConstraint(ConstraintMergeData.VertexFaceSpringConstraintIndex);
			check(ConstraintMergeData.OtherVertexFaceSpringConstraintIndex != INDEX_NONE);
			FEmbeddedSpringConstraintFacade InConstraintFacade = InSpringFacade.GetSpringConstraintConst(ConstraintMergeData.OtherVertexFaceSpringConstraintIndex);

			if (PropertyName == FName(TEXT("VertexFaceSpringExtensionStiffness")))
			{
				WeightMap = OutConstraintFacade.GetExtensionStiffness();
				InWeightMap = InConstraintFacade.GetExtensionStiffnessConst();
			}
			else if (PropertyName == FName(TEXT("VertexFaceSpringCompressionStiffness")))
			{
				WeightMap = OutConstraintFacade.GetCompressionStiffness();
				InWeightMap = InConstraintFacade.GetCompressionStiffnessConst();
			}
			else
			{
				check(PropertyName == FName(TEXT("VertexFaceSpringDamping")));
				WeightMap = OutConstraintFacade.GetDamping();
				InWeightMap = InConstraintFacade.GetDampingConst();
			}
			const int32 InNumSprings = InConstraintFacade.GetNumSprings();
			const int32 OutNumSprings = bAppendedCloth ? OutConstraintFacade.GetNumSprings() - InNumSprings : 0;
			check(OutNumSprings >= 0);
			FillWeightMap(WeightMap.Left(OutNumSprings), WeightMap.Left(OutNumSprings), PropertyBounds, OutPropertyBounds);
			FillWeightMap(WeightMap.Right(InNumSprings), InWeightMap, PropertyBounds, InPropertyBounds);
		}
		else
		{
			FEmbeddedSpringConstraintFacade OutConstraintFacade = OutSpringFacade.GetSpringConstraint(ConstraintMergeData.FaceSpringConstraintIndex);
			check(ConstraintMergeData.OtherFaceSpringConstraintIndex != INDEX_NONE);
			FEmbeddedSpringConstraintFacade InConstraintFacade = InSpringFacade.GetSpringConstraintConst(ConstraintMergeData.OtherFaceSpringConstraintIndex);

			if (PropertyName == FName(TEXT("FaceSpringExtensionStiffness")))
			{
				WeightMap = OutConstraintFacade.GetExtensionStiffness();
				InWeightMap = InConstraintFacade.GetExtensionStiffnessConst();
			}
			else if (PropertyName == FName(TEXT("FaceSpringCompressionStiffness")))
			{
				WeightMap = OutConstraintFacade.GetCompressionStiffness();
				InWeightMap = InConstraintFacade.GetCompressionStiffnessConst();
			}
			else
			{
				check(PropertyName == FName(TEXT("FaceSpringDamping")));
				WeightMap = OutConstraintFacade.GetDamping();
				InWeightMap = InConstraintFacade.GetDampingConst();
			}
			const int32 InNumSprings = InConstraintFacade.GetNumSprings();
			const int32 OutNumSprings = bAppendedCloth ? OutConstraintFacade.GetNumSprings() - InNumSprings : 0;
			check(OutNumSprings >= 0);
			FillWeightMap(WeightMap.Left(OutNumSprings), WeightMap.Left(OutNumSprings), PropertyBounds, OutPropertyBounds);
			FillWeightMap(WeightMap.Right(InNumSprings), InWeightMap, PropertyBounds, InPropertyBounds);
		}
	}

	static ::Chaos::Softs::ECollectionPropertyFlags MergePropertyFlags(
		const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyFacade,
			  ::Chaos::Softs::FCollectionPropertyMutableFacade& OutPropertyFacade,
			  const int32 InKeyIndex,
			  const int32 OutKeyIndex,
			  const ::Chaos::Softs::ECollectionPropertyFlags InPropertyFlags,
			  const FName& PropertyName)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// TODO: GetFlags needs to return an ECollectionPropertyFlags, not an uint8, but the uint8 getter needs to be deprecated first
		const ::Chaos::Softs::ECollectionPropertyFlags OutPropertyFlags = (::Chaos::Softs::ECollectionPropertyFlags)OutPropertyFacade.GetFlags(OutKeyIndex);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		::Chaos::Softs::ECollectionPropertyFlags PropertyFlags;
		if(!OutPropertyFacade.IsEnabled(OutKeyIndex) && InPropertyFacade.IsEnabled(InKeyIndex))
		{
			PropertyFlags = InPropertyFlags;
		}
		else if(OutPropertyFacade.IsEnabled(OutKeyIndex) && !InPropertyFacade.IsEnabled(InKeyIndex))
		{
			PropertyFlags = OutPropertyFlags;
		}
		else
		{
			PropertyFlags = OutPropertyFlags;
			if(OutPropertyFacade.IsAnimatable(OutKeyIndex) || InPropertyFacade.IsAnimatable(InKeyIndex))
			{
				EnumAddFlags(PropertyFlags, ::Chaos::Softs::ECollectionPropertyFlags::Animatable);  // Animatable
			}
			if(!ensure(OutPropertyFacade.IsIntrinsic(OutKeyIndex) == InPropertyFacade.IsIntrinsic(InKeyIndex)))
			{
				const FString PropertyNameString = PropertyName.ToString();
				UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "MergeClothCollectionsNode: Mismatch in intrinsic flag onto %ls property", *PropertyNameString);
			}
			if(!ensure(OutPropertyFacade.IsLegacy(OutKeyIndex) == InPropertyFacade.IsLegacy(InKeyIndex)))
			{
				const FString PropertyNameString = PropertyName.ToString();
				UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "MergeClothCollectionsNode: Mismatch in legacy flag onto %ls property", *PropertyNameString);
			}
			if(!ensure(OutPropertyFacade.IsInterpolable(OutKeyIndex) == InPropertyFacade.IsInterpolable(InKeyIndex)))
			{
				const FString PropertyNameString = PropertyName.ToString();
				UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "MergeClothCollectionsNode: Mismatch in interpolable flag onto %ls property", *PropertyNameString);
			}
		}
		return PropertyFlags;
	}

	/** Append input properties to the output property facade and add potential weight maps */
	static void AppendInputProperties(const FDataflowNode& DataflowNode,
		bool bAppendedCloth,
		const FCollectionClothConstFacade& InClothFacade,
			  FCollectionClothFacade& OutClothFacade,
		const ::Chaos::Softs::FCollectionPropertyConstFacade& InPropertyFacade,
			  ::Chaos::Softs::FCollectionPropertyMutableFacade& OutPropertyFacade,
		const ::Chaos::Softs::FEmbeddedSpringFacade& InSpringFacade,
			  ::Chaos::Softs::FEmbeddedSpringFacade& OutSpringFacade,
		const FConstraintMergeData& ConstraintMergeData)
	{
		const int32 InNumInKeys = InPropertyFacade.Num();
		TMap<FString,FMergedProperty> MergedPropertyMaps;
		for (int32 InKeyIndex = 0; InKeyIndex < InNumInKeys; ++InKeyIndex)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// TODO: GetFlags needs to return an ECollectionPropertyFlags, not an uint8, but the uint8 getter needs to be deprecated first
			const ::Chaos::Softs::ECollectionPropertyFlags InPropertyFlags = (::Chaos::Softs::ECollectionPropertyFlags)InPropertyFacade.GetFlags(InKeyIndex);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			// Get the matching output key for the given input one 
			const FName& InPropertyKey = InPropertyFacade.GetKeyName(InKeyIndex);
			int32 OutKeyIndex = OutPropertyFacade.GetKeyNameIndex(InPropertyKey);

			// We first check if the output key exists into the output facade
			bool bOverrideProperty = true;
			if(OutKeyIndex != INDEX_NONE)
			{
				if (InPropertyFacade.IsInterpolable(InKeyIndex))
				{
					// If it exists we compute the min of the property low values and the max of the property high values
					const FVector2f InPropertyBounds = InPropertyFacade.GetWeightedFloatValue(InKeyIndex);
					const FVector2f OutPropertyBounds = OutPropertyFacade.GetWeightedFloatValue(OutKeyIndex);
					const FVector2f PropertyBounds = MergePropertyBounds(InPropertyBounds, OutPropertyBounds);

					const ::Chaos::Softs::ECollectionPropertyFlags PropertyFlags = MergePropertyFlags(
						InPropertyFacade, OutPropertyFacade, InKeyIndex, OutKeyIndex, InPropertyFlags, InPropertyKey);

					OutPropertyFacade.SetFlags(OutKeyIndex, PropertyFlags);
					OutPropertyFacade.SetWeightedFloatValue(OutKeyIndex, PropertyBounds);

					if (IsSpringConstraintProperty(InPropertyKey))
					{
						UpdateSpringConstraintWeights(bAppendedCloth, InSpringFacade, OutSpringFacade, ConstraintMergeData, InPropertyBounds, OutPropertyBounds, PropertyBounds, InPropertyKey);
					}
					else
					{
						// We keep the string value to be the one in the output if defined
						const FString WeightMapName = BuildWeightMaps(DataflowNode, bAppendedCloth, InClothFacade, OutClothFacade,
							InPropertyBounds, OutPropertyBounds, PropertyBounds, InPropertyKey,
							InPropertyFacade.GetStringValue(InKeyIndex), OutPropertyFacade.GetStringValue(OutKeyIndex), MergedPropertyMaps);

						OutPropertyFacade.SetStringValue(OutKeyIndex, WeightMapName);
					}
					bOverrideProperty = false;
				}
			}
			else
			{
				// If not we add a new property with the flags/bounds/string of the input one
				if (!OutPropertyFacade.IsValid())
				{
					OutPropertyFacade.DefineSchema();
				}
				OutKeyIndex = OutPropertyFacade.AddProperty(InPropertyKey, InPropertyFlags);
			}
			if(bOverrideProperty)
			{
				OutPropertyFacade.SetFlags(OutKeyIndex, InPropertyFlags);
				OutPropertyFacade.SetWeightedValue(OutKeyIndex, InPropertyFacade.GetLowValue<FVector3f>(InKeyIndex), InPropertyFacade.GetHighValue<FVector3f>(InKeyIndex));
				OutPropertyFacade.SetStringValue(OutKeyIndex, InPropertyFacade.GetStringValue(InKeyIndex));
			}
		}
	}
}

FChaosClothAssetMergeClothCollectionsNode_v2::FChaosClothAssetMergeClothCollectionsNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	check(GetNumInputs() == NumRequiredInputs);

	// Add two sets of pins to start.
	for (int32 Index = 0; Index < NumInitialOptionalInputs; ++Index)
	{
		AddPins();
	}
	RegisterOutputConnection(&Collection)
		.SetPassthroughInput(GetConnectionReference(0));
}

void FChaosClothAssetMergeClothCollectionsNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;
		using namespace Chaos::Softs;

		// Evaluate in collection 0.
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, GetConnectionReference(0));
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		// Keep track of whether any of these collections are valid cloth collections
		FCollectionClothFacade ClothFacade(ClothCollection);
		bool bAreAnyValid = ClothFacade.IsValid();

		// Make it a valid cloth collection if needed
		if (!bAreAnyValid)
		{
			ClothFacade.DefineSchema();
		}

		FCollectionPropertyMutableFacade PropertyFacade(ClothCollection);
		bAreAnyValid |= PropertyFacade.IsValid();

		FCollectionClothSelectionFacade SelectionFacade(ClothCollection);
		bAreAnyValid |= SelectionFacade.IsValid();

		FEmbeddedSpringFacade SpringFacade(ClothCollection.Get(), ClothCollectionGroup::SimVertices3D);
		Private::FConstraintMergeData ConstraintMergeData;
		if (SpringFacade.IsValid())
		{
			for (int32 ConstraintIndex = 0; ConstraintIndex < SpringFacade.GetNumSpringConstraints(); ++ConstraintIndex)
			{
				const FEmbeddedSpringConstraintFacade ConstraintFacade = SpringFacade.GetSpringConstraintConst(ConstraintIndex);
				const FUintVector2 EndPoints = ConstraintFacade.GetConstraintEndPointNumIndices();
				if (EndPoints == FUintVector2(1, 1))
				{
					checkf(ConstraintMergeData.VertexSpringConstraintIndex == INDEX_NONE, TEXT("Multiple vertex spring constraints found"));
					ConstraintMergeData.VertexSpringConstraintIndex = ConstraintIndex;
					bAreAnyValid = true;
				}
				else if (EndPoints == FUintVector2(1, 3))
				{
					const FString& ConstraintName = ConstraintFacade.GetConstraintName();
					if (ConstraintName == TEXT("VertexFaceRepulsionConstraint"))
					{
						checkf(ConstraintMergeData.VertexFaceRepulsionConstraintIndex == INDEX_NONE, TEXT("Multiple vertex-face repulsion constraints found"));
						ConstraintMergeData.VertexFaceRepulsionConstraintIndex = ConstraintIndex;
					}
					else
					{
						checkf(ConstraintMergeData.VertexFaceSpringConstraintIndex == INDEX_NONE, TEXT("Multiple vertex-face spring constraints found"));
						ConstraintMergeData.VertexFaceSpringConstraintIndex = ConstraintIndex;
					}
					bAreAnyValid = true;
				}
				else if (EndPoints == FUintVector2(3, 3))
				{
					checkf(ConstraintMergeData.FaceSpringConstraintIndex == INDEX_NONE, TEXT("Multiple face spring constraints found"));
					ConstraintMergeData.FaceSpringConstraintIndex = ConstraintIndex;
					bAreAnyValid = true;
				}
				else
				{
					checkf(false, TEXT("Unexpected spring constraint type found with end points (%d, %d)"), EndPoints[0], EndPoints[1]);
				}
			}
		}

		// Iterate through the inputs and append them to LOD 0
		for (int32 InputIndex = 1; InputIndex < Collections.Num(); ++InputIndex)
		{
			FManagedArrayCollection OtherCollection = GetValue<FManagedArrayCollection>(Context, GetConnectionReference(InputIndex));  // Can't use a const reference here sadly since the facade needs a SharedRef to be created
			const TSharedRef<const FManagedArrayCollection> OtherClothCollection = MakeShared<const FManagedArrayCollection>(MoveTemp(OtherCollection));

			// Selections need to update with offsets. Gather offsets before appending cloth data.
			const FCollectionClothSelectionConstFacade OtherSelectionFacade(OtherClothCollection);
			TMap<FName, int32> GroupNameOffsets;
			if (OtherSelectionFacade.IsValid())
			{
				const TArray<FName> SelectionNames = OtherSelectionFacade.GetNames();
				for (const FName& SelectionName : SelectionNames)
				{
					const FName GroupName = OtherSelectionFacade.GetSelectionGroup(SelectionName);
					if (!GroupNameOffsets.Find(GroupName))
					{
						GroupNameOffsets.Add(GroupName) = ClothCollection->NumElements(GroupName); // NumElements will return zero if the group doesn't exist.
					}
				}
			}

			// Springs need to update with offsets. Gather offsets before appending cloth data.
			const FEmbeddedSpringFacade OtherEmbeddedSpringFacade(OtherClothCollection.Get(), ClothCollectionGroup::SimVertices3D);
			if (OtherEmbeddedSpringFacade.IsValid())
			{
				if (!GroupNameOffsets.Find(ClothCollectionGroup::SimVertices3D))
				{
					GroupNameOffsets.Add(ClothCollectionGroup::SimVertices3D) = ClothCollection->NumElements(ClothCollectionGroup::SimVertices3D);
				}
			}

			// Append cloth
			const FCollectionClothConstFacade OtherClothFacade(OtherClothCollection);
			bool bAppendedCloth = false;
			if (OtherClothFacade.IsValid())
			{
				FSoftObjectPath RemapSkeletalMeshPath;
				TArray<int32> ThisBoneRemap, OtherBoneRemap;
				FText IncompatibleErrorDetails;
				if (FClothEngineTools::CalculateRemappedBoneIndicesIfCompatible(ClothFacade, OtherClothFacade, RemapSkeletalMeshPath, ThisBoneRemap, OtherBoneRemap, &IncompatibleErrorDetails))
				{
					ClothFacade.SetSkeletalMeshSoftObjectPathName(RemapSkeletalMeshPath);  // The path may change even with no remaps (e.g. when the second skeleton is extended from the first one)
					if (!ThisBoneRemap.IsEmpty())
					{
						FClothEngineTools::RemapBoneIndices(ClothFacade, ThisBoneRemap);
					}

					if (!GroupNameOffsets.Find(ClothCollectionGroup::SimVertices3D))
					{
						GroupNameOffsets.Add(ClothCollectionGroup::SimVertices3D) = ClothCollection->NumElements(ClothCollectionGroup::SimVertices3D);
					}
					if (!GroupNameOffsets.Find(ClothCollectionGroup::RenderVertices))
					{
						GroupNameOffsets.Add(ClothCollectionGroup::RenderVertices) = ClothCollection->NumElements(ClothCollectionGroup::RenderVertices);
					}

					ClothFacade.Append(OtherClothFacade);
					bAreAnyValid = true;
					bAppendedCloth = true;

					if (!OtherBoneRemap.IsEmpty())
					{
						FClothEngineTools::RemapBoneIndices(ClothFacade, OtherBoneRemap, GroupNameOffsets[ClothCollectionGroup::SimVertices3D], GroupNameOffsets[ClothCollectionGroup::RenderVertices]);
					}
				}
				else
				{
					Context.Warning(IncompatibleErrorDetails, this, Out);
				}
			}

			// Append selections (with offsets)
			if (bAppendedCloth && OtherSelectionFacade.IsValid())
			{
				constexpr bool bUpdateExistingSelections = true; // Want last one wins.
				SelectionFacade.AppendWithOffsets(OtherSelectionFacade, bUpdateExistingSelections, GroupNameOffsets);
				bAreAnyValid = true;
			}

			// Append springs (with offsets)
			ConstraintMergeData.ResetOtherData();
			if (bAppendedCloth && OtherEmbeddedSpringFacade.IsValid())
			{
				for (int32 ConstraintIndex = 0; ConstraintIndex < OtherEmbeddedSpringFacade.GetNumSpringConstraints(); ++ConstraintIndex)
				{
					const FEmbeddedSpringConstraintFacade OtherConstraintFacade = OtherEmbeddedSpringFacade.GetSpringConstraintConst(ConstraintIndex);
					const FUintVector2 EndPoints = OtherConstraintFacade.GetConstraintEndPointNumIndices();
					if (EndPoints == FUintVector2(1, 1))
					{
						checkf(ConstraintMergeData.OtherVertexSpringConstraintIndex == INDEX_NONE, TEXT("Multiple vertex spring constraints found"));
						ConstraintMergeData.OtherVertexSpringConstraintIndex = ConstraintIndex;
						if (ConstraintMergeData.VertexSpringConstraintIndex == INDEX_NONE)
						{
							// Create new constraint.
							FEmbeddedSpringConstraintFacade NewConstraintFacade = SpringFacade.AddGetSpringConstraint();
							NewConstraintFacade.Initialize(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
							ConstraintMergeData.VertexSpringConstraintIndex = NewConstraintFacade.GetConstraintIndex();
						}
						else
						{
							// Append to existing constraint
							FEmbeddedSpringConstraintFacade ConstraintFacade = SpringFacade.GetSpringConstraint(ConstraintMergeData.VertexSpringConstraintIndex);
							ConstraintFacade.Append(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
						}

						bAreAnyValid = true;
					}
					else if (EndPoints == FUintVector2(1, 3))
					{
						const FString& ConstraintName = OtherConstraintFacade.GetConstraintName();
						if (ConstraintName == TEXT("VertexFaceRepulsionConstraint"))
						{
							checkf(ConstraintMergeData.OtherVertexFaceRepulsionConstraintIndex == INDEX_NONE, TEXT("Multiple vertex-face repulsion constraints found"));
							ConstraintMergeData.OtherVertexFaceRepulsionConstraintIndex = ConstraintIndex;
							if (ConstraintMergeData.VertexFaceRepulsionConstraintIndex == INDEX_NONE)
							{
								// Create new constraint.
								FEmbeddedSpringConstraintFacade NewConstraintFacade = SpringFacade.AddGetSpringConstraint();
								NewConstraintFacade.Initialize(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
								ConstraintMergeData.VertexFaceRepulsionConstraintIndex = NewConstraintFacade.GetConstraintIndex();
							}
							else
							{
								// Append to existing constraint
								FEmbeddedSpringConstraintFacade ConstraintFacade = SpringFacade.GetSpringConstraint(ConstraintMergeData.VertexFaceRepulsionConstraintIndex);
								ConstraintFacade.Append(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
							}
						}
						else
						{
							checkf(ConstraintMergeData.OtherVertexFaceSpringConstraintIndex == INDEX_NONE, TEXT("Multiple vertex-face spring constraints found"));
							ConstraintMergeData.OtherVertexFaceSpringConstraintIndex = ConstraintIndex;
							if (ConstraintMergeData.VertexFaceSpringConstraintIndex == INDEX_NONE)
							{
								// Create new constraint.
								FEmbeddedSpringConstraintFacade NewConstraintFacade = SpringFacade.AddGetSpringConstraint();
								NewConstraintFacade.Initialize(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
								ConstraintMergeData.VertexFaceSpringConstraintIndex = NewConstraintFacade.GetConstraintIndex();
							}
							else
							{
								// Append to existing constraint
								FEmbeddedSpringConstraintFacade ConstraintFacade = SpringFacade.GetSpringConstraint(ConstraintMergeData.VertexFaceSpringConstraintIndex);
								ConstraintFacade.Append(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
							}
						}
						bAreAnyValid = true;
					}
					else if (EndPoints == FUintVector2(3, 3))
					{
						checkf(ConstraintMergeData.OtherFaceSpringConstraintIndex == INDEX_NONE, TEXT("Multiple face spring constraints found"));
						ConstraintMergeData.OtherFaceSpringConstraintIndex = ConstraintIndex;
						if (ConstraintMergeData.FaceSpringConstraintIndex == INDEX_NONE)
						{
							// Create new constraint.
							FEmbeddedSpringConstraintFacade NewConstraintFacade = SpringFacade.AddGetSpringConstraint();
							NewConstraintFacade.Initialize(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
							ConstraintMergeData.FaceSpringConstraintIndex = NewConstraintFacade.GetConstraintIndex();
						}
						else
						{
							// Append to existing constraint
							FEmbeddedSpringConstraintFacade ConstraintFacade = SpringFacade.GetSpringConstraint(ConstraintMergeData.FaceSpringConstraintIndex);
							ConstraintFacade.Append(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
						}
						bAreAnyValid = true;
					}
					else
					{
						checkf(false, TEXT("Unexpected spring constraint type found with end points (%d, %d)"), EndPoints[0], EndPoints[1]);
					}
				}
			}

			// Copy properties
			const FCollectionPropertyConstFacade OtherPropertyFacade(OtherClothCollection);
			if (OtherPropertyFacade.IsValid())
			{
				// Change that boolean to come back to the old behavior
				static constexpr bool bOverrideProperties = false;
				if(bOverrideProperties)
				{
					constexpr bool bUpdateExistingProperties = true; // Want last one wins.
					PropertyFacade.Append(OtherClothCollection.ToSharedPtr(), bUpdateExistingProperties);
				}
				else
				{
					Private::AppendInputProperties(*this, bAppendedCloth, OtherClothFacade, ClothFacade, OtherPropertyFacade, PropertyFacade, OtherEmbeddedSpringFacade, SpringFacade, ConstraintMergeData);
				}
				bAreAnyValid = true;
			}
		}

		// Set the output
		if (bAreAnyValid)
		{
			// Use the merged cloth collection, but only if there were at least one valid input cloth collections
			SetValue(Context, MoveTemp(*ClothCollection), &Collection);
		}
		else
		{
			// Otherwise pass through the first input unchanged
			SafeForwardInput(Context, GetConnectionReference(0), &Collection);
		}
	}
}

TArray<UE::Dataflow::FPin> FChaosClothAssetMergeClothCollectionsNode_v2::AddPins()
{
	const int32 Index = Collections.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FChaosClothAssetMergeClothCollectionsNode_v2::GetPinsToRemove() const
{
	const int32 Index = Collections.Num() - 1;
	check(Collections.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FChaosClothAssetMergeClothCollectionsNode_v2::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = Collections.Num() - 1;
	check(Collections.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	Collections.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

void FChaosClothAssetMergeClothCollectionsNode_v2::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		if (Collections.Num() < NumInitialOptionalInputs)
		{
			Collections.SetNum(NumInitialOptionalInputs);  // In case the FManagedArrayCollection wasn't serialized with the node (pre the WithSerializer trait)
		}

		for (int32 Index = 0; Index < NumInitialOptionalInputs; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialOptionalInputs; Index < Collections.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialOptionalInputs);
			const int32 OrigNumCollections = Collections.Num();
			const int32 OrigNumRegisteredCollections = OrigNumRegisteredInputs - NumRequiredInputs;
			if (OrigNumRegisteredCollections > OrigNumCollections)
			{
				// Inputs have been removed.
				// Temporarily expand Collections so we can get connection references.
				Collections.SetNum(GetNumInputs() - 1);
				for (int32 Index = OrigNumCollections; Index < Collections.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				Collections.SetNum(OrigNumCollections);
			}
		}
		else
		{
			ensureAlways(Collections.Num() == GetNumInputs());
		}
	}
}

UE::Dataflow::TConnectionReference<FManagedArrayCollection> FChaosClothAssetMergeClothCollectionsNode_v2::GetConnectionReference(int32 Index) const
{
	return { &Collections[Index], Index, &Collections };
}




FChaosClothAssetMergeClothCollectionsNode::FChaosClothAssetMergeClothCollectionsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection)
		.SetPassthroughInput(&Collection);

	check(GetNumInputs() == NumRequiredInputs + NumInitialOptionalInputs); // Update NumRequiredInputs if you add more Inputs. This is used by Serialize.
}

void FChaosClothAssetMergeClothCollectionsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;
		using namespace Chaos::Softs;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		// Keep track of whether any of these collections are valid cloth collections
		FCollectionClothFacade ClothFacade(ClothCollection);
		bool bAreAnyValid = ClothFacade.IsValid();

		// Make it a valid cloth collection if needed
		if (!bAreAnyValid)
		{
			ClothFacade.DefineSchema();
		}

		FCollectionPropertyMutableFacade PropertyFacade(ClothCollection);
		bAreAnyValid |= PropertyFacade.IsValid();

		FCollectionClothSelectionFacade SelectionFacade(ClothCollection);
		bAreAnyValid |= SelectionFacade.IsValid();

		FEmbeddedSpringFacade SpringFacade(ClothCollection.Get(), ClothCollectionGroup::SimVertices3D);
		Private::FConstraintMergeData ConstraintMergeData;
		if (SpringFacade.IsValid())
		{
			bAreAnyValid = true;
			if (ClothFacade.HasUserDefinedAttribute<int32>(TEXT("VertexSpringConstraintIndex"), ClothCollectionGroup::Lods))
			{
				ConstraintMergeData.VertexSpringConstraintIndex = ClothFacade.GetUserDefinedAttribute<int32>(TEXT("VertexSpringConstraintIndex"), ClothCollectionGroup::Lods)[0];
				check(ConstraintMergeData.VertexSpringConstraintIndex == INDEX_NONE || (ConstraintMergeData.VertexSpringConstraintIndex >= 0 && ConstraintMergeData.VertexSpringConstraintIndex < SpringFacade.GetNumSpringConstraints()));
			}
			if (ClothFacade.HasUserDefinedAttribute<int32>(TEXT("VertexFaceSpringConstraintIndex"), ClothCollectionGroup::Lods))
			{
				ConstraintMergeData.VertexFaceSpringConstraintIndex = ClothFacade.GetUserDefinedAttribute<int32>(TEXT("VertexFaceSpringConstraintIndex"), ClothCollectionGroup::Lods)[0];
				check(ConstraintMergeData.VertexFaceSpringConstraintIndex == INDEX_NONE || (ConstraintMergeData.VertexFaceSpringConstraintIndex >= 0 && ConstraintMergeData.VertexFaceSpringConstraintIndex < SpringFacade.GetNumSpringConstraints()));
			}
		}

		// Iterate through the inputs and append them to LOD 0
		const TArray<const FManagedArrayCollection*> Collections = GetCollections();
		for (int32 InputIndex = 1; InputIndex < Collections.Num(); ++InputIndex)
		{
			FManagedArrayCollection OtherCollection = GetValue<FManagedArrayCollection>(Context, Collections[InputIndex]);  // Can't use a const reference here sadly since the facade needs a SharedRef to be created
			const TSharedRef<const FManagedArrayCollection> OtherClothCollection = MakeShared<const FManagedArrayCollection>(MoveTemp(OtherCollection));

			// Selections need to update with offsets. Gather offsets before appending cloth data.
			const FCollectionClothSelectionConstFacade OtherSelectionFacade(OtherClothCollection);
			TMap<FName, int32> GroupNameOffsets;
			if (OtherSelectionFacade.IsValid())
			{
				const TArray<FName> SelectionNames = OtherSelectionFacade.GetNames();
				for (const FName& SelectionName : SelectionNames)
				{
					const FName GroupName = OtherSelectionFacade.GetSelectionGroup(SelectionName);
					if (!GroupNameOffsets.Find(GroupName))
					{
						GroupNameOffsets.Add(GroupName) = ClothCollection->NumElements(GroupName); // NumElements will return zero if the group doesn't exist.
					}
				}
			}

			// Springs need to update with offsets. Gather offsets before appending cloth data.
			const FEmbeddedSpringFacade OtherEmbeddedSpringFacade(OtherClothCollection.Get(), ClothCollectionGroup::SimVertices3D);
			if (OtherEmbeddedSpringFacade.IsValid())
			{
				if (!GroupNameOffsets.Find(ClothCollectionGroup::SimVertices3D))
				{
					GroupNameOffsets.Add(ClothCollectionGroup::SimVertices3D) = ClothCollection->NumElements(ClothCollectionGroup::SimVertices3D);
				}
			}

			// Append cloth
			const FCollectionClothConstFacade OtherClothFacade(OtherClothCollection);
			TArray<int32> RemapBoneIndices;
			bool bAppendedCloth = false;
			if (OtherClothFacade.IsValid())
			{

				FSoftObjectPath RemapSkeletalMeshPath;
				TArray<int32> ThisBoneRemap, OtherBoneRemap;
				FText IncompatibleErrorDetails;
				if (FClothEngineTools::CalculateRemappedBoneIndicesIfCompatible(ClothFacade, OtherClothFacade, RemapSkeletalMeshPath, ThisBoneRemap, OtherBoneRemap, &IncompatibleErrorDetails))
				{
					ClothFacade.SetSkeletalMeshSoftObjectPathName(RemapSkeletalMeshPath);  // The path may change even with no remaps (e.g. when the second skeleton is extended from the first one)
					if (!ThisBoneRemap.IsEmpty())
					{
						FClothEngineTools::RemapBoneIndices(ClothFacade, ThisBoneRemap);
					}

					if (!GroupNameOffsets.Find(ClothCollectionGroup::SimVertices3D))
					{
						GroupNameOffsets.Add(ClothCollectionGroup::SimVertices3D) = ClothCollection->NumElements(ClothCollectionGroup::SimVertices3D);
					}
					if (!GroupNameOffsets.Find(ClothCollectionGroup::RenderVertices))
					{
						GroupNameOffsets.Add(ClothCollectionGroup::RenderVertices) = ClothCollection->NumElements(ClothCollectionGroup::RenderVertices);
					}

					ClothFacade.Append(OtherClothFacade);
					bAreAnyValid = true;
					bAppendedCloth = true;

					if (!OtherBoneRemap.IsEmpty())
					{
						FClothEngineTools::RemapBoneIndices(ClothFacade, OtherBoneRemap, GroupNameOffsets[ClothCollectionGroup::SimVertices3D], GroupNameOffsets[ClothCollectionGroup::RenderVertices]);
					}
				}
				else
				{
					Context.Warning(IncompatibleErrorDetails, this, Out);
				}
			}

			// Append selections (with offsets)
			if (bAppendedCloth && OtherSelectionFacade.IsValid())
			{
				constexpr bool bUpdateExistingSelections = true; // Want last one wins.
				SelectionFacade.AppendWithOffsets(OtherSelectionFacade, bUpdateExistingSelections, GroupNameOffsets);
				bAreAnyValid = true;
			}
			// Append springs (with offsets)
			ConstraintMergeData.ResetOtherData();
			if (bAppendedCloth && OtherEmbeddedSpringFacade.IsValid())
			{
				if (OtherClothFacade.HasUserDefinedAttribute<int32>(TEXT("VertexSpringConstraintIndex"), ClothCollectionGroup::Lods))
				{
					ConstraintMergeData.OtherVertexSpringConstraintIndex = OtherClothFacade.GetUserDefinedAttribute<int32>(TEXT("VertexSpringConstraintIndex"), ClothCollectionGroup::Lods)[0];
					if (ConstraintMergeData.OtherVertexSpringConstraintIndex >= 0 && ConstraintMergeData.OtherVertexSpringConstraintIndex < OtherEmbeddedSpringFacade.GetNumSpringConstraints())
					{
						const FEmbeddedSpringConstraintFacade OtherConstraintFacade = OtherEmbeddedSpringFacade.GetSpringConstraintConst(ConstraintMergeData.OtherVertexSpringConstraintIndex);

						if (ConstraintMergeData.VertexSpringConstraintIndex == INDEX_NONE)
						{
							FEmbeddedSpringConstraintFacade NewConstraintFacade = SpringFacade.AddGetSpringConstraint();
							NewConstraintFacade.Initialize(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
							ConstraintMergeData.VertexSpringConstraintIndex = NewConstraintFacade.GetConstraintIndex();
						}
						else
						{
							FEmbeddedSpringConstraintFacade NewConstraintFacade = SpringFacade.GetSpringConstraint(ConstraintMergeData.VertexSpringConstraintIndex);
							NewConstraintFacade.Append(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
						}
						bAreAnyValid = true;
					}
				}
				if (OtherClothFacade.HasUserDefinedAttribute<int32>(TEXT("VertexFaceSpringConstraintIndex"), ClothCollectionGroup::Lods))
				{
					ConstraintMergeData.OtherVertexFaceSpringConstraintIndex = OtherClothFacade.GetUserDefinedAttribute<int32>(TEXT("VertexFaceSpringConstraintIndex"), ClothCollectionGroup::Lods)[0];
					if (ConstraintMergeData.OtherVertexFaceSpringConstraintIndex >= 0 && ConstraintMergeData.OtherVertexFaceSpringConstraintIndex < OtherEmbeddedSpringFacade.GetNumSpringConstraints())
					{
						const FEmbeddedSpringConstraintFacade OtherConstraintFacade = OtherEmbeddedSpringFacade.GetSpringConstraintConst(ConstraintMergeData.OtherVertexFaceSpringConstraintIndex);

						if (ConstraintMergeData.VertexFaceSpringConstraintIndex == INDEX_NONE)
						{
							FEmbeddedSpringConstraintFacade NewConstraintFacade = SpringFacade.AddGetSpringConstraint();
							NewConstraintFacade.Initialize(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
							ConstraintMergeData.VertexFaceSpringConstraintIndex = NewConstraintFacade.GetConstraintIndex();
						}
						else
						{
							FEmbeddedSpringConstraintFacade NewConstraintFacade = SpringFacade.GetSpringConstraint(ConstraintMergeData.VertexFaceSpringConstraintIndex);
							NewConstraintFacade.Append(OtherConstraintFacade, GroupNameOffsets[ClothCollectionGroup::SimVertices3D]);
						}
						bAreAnyValid = true;
					}
				}
			}

			// Copy properties
			const FCollectionPropertyConstFacade OtherPropertyFacade(OtherClothCollection);
			if (OtherPropertyFacade.IsValid())
			{
				// Change that boolean to come back to the old behavior
				static constexpr bool bOverrideProperties = false;
				if (bOverrideProperties)
				{
					constexpr bool bUpdateExistingProperties = true; // Want last one wins.
					PropertyFacade.Append(OtherClothCollection.ToSharedPtr(), bUpdateExistingProperties);
				}
				else
				{
					Private::AppendInputProperties(*this, bAppendedCloth, OtherClothFacade, ClothFacade, OtherPropertyFacade, PropertyFacade, OtherEmbeddedSpringFacade, SpringFacade, ConstraintMergeData);
				}
				bAreAnyValid = true;
			}
		}

		// Set the output
		if (bAreAnyValid)
		{
			// Use the merged cloth collection, but only if there were at least one valid input cloth collections
			SetValue(Context, MoveTemp(*ClothCollection), &Collection);
		}
		else
		{
			// Otherwise pass through the first input unchanged
			const FManagedArrayCollection& Passthrough = GetValue<FManagedArrayCollection>(Context, &Collection);
			SetValue(Context, Passthrough, &Collection);
		}
	}
}

TArray<UE::Dataflow::FPin> FChaosClothAssetMergeClothCollectionsNode::AddPins()
{
	auto AddInput = [this](const FManagedArrayCollection* InCollection) -> TArray<UE::Dataflow::FPin>
	{
		RegisterInputConnection(InCollection);
		const FDataflowInput* const Input = FindInput(InCollection);
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	};

	switch (NumInputs)
	{
	case 1: ++NumInputs; return AddInput(&Collection1);
	case 2: ++NumInputs; return AddInput(&Collection2);
	case 3: ++NumInputs; return AddInput(&Collection3);
	case 4: ++NumInputs; return AddInput(&Collection4);
	case 5: ++NumInputs; return AddInput(&Collection5);
	default: break;
	}

	return Super::AddPins();
}

TArray<UE::Dataflow::FPin> FChaosClothAssetMergeClothCollectionsNode::GetPinsToRemove() const
{
	auto PinToRemove = [this](const FManagedArrayCollection* InCollection) -> TArray<UE::Dataflow::FPin>
	{
		const FDataflowInput* const Input = FindInput(InCollection);
		check(Input);
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	};

	switch (NumInputs - 1)
	{
	case 1: return PinToRemove(&Collection1);
	case 2: return PinToRemove(&Collection2);
	case 3: return PinToRemove(&Collection3);
	case 4: return PinToRemove(&Collection4);
	case 5: return PinToRemove(&Collection5);
	default: break;
	}
	return Super::GetPinsToRemove();
}

void FChaosClothAssetMergeClothCollectionsNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	auto CheckPinRemoved = [this, &Pin](const FManagedArrayCollection* InCollection)
	{
		check(Pin.Direction == UE::Dataflow::FPin::EDirection::INPUT);
#if DO_CHECK
		const FDataflowInput* const Input = FindInput(InCollection);
		check(Input);
		check(Input->GetName() == Pin.Name);
		check(Input->GetType() == Pin.Type);
#endif
	};

	switch (NumInputs - 1)
	{
	case 1:
		CheckPinRemoved(&Collection1);
		--NumInputs;
		break;
	case 2:
		CheckPinRemoved(&Collection2);
		--NumInputs;
		break;
	case 3:
		CheckPinRemoved(&Collection3);
		--NumInputs;
		break;
	case 4:
		CheckPinRemoved(&Collection4);
		--NumInputs;
		break;
	case 5:
		CheckPinRemoved(&Collection5);
		--NumInputs;
		break;
	default:
		checkNoEntry();
		break;
	}

	return Super::OnPinRemoved(Pin);
}

TArray<const FManagedArrayCollection*> FChaosClothAssetMergeClothCollectionsNode::GetCollections() const
{
	TArray<const FManagedArrayCollection*> Collections;
	Collections.SetNumUninitialized(NumInputs);

	for (int32 InputIndex = 0; InputIndex < NumInputs; ++InputIndex)
	{
		switch (InputIndex)
		{
		case 0: Collections[InputIndex] = &Collection; break;
		case 1: Collections[InputIndex] = &Collection1; break;
		case 2: Collections[InputIndex] = &Collection2; break;
		case 3: Collections[InputIndex] = &Collection3; break;
		case 4: Collections[InputIndex] = &Collection4; break;
		case 5: Collections[InputIndex] = &Collection5; break;
		default: Collections[InputIndex] = nullptr; check(false); break;
		}
	}
	return Collections;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // Unexpected deprecation message on some platforms otherwise
const FManagedArrayCollection* FChaosClothAssetMergeClothCollectionsNode::GetCollection(int32 Index) const
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	switch (Index)
	{
	case 0: return &Collection;
	case 1: return &Collection1;
	case 2: return &Collection2;
	case 3: return &Collection3;
	case 4: return &Collection4;
	case 5: return &Collection5;
	default: check(false) return nullptr;
	}
}

void FChaosClothAssetMergeClothCollectionsNode::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		const int32 OrigNumRegisteredInputs = GetNumInputs() - NumRequiredInputs;
		const int32 OrigNumInputs = NumInputs;
		const int32 NumInputsToAdd = OrigNumInputs - OrigNumRegisteredInputs;
		check(Ar.IsTransacting() || OrigNumRegisteredInputs == NumInitialOptionalInputs)
		if (NumInputsToAdd > 0)
		{
			NumInputs = OrigNumRegisteredInputs;  // AddPin will increment it again
			for (int32 InputIndex = 0; InputIndex < NumInputsToAdd; ++InputIndex)
			{
				AddPins();
			}
		}
		else if (NumInputsToAdd < 0)
		{
			check(Ar.IsTransacting());
			for (int32 Index = NumInputs; Index < OrigNumRegisteredInputs; ++Index)
			{
				UnregisterInputConnection(GetCollection(Index));
			}
		}
		check(NumInputs + NumRequiredInputs == GetNumInputs());
	}
}

#undef LOCTEXT_NAMESPACE
