// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ProceduralSelectionNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProceduralSelectionNode)
#define LOCTEXT_NAMESPACE "FChaosClothAssetProceduralSelectionNode"



FChaosClothAssetProceduralSelectionNode::FChaosClothAssetProceduralSelectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&OutputName);
	RegisterInputConnection(&ConversionInputName.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetProceduralSelectionNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const FName SelectionName(*OutputName);
		if (SelectionName == NAME_None || Group.Name.IsEmpty())
		{
			SafeForwardInput(Context, &Collection, &Collection);
			return;
		}

		const FName SelectionGroupName(*Group.Name);

		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothConstFacade ClothFacade(ClothCollection);

		if (ClothFacade.IsValid())
		{
			FCollectionClothSelectionFacade SelectionFacade(ClothCollection);
			SelectionFacade.DefineSchema();

			switch (SelectionType)
			{
			case EChaosClothAssetProceduralSelectionType::SelectAll:
				FClothGeometryTools::SelectAllInGroupType(ClothCollection, SelectionName, SelectionGroupName);
				SetValue(Context, MoveTemp(*ClothCollection), &Collection);
				return;
			case EChaosClothAssetProceduralSelectionType::Conversion:
			{
				const FName InConversionName(*GetValue(Context, &ConversionInputName.StringValue, ConversionInputName.StringValue));
				if (InConversionName != NAME_None)
				{
					TSet<int32> ConvertedSet;
					if (FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, SelectionName, SelectionGroupName, ConvertedSet))
					{
						SelectionFacade.FindOrAddSelectionSet(InConversionName, SelectionGroupName) = ConvertedSet;
						SetValue(Context, MoveTemp(*ClothCollection), &Collection);
						return;
					}
					else
					{
						Context.Warning(
							FText::Format(LOCTEXT("ConversionFailureDetails", "Failed to convert selection '{0}' to group type '{1}' either because '{0}' does not exist or the conversion type is unsupported."),
								FText::FromName(InConversionName), FText::FromName(SelectionGroupName))
							, this, Out
						);
					}
				}
			}
			break;
			default:
				checkNoEntry();
			}
		}
		SafeForwardInput(Context, &Collection, &Collection);
	}
	else if (Out->IsA<FString>(&OutputName))
	{
		SetValue(Context, OutputName, &OutputName);
	}
}


#undef LOCTEXT_NAMESPACE
