// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ProxyDeformerNode.h"
#include "ChaosClothAsset/ClothCollectionAttribute.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Utils/ClothingMeshUtils.h"
#include "PointWeightMap.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProxyDeformerNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetProxyDeformerNode"

namespace UE::Chaos::ClothAsset::Private
{
	struct FDeformerMappingDataGenerator
	{
		TConstArrayView<FVector3f> SimPositions;
		TConstArrayView<FIntVector3> SimIndices;
		TConstArrayView<FVector3f> RenderPositions;
		TConstArrayView<FVector3f> RenderNormals;
		TConstArrayView<FVector3f> RenderTangentUs;
		TConstArrayView<FIntVector3> RenderIndices;
		FPointWeightMap PointWeightMap;
		TArray<ClothingMeshUtils::FMeshToMeshFilterSet> MeshToMeshFilterSet;

		TArrayView<TArray<FVector4f>> RenderDeformerPositionBaryCoordsAndDist;
		TArrayView<TArray<FVector4f>> RenderDeformerNormalBaryCoordsAndDist;
		TArrayView<TArray<FVector4f>> RenderDeformerTangentBaryCoordsAndDist;
		TArrayView<TArray<FIntVector3>> RenderDeformerSimIndices3D;
		TArrayView<TArray<float>> RenderDeformerWeight;
		TArrayView<float> RenderDeformerSkinningBlend;

		int32 Generate(bool bUseSmoothTransition, bool bUseMultipleInfluences, float InfluenceRadius, bool bDoSkinningBlend)
		{
			check(RenderPositions.Num() == RenderNormals.Num());
			check(RenderPositions.Num() == RenderDeformerPositionBaryCoordsAndDist.Num());
			check(RenderPositions.Num() == RenderDeformerNormalBaryCoordsAndDist.Num());
			check(RenderPositions.Num() == RenderDeformerTangentBaryCoordsAndDist.Num());
			check(RenderPositions.Num() == RenderDeformerSimIndices3D.Num());
			check(RenderPositions.Num() == RenderDeformerWeight.Num());
			check(RenderPositions.Num() == RenderDeformerSkinningBlend.Num());

			if (!ensureMsgf(SimPositions.Num() <= (int32)TNumericLimits<uint16>::Max() + 1, TEXT("FMeshToMeshVertData data is limited to 16bit unsigned int indexes (65536 indices max).")))
			{
				return 0;
			}

			TArray<uint32> ScalarSimIndices;
			ScalarSimIndices.Reserve(SimIndices.Num() * 3);
			for (const FIntVector3& SimIndex : SimIndices)
			{
				ScalarSimIndices.Add(SimIndex[0]);
				ScalarSimIndices.Add(SimIndex[1]);
				ScalarSimIndices.Add(SimIndex[2]);
			}
			TArray<uint32> ScalarRenderIndices;
			ScalarRenderIndices.Reserve(RenderIndices.Num() * 3);
			for (const FIntVector3& RenderIndex : RenderIndices)
			{
				ScalarRenderIndices.Add(RenderIndex[0]);
				ScalarRenderIndices.Add(RenderIndex[1]);
				ScalarRenderIndices.Add(RenderIndex[2]);
			}

			const ClothingMeshUtils::ClothMeshDesc SimMeshDesc(SimPositions, ScalarSimIndices);
			const ClothingMeshUtils::ClothMeshDesc RenderMeshDesc(RenderPositions, RenderNormals, RenderTangentUs, ScalarRenderIndices);

			TArray<FMeshToMeshVertData> MeshToMeshVertData;

			ClothingMeshUtils::GenerateMeshToMeshVertData(
				MeshToMeshVertData,
				RenderMeshDesc,
				SimMeshDesc,
				&PointWeightMap,
				bUseSmoothTransition,
				bUseMultipleInfluences,
				InfluenceRadius,
				MeshToMeshFilterSet);

			const int32 NumInfluences = MeshToMeshVertData.Num() / RenderPositions.Num();
			check(MeshToMeshVertData.Num() == RenderPositions.Num() * NumInfluences);  // Check modulo
			check((!bUseMultipleInfluences && NumInfluences == 1) || (bUseMultipleInfluences && NumInfluences > 1));

			auto IsRenderVertexInFilterSets = [this](int32 Index) -> bool
				{
					for (const ClothingMeshUtils::FMeshToMeshFilterSet& Set : MeshToMeshFilterSet)
					{
						if (Set.TargetVertices.Contains(Index))
						{
							return true;
						}
					}
					return false;
				};

			for (int32 Index = 0; Index < RenderPositions.Num(); ++Index)
			{
				RenderDeformerPositionBaryCoordsAndDist[Index].SetNum(NumInfluences);
				RenderDeformerNormalBaryCoordsAndDist[Index].SetNum(NumInfluences);
				RenderDeformerTangentBaryCoordsAndDist[Index].SetNum(NumInfluences);
				RenderDeformerSimIndices3D[Index].SetNum(NumInfluences);
				RenderDeformerWeight[Index].SetNum(NumInfluences);

				RenderDeformerSkinningBlend[Index] = bDoSkinningBlend || IsRenderVertexInFilterSets(Index) ? 0.f : 1.f;

				for (int32 Influence = 0; Influence < NumInfluences; ++Influence)
				{
					const FMeshToMeshVertData& MeshToMeshVertDatum = MeshToMeshVertData[Index * NumInfluences + Influence];

					RenderDeformerPositionBaryCoordsAndDist[Index][Influence] = MeshToMeshVertDatum.PositionBaryCoordsAndDist;
					RenderDeformerNormalBaryCoordsAndDist[Index][Influence] = MeshToMeshVertDatum.NormalBaryCoordsAndDist;
					RenderDeformerTangentBaryCoordsAndDist[Index][Influence] = MeshToMeshVertDatum.TangentBaryCoordsAndDist;
					RenderDeformerSimIndices3D[Index][Influence] = FIntVector3(
						MeshToMeshVertDatum.SourceMeshVertIndices[0],
						MeshToMeshVertDatum.SourceMeshVertIndices[1],
						MeshToMeshVertDatum.SourceMeshVertIndices[2]);
					RenderDeformerWeight[Index][Influence] = MeshToMeshVertDatum.Weight;

					if (bDoSkinningBlend)
					{
						RenderDeformerSkinningBlend[Index] += MeshToMeshVertDatum.Weight * (float)MeshToMeshVertDatum.SourceMeshVertIndices[3] / (float)TNumericLimits<uint16>::Max();
					}
				}
			}
			return NumInfluences;
		}
	};

	static FPointWeightMap SelectionToPointWeightMap(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const FCollectionClothConstFacade& ClothFacade, const FName& SelectionName)
	{
		constexpr float SelectedValue = 1.f; 
		
		TSet<int32> SimVertex3DSelectionSet;
		if (FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, SelectionName, ClothCollectionGroup::SimVertices3D, SimVertex3DSelectionSet))
		{
			FPointWeightMap PointWeightMap;
			PointWeightMap.Initialize(ClothFacade.GetNumSimVertices3D());  // Init to zero (unselected)
			for (const int32 VertexIndex : SimVertex3DSelectionSet)
			{
				PointWeightMap[VertexIndex] = SelectedValue;
			}
			return PointWeightMap;
		}
		// Invalid or no selection found, all points are dynamic 
		return FPointWeightMap(ClothFacade.GetNumSimVertices3D(), SelectedValue);
	}

	static TArray<ClothingMeshUtils::FMeshToMeshFilterSet> SelectionsToMeshToMeshFilterSets_v2(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const TArray<TPair<FName, FName>>& SelectionNames)
	{
		// Fill up the MeshToMeshFilterSets
		TArray<ClothingMeshUtils::FMeshToMeshFilterSet> MeshToMeshFilterSets;
		MeshToMeshFilterSets.Reserve(SelectionNames.Num());
		for (const TPair<FName, FName>& SelectionName : SelectionNames)
		{
			TSet<int32> SimFaceSelection;
			TSet<int32> RenderVertexSelection;
			if (FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, SelectionName.Get<0>(), ClothCollectionGroup::RenderVertices, RenderVertexSelection)
				&& FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, SelectionName.Get<1>(), ClothCollectionGroup::SimFaces, SimFaceSelection))
			{
				if (!SimFaceSelection.Num() && !RenderVertexSelection.Num())
				{
					continue;  // Nothing selected
				}

				ClothingMeshUtils::FMeshToMeshFilterSet& MeshToMeshFilterSet = MeshToMeshFilterSets.AddDefaulted_GetRef();
				MeshToMeshFilterSet.SourceTriangles = MoveTemp(SimFaceSelection);
				MeshToMeshFilterSet.TargetVertices = MoveTemp(RenderVertexSelection);
			}
		}

		return MeshToMeshFilterSets;
	}
	
	
	static TArray<ClothingMeshUtils::FMeshToMeshFilterSet> SelectionsToMeshToMeshFilterSets(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const TArray<FName>& SelectionNames)
	{
		// Fill up the MeshToMeshFilterSets
		TArray<ClothingMeshUtils::FMeshToMeshFilterSet> MeshToMeshFilterSets;
		{
			MeshToMeshFilterSets.Reserve(SelectionNames.Num());

			for (const FName& SelectionName : SelectionNames)
			{
				TSet<int32> SimFaceSelection;
				bool bRenderVertexSelectionIsSecondary = true;
				if (!FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, SelectionName, ClothCollectionGroup::SimFaces, SimFaceSelection))
				{
					// Try using secondary selection
					bRenderVertexSelectionIsSecondary = false;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
					if(!FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, SelectionName, ClothCollectionGroup::SimFaces, true, SimFaceSelection))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					{
						// Failed to get SimFaces set
						continue;
					}
				}

				if (SimFaceSelection.IsEmpty())
				{
					continue;  // Nothing selected
				}

				TSet<int32> RenderVertexSelection;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, SelectionName, ClothCollectionGroup::RenderVertices, bRenderVertexSelectionIsSecondary, RenderVertexSelection);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

				if (RenderVertexSelection.IsEmpty())
				{
					continue;  // Nothing selected
				}

				ClothingMeshUtils::FMeshToMeshFilterSet& MeshToMeshFilterSet = MeshToMeshFilterSets.AddDefaulted_GetRef();
				MeshToMeshFilterSet.SourceTriangles = MoveTemp(SimFaceSelection);
				MeshToMeshFilterSet.TargetVertices = MoveTemp(RenderVertexSelection);
			}
		}

		return MeshToMeshFilterSets;
	}

}  // End namespace UE::Chaos::ClothAsset::Private

FChaosClothAssetProxyDeformerNode_v3::FChaosClothAssetProxyDeformerNode_v3(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetProxyDeformerNode_v2(InParam, InGuid, true)
{
}

FChaosClothAssetProxyDeformerNode_v2::FChaosClothAssetProxyDeformerNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid, bool bInPreserveRenderTangents)
	: FDataflowNode(InParam, InGuid)
	, bPreserveRenderTangents(bInPreserveRenderTangents)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection)
		.SetPassthroughInput(&Collection);

	// Start with one set of option pins.
	for (int32 Index = 0; Index < NumInitialSelectionFilterSets; ++Index)
	{
		AddPins();
	}

	check(GetNumInputs() == NumRequiredInputs + NumInitialSelectionFilterSets * 2); // Update NumRequiredInputs if you add more Inputs. This is used by Serialize.
}

void FChaosClothAssetProxyDeformerNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		// Always check for a valid cloth collection/facade/sim mesh to avoid processing non cloth collections or pure render mesh cloth assets
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid() && ClothFacade.HasValidData())
		{
			FCollectionClothSelectionFacade SelectionFacade(ClothCollection);

			// Add the optional render deformer schema
			if (!ClothFacade.IsValid(EClothCollectionExtendedSchemas::RenderDeformer))
			{
				ClothFacade.DefineSchema(EClothCollectionExtendedSchemas::RenderDeformer);
			}

			// Create the render weight map for storing the skinning blend weights
			Private::FDeformerMappingDataGenerator DeformerMappingDataGenerator;
			DeformerMappingDataGenerator.SimPositions = ClothFacade.GetSimPosition3D();
			DeformerMappingDataGenerator.SimIndices = ClothFacade.GetSimIndices3D();
			DeformerMappingDataGenerator.RenderPositions = ClothFacade.GetRenderPosition();
			DeformerMappingDataGenerator.RenderNormals = ClothFacade.GetRenderNormal();
			DeformerMappingDataGenerator.RenderTangentUs = bPreserveRenderTangents ? ClothFacade.GetRenderTangentU() : TConstArrayView<FVector3f>();
			DeformerMappingDataGenerator.RenderIndices = ClothFacade.GetRenderIndices();
			DeformerMappingDataGenerator.PointWeightMap = FPointWeightMap(); // _v2 no longer compute SkinningBlend transitions
			DeformerMappingDataGenerator.MeshToMeshFilterSet = Private::SelectionsToMeshToMeshFilterSets_v2(ClothCollection, GetSelectionFilterNames(Context));
			DeformerMappingDataGenerator.RenderDeformerPositionBaryCoordsAndDist = ClothFacade.GetRenderDeformerPositionBaryCoordsAndDist();
			DeformerMappingDataGenerator.RenderDeformerNormalBaryCoordsAndDist = ClothFacade.GetRenderDeformerNormalBaryCoordsAndDist();
			DeformerMappingDataGenerator.RenderDeformerTangentBaryCoordsAndDist = ClothFacade.GetRenderDeformerTangentBaryCoordsAndDist();
			DeformerMappingDataGenerator.RenderDeformerSimIndices3D = ClothFacade.GetRenderDeformerSimIndices3D();
			DeformerMappingDataGenerator.RenderDeformerWeight = ClothFacade.GetRenderDeformerWeight();
			DeformerMappingDataGenerator.RenderDeformerSkinningBlend = ClothFacade.GetRenderDeformerSkinningBlend();

			constexpr bool bUseSmoothTransition = false;  // _v2 no longer compute SkinningBlend transitions
			constexpr bool bDoSkinningBlend = false;  // _v2 no longer compute SkinningBlend transitions
			const int32 NumInfluences = DeformerMappingDataGenerator.Generate(bUseSmoothTransition, bUseMultipleInfluences, InfluenceRadius, bDoSkinningBlend);

			for (int32 RenderPatternIndex = 0; RenderPatternIndex < ClothFacade.GetNumRenderPatterns(); ++RenderPatternIndex)
			{
				FCollectionClothRenderPatternFacade RenderPatternFacade = ClothFacade.GetRenderPattern(RenderPatternIndex);
				RenderPatternFacade.SetRenderDeformerNumInfluences(NumInfluences);
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

TArray<UE::Dataflow::FPin> FChaosClothAssetProxyDeformerNode_v2::AddPins()
{
	const int32 Index = SelectionFilterSets.AddDefaulted();
	TArray<UE::Dataflow::FPin> Pins;
	Pins.Reserve(2);
	{
		const FDataflowInput& Input = RegisterInputArrayConnection(GetRenderConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() });
	}
	{
		const FDataflowInput& Input = RegisterInputArrayConnection(GetSimConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() });
	}
	return Pins;
}

TArray<UE::Dataflow::FPin> FChaosClothAssetProxyDeformerNode_v2::GetPinsToRemove() const
{
	const int32 Index = SelectionFilterSets.Num() - 1;
	check(SelectionFilterSets.IsValidIndex(Index));
	TArray<UE::Dataflow::FPin> Pins;
	Pins.Reserve(2);
	if (const FDataflowInput* const Input = FindInput(GetRenderConnectionReference(Index)))
	{
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() });
	}
	if (const FDataflowInput* const Input = FindInput(GetSimConnectionReference(Index)))
	{
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() });
	}
	return Pins;
}

void FChaosClothAssetProxyDeformerNode_v2::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = SelectionFilterSets.Num() - 1;
	check(SelectionFilterSets.IsValidIndex(Index));
	const FDataflowInput* const FirstInput = FindInput(GetRenderConnectionReference(Index));
	const FDataflowInput* const SecondInput = FindInput(GetSimConnectionReference(Index));
	check(FirstInput || SecondInput);
	const bool bIsFirstInput = FirstInput && FirstInput->GetName() == Pin.Name;
	const bool bIsSecondInput = SecondInput && SecondInput->GetName() == Pin.Name;
	if ((bIsFirstInput && !SecondInput) || (bIsSecondInput && !FirstInput))
	{
		// Both inputs removed. Remove array index.
		SelectionFilterSets.SetNum(Index);
	}
	return Super::OnPinRemoved(Pin);
}

void FChaosClothAssetProxyDeformerNode_v2::PostSerialize(const FArchive& Ar)
{
	// Restore the pins when re-loading so they can get properly reconnected
	if (Ar.IsLoading())
	{
		check(SelectionFilterSets.Num() >= NumInitialSelectionFilterSets);
		for (int32 Index = 0; Index < NumInitialSelectionFilterSets; ++Index)
		{
			check(FindInput(GetRenderConnectionReference(Index)));
			check(FindInput(GetSimConnectionReference(Index)));
		}

		for (int32 Index = NumInitialSelectionFilterSets; Index < SelectionFilterSets.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetRenderConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
			FindOrRegisterInputArrayConnection(GetSimConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		}

		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialSelectionFilterSets * 2);
			const int32 OrigNumSelectionFilterSets = SelectionFilterSets.Num();
			const int32 OrigNumRegisteredSelectionFilterSets = (OrigNumRegisteredInputs - NumRequiredInputs) / 2;

			if (OrigNumRegisteredSelectionFilterSets > OrigNumSelectionFilterSets)
			{
				ensure(Ar.IsTransacting());
				// Temporarily expand SelectionFilterSets so we can get connection references.
				SelectionFilterSets.SetNum(OrigNumRegisteredSelectionFilterSets);
				for (int32 Index = OrigNumSelectionFilterSets; Index < SelectionFilterSets.Num(); ++Index)
				{
					UnregisterInputConnection(GetSimConnectionReference(Index));
					UnregisterInputConnection(GetRenderConnectionReference(Index));
				}
				SelectionFilterSets.SetNum(OrigNumSelectionFilterSets);
			}
		}
		else
		{
			ensureAlways(SelectionFilterSets.Num() * 2 + NumRequiredInputs == GetNumInputs());
		}
	}
}

TArray<TPair<FName, FName>> FChaosClothAssetProxyDeformerNode_v2::GetSelectionFilterNames(UE::Dataflow::FContext& Context) const
{
	TArray<TPair<FName, FName>> SelectionFilterNames;
	SelectionFilterNames.SetNumUninitialized(SelectionFilterSets.Num());

	for (int32 Index = 0; Index < SelectionFilterSets.Num(); ++Index)
	{
		SelectionFilterNames[Index] = MakeTuple<FName, FName>(
			FName(*GetValue(Context, GetRenderConnectionReference(Index))),
			FName(*GetValue(Context, GetSimConnectionReference(Index))));
	}
	return SelectionFilterNames;
}

UE::Dataflow::TConnectionReference<FString> FChaosClothAssetProxyDeformerNode_v2::GetRenderConnectionReference(int32 Index) const
{
	return { &SelectionFilterSets[Index].RenderSelection.StringValue, Index, &SelectionFilterSets };
}

UE::Dataflow::TConnectionReference<FString> FChaosClothAssetProxyDeformerNode_v2::GetSimConnectionReference(int32 Index) const
{
	return { &SelectionFilterSets[Index].SimSelection.StringValue, Index, &SelectionFilterSets };
}


FChaosClothAssetProxyDeformerNode::FChaosClothAssetProxyDeformerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	using namespace UE::Chaos::ClothAsset;
	SimVertexSelection.StringValue = FString();  // An empty selection is an accepted input, but a non existing one isn't
	SkinningBlendName = ClothCollectionAttribute::RenderDeformerSkinningBlend.ToString();

	// If you change the number of InputConnections registered here, you must change the NumRequiredInputs in Serialize()
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&SimVertexSelection.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterInputConnection(&SelectionFilterSet0.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterOutputConnection(&Collection)
		.SetPassthroughInput(&Collection);
	RegisterOutputConnection(&SkinningBlendName);

	check(GetNumInputs() == NumRequiredInputs + NumInitialOptionalInputs); // Update NumRequiredInputs if you add more Inputs. This is used by Serialize.
}

void FChaosClothAssetProxyDeformerNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		const TArray<const FChaosClothAssetConnectableStringValue*> Non0SelectionFilterSets = Get1To9SelectionFilterSets();
		for (int32 FilterSetIndex = 1; FilterSetIndex < NumFilterSets; ++FilterSetIndex)
		{
			const FChaosClothAssetConnectableStringValue& SelectionFilterSet = *Non0SelectionFilterSets[FilterSetIndex - 1];
		}

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		// Always check for a valid cloth collection/facade/sim mesh to avoid processing non cloth collections or pure render mesh cloth assets
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid() && ClothFacade.HasValidData())
		{
			FCollectionClothSelectionFacade SelectionFacade(ClothCollection);

			// Retrieve the SimVertexSelection name
			FName SimVertexSelectionName = FName(*GetValue<FString>(Context, &SimVertexSelection.StringValue));
			if (SimVertexSelectionName != NAME_None && (!SelectionFacade.IsValid() || !SelectionFacade.FindSelectionSet(SimVertexSelectionName)))
			{
				Context.Warning(
					LOCTEXT("HasSimVertexSelectionDetails", "The specified SimVertexSelection does't exist within the input Cloth Collection.")
					, this, Out
				);
				SimVertexSelectionName = NAME_None;
			}

			// Add the optional render deformer schema
			if (!ClothFacade.IsValid(EClothCollectionExtendedSchemas::RenderDeformer))
			{
				ClothFacade.DefineSchema(EClothCollectionExtendedSchemas::RenderDeformer);
			}

			// Create the render weight map for storing the skinning blend weights
			Private::FDeformerMappingDataGenerator DeformerMappingDataGenerator;
			DeformerMappingDataGenerator.SimPositions = ClothFacade.GetSimPosition3D();
			DeformerMappingDataGenerator.SimIndices = ClothFacade.GetSimIndices3D();
			DeformerMappingDataGenerator.RenderPositions = ClothFacade.GetRenderPosition();
			DeformerMappingDataGenerator.RenderNormals = ClothFacade.GetRenderNormal();
			DeformerMappingDataGenerator.RenderTangentUs = TConstArrayView<FVector3f>();
			DeformerMappingDataGenerator.RenderIndices = ClothFacade.GetRenderIndices();
			DeformerMappingDataGenerator.PointWeightMap = Private::SelectionToPointWeightMap(ClothCollection, ClothFacade, SimVertexSelectionName);
			DeformerMappingDataGenerator.MeshToMeshFilterSet = Private::SelectionsToMeshToMeshFilterSets(ClothCollection, GetSelectionFilterNames(Context));
			DeformerMappingDataGenerator.RenderDeformerPositionBaryCoordsAndDist = ClothFacade.GetRenderDeformerPositionBaryCoordsAndDist();
			DeformerMappingDataGenerator.RenderDeformerNormalBaryCoordsAndDist = ClothFacade.GetRenderDeformerNormalBaryCoordsAndDist();
			DeformerMappingDataGenerator.RenderDeformerTangentBaryCoordsAndDist = ClothFacade.GetRenderDeformerTangentBaryCoordsAndDist();
			DeformerMappingDataGenerator.RenderDeformerSimIndices3D = ClothFacade.GetRenderDeformerSimIndices3D();
			DeformerMappingDataGenerator.RenderDeformerWeight = ClothFacade.GetRenderDeformerWeight();
			DeformerMappingDataGenerator.RenderDeformerSkinningBlend = ClothFacade.GetRenderDeformerSkinningBlend();

			constexpr bool bDoSkinningBlend = true;  // Compute SkinningBlend (legacy behavior)
			const int32 NumInfluences = DeformerMappingDataGenerator.Generate(bUseSmoothTransition, bUseMultipleInfluences, InfluenceRadius, bDoSkinningBlend);

			for (int32 RenderPatternIndex = 0; RenderPatternIndex < ClothFacade.GetNumRenderPatterns(); ++RenderPatternIndex)
			{
				FCollectionClothRenderPatternFacade RenderPatternFacade = ClothFacade.GetRenderPattern(RenderPatternIndex);
				RenderPatternFacade.SetRenderDeformerNumInfluences(NumInfluences);
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
	else if (Out->IsA<FString>(&SkinningBlendName))
	{
		SetValue(Context, SkinningBlendName, &SkinningBlendName);
	}
}

TArray<UE::Dataflow::FPin> FChaosClothAssetProxyDeformerNode::AddPins()
{
	check(NumFilterSets >= NumInitialOptionalInputs);
	const FChaosClothAssetConnectableStringValue* const SelectionFilterSet = Get1To9SelectionFilterSets()[NumFilterSets - 1];

	RegisterInputConnection(&SelectionFilterSet->StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	++NumFilterSets;
	const FDataflowInput* const Input = FindInput(SelectionFilterSet);
	check(Input);
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
}

TArray<UE::Dataflow::FPin> FChaosClothAssetProxyDeformerNode::GetPinsToRemove() const
{
	check(NumFilterSets > NumInitialOptionalInputs);
	const FChaosClothAssetConnectableStringValue* const SelectionFilterSet = Get1To9SelectionFilterSets()[NumFilterSets - 2];
	const FDataflowInput* const Input = FindInput(SelectionFilterSet);
	check(Input);
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
}

void FChaosClothAssetProxyDeformerNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	check(NumFilterSets > NumInitialOptionalInputs);
	const FChaosClothAssetConnectableStringValue* const SelectionFilterSet = Get1To9SelectionFilterSets()[NumFilterSets - 2];
	check(Pin.Direction == UE::Dataflow::FPin::EDirection::INPUT);
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(SelectionFilterSet);
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	--NumFilterSets;
	return Super::OnPinRemoved(Pin);
}

void FChaosClothAssetProxyDeformerNode::PostSerialize(const FArchive& Ar)
{
	// Restore the pins when re-loading so they can get properly reconnected
	if (Ar.IsLoading())
	{
		const int32 OrigNumRegisteredInputs = GetNumInputs();
		check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialOptionalInputs);
		const int32 OrigNumSelectionFilterSets = NumFilterSets;
		const int32 OrigNumRegisteredSelectionFilterSets = OrigNumRegisteredInputs - NumRequiredInputs;
		const int32 NumFilterSetsToAdd = (OrigNumSelectionFilterSets - OrigNumRegisteredSelectionFilterSets);
		check(Ar.IsTransacting() || OrigNumRegisteredSelectionFilterSets == NumInitialOptionalInputs);
		if (NumFilterSetsToAdd > 0)
		{
			NumFilterSets = OrigNumRegisteredSelectionFilterSets;  // Reset to default, add pin will increment it again 
			for (int32 Index = 0; Index < NumFilterSetsToAdd; ++Index)
			{
				AddPins();
			}
		}
		else if (NumFilterSetsToAdd < 0)
		{
			check(Ar.IsTransacting());
			TArray<const FChaosClothAssetConnectableStringValue*> Non0SelectionFilterSets = Get1To9SelectionFilterSets();
			for (int32 Index = NumFilterSets; Index < OrigNumRegisteredSelectionFilterSets; ++Index)
			{
				UnregisterInputConnection(Non0SelectionFilterSets[Index-1]);
			}
		}
		check(NumFilterSets + NumRequiredInputs == GetNumInputs());
	}
}

TArray<FName> FChaosClothAssetProxyDeformerNode::GetSelectionFilterNames(UE::Dataflow::FContext& Context) const
{
	check(NumFilterSets > 0);

	TArray<FName> SelectionFilterSets;
	SelectionFilterSets.SetNumUninitialized(NumFilterSets);

	SelectionFilterSets[0] = FName(*GetValue(Context, &SelectionFilterSet0.StringValue));

	TArray<const FChaosClothAssetConnectableStringValue*> Non0SelectionFilterSets = Get1To9SelectionFilterSets();

	for (int32 FilterSetIndex = 1; FilterSetIndex < NumFilterSets; ++FilterSetIndex)
	{
		SelectionFilterSets[FilterSetIndex] = FName(*GetValue(Context, &Non0SelectionFilterSets[FilterSetIndex - 1]->StringValue));
	}
	return SelectionFilterSets;
}

TArray<const FChaosClothAssetConnectableStringValue*> FChaosClothAssetProxyDeformerNode::Get1To9SelectionFilterSets() const
{
	return TArray<const FChaosClothAssetConnectableStringValue*>(
		{
			&SelectionFilterSet1,
			&SelectionFilterSet2,
			&SelectionFilterSet3,
			&SelectionFilterSet4,
			&SelectionFilterSet5,
			&SelectionFilterSet6,
			&SelectionFilterSet7,
			&SelectionFilterSet8,
			&SelectionFilterSet9
		});
}

#undef LOCTEXT_NAMESPACE
