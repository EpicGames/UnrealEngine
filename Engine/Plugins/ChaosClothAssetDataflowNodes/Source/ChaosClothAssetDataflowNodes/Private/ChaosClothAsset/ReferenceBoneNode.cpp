// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ReferenceBoneNode.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Engine/SkeletalMesh.h"
#include "ReferenceSkeleton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReferenceBoneNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetReferenceBoneNode"


FChaosClothAssetReferenceBoneNode::FChaosClothAssetReferenceBoneNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
	, CalculateDefaultReferenceBone(FDataflowFunctionProperty::FDelegate::CreateRaw(this, &FChaosClothAssetReferenceBoneNode::OnCalculateDefaultReferenceBone))
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetReferenceBoneNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;
		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		// Always check for a valid cloth collection/facade/sim mesh to avoid processing non cloth collections or pure render mesh cloth assets
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid() && ClothFacade.HasValidData())
		{
			ClothFacade.SetReferenceBoneName(ReferenceBone.Name);
		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

void FChaosClothAssetReferenceBoneNode::OnCalculateDefaultReferenceBone(UE::Dataflow::FContext& Context)
{
	using namespace UE::Chaos::ClothAsset;

	ReferenceBone.Name = NAME_None;

	FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

	FCollectionClothConstFacade ClothFacade(ClothCollection);

	const FSoftObjectPath& SkeletalMeshPathName = ClothFacade.GetSkeletalMeshSoftObjectPathName();
	if (const USkeletalMesh* const SkeletalMesh = Cast<USkeletalMesh>(SkeletalMeshPathName.TryLoad()))
	{
		const FReferenceSkeleton& RefSkeleton = SkeletalMesh->GetRefSkeleton();
		const int32 ReferenceBoneIndex = FClothEngineTools::CalculateReferenceBoneIndex(ClothCollection, RefSkeleton);
		if (RefSkeleton.GetRawRefBoneInfo().IsValidIndex(ReferenceBoneIndex))
		{
			ReferenceBone.Name = RefSkeleton.GetRawRefBoneInfo()[ReferenceBoneIndex].Name;
		}
	}
}

#undef LOCTEXT_NAMESPACE
