// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/RemeshNode.h"

#include "Algo/Find.h"
#include "BoneWeights.h"
#include "Algo/RemoveIf.h"
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothCollectionAttribute.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/ClothPatternToDynamicMesh.h"
#include "ChaosClothAsset/ClothPatternToDynamicMeshMappingSupport.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/SimulationSelfCollisionSpheresConfigNode.h"
#include "ChaosClothAsset/SimulationLongRangeAttachmentConfigNode.h"
#include "ChaosClothAsset/SimulationMaxDistanceConfigNode.h"
#include "CleaningOps/RemeshMeshOp.h"
#include "CleaningOps/SimplifyMeshOp.h"
#include "Dataflow/DataflowInputOutput.h"
#include "DynamicMesh/MeshTangents.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "IMeshReductionManagerModule.h"
#include "MeshBoundaryLoops.h"
#include "MeshConstraintsUtil.h"
#include "MeshUVChannelInfo.h"
#include "Spatial/PointSetHashTable.h"
#include "SubRegionRemesher.h"

#if WITH_EDITOR
#include "ChaosClothAsset/ClothDataflowViewModes.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "Misc/StringBuilder.h"
#endif


#include UE_INLINE_GENERATED_CPP_BY_NAME(RemeshNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetRemeshNode"

namespace UE::Chaos::ClothAsset::Private
{

	/** 
	 * Given weight values for the edge vertices, compute a new edge length multiplier to be used in the remesher. 
	 * A greater edge length means the edge is more likely to be split, resulting in a higher overall vertex density 
	 */
	static double EdgeScalingFunction(float LocalTriangleDensityWeightValueA, float LocalTriangleDensityWeightValueB)
	{
		// Interpret the weight values as desired local triangle density scalars. E.g. A value of 2.0 on all vertices should roughly double the total number of triangles.
		// To turn this into a scalar that we can multiply the computed edge lengths by during remesing, we take the square root of the vertex weights since AvgEdgeLength \propto Sqrt(AvgTriangleArea)
		// Then we take the simple mean average of the two edge vertex values
		return 0.5f * (FMath::Sqrt(LocalTriangleDensityWeightValueA) + FMath::Sqrt(LocalTriangleDensityWeightValueB));
	}

	//
	// Functions to support seam remeshing
	//

	struct FSeamCollapseParameters
	{
		bool bCanCollapse = false;
		int32 KeepStitchIndex = -1;
		int32 DeleteStitchIndex = -1;
		double NewPositionEdgeParameter = 0.5;
	};

	/**
	 * A "Seam Edge" is a pair of consecutive stitches in a seam. Corresponds to two "mesh" edges.
	 */
	struct FSeamEdge
	{
		FSeamEdge(FIntVector2 A, FIntVector2 B)
		{
			Stitches[0] = A;
			Stitches[1] = B;
		}

		TStaticArray<FIntVector2, 2> Stitches;

		bool operator==(const FSeamEdge& Other) const
		{
			return (Other.Stitches == Stitches);
		}
	};

	// Try to identify if the corner A -- B -- C is "sharp".
	// 
	// The function computes the dot product of normalized vectors AB and BC, and returns true if the result is less than the given threshold
	// Also returns true if A \approx B or B \approx C
	static bool IsSharpCorner(const FVector3d& A, const FVector3d& B, const FVector3d& C, double CosAngleThreshold)
	{
		const FVector3d AB = B - A;
		const FVector3d BC = C - B;
		const double NormAB = AB.Length();
		const double NormBC = BC.Length();

		if (FMath::Abs(NormAB) < UE_SMALL_NUMBER || FMath::Abs(NormBC) < UE_SMALL_NUMBER)
		{
			return true;
		}

		const double CosAngle = AB.Dot(BC) / (NormAB * NormBC);
		return (CosAngle < CosAngleThreshold);
	}

	static FSeamCollapseParameters GetSeamEdgeCollapseParameters(int32 SeamID, int32 StitchID, const UE::Geometry::FDynamicMesh3& Mesh, const TArray<TArray<FIntVector2>>& Seams, double CosAngleCornerTheshold = 0.5)
	{
		using namespace UE::Geometry;

		const FSeamEdge SeamEdge(Seams[SeamID][StitchID], Seams[SeamID][StitchID + 1]);

		// Check if the pair of mesh edges refer to the same edge
		const int EdgeA = Mesh.FindEdge(SeamEdge.Stitches[0][0], SeamEdge.Stitches[1][0]);
		const int EdgeB = Mesh.FindEdge(SeamEdge.Stitches[0][1], SeamEdge.Stitches[1][1]);

		if (SeamEdge.Stitches[0][0] == SeamEdge.Stitches[0][1])
		{
			FSeamCollapseParameters Constraints;
			Constraints.bCanCollapse = false;
			return Constraints;
		}

		if (SeamEdge.Stitches[1][0] == SeamEdge.Stitches[1][1])
		{
			FSeamCollapseParameters Constraints;
			Constraints.bCanCollapse = false;
			return Constraints;
		}

		if (EdgeA == EdgeB)
		{
			// Don't collapse the same edge twice
			FSeamCollapseParameters Constraints;
			Constraints.bCanCollapse = false;
			return Constraints;
		}

		// Check if any vertex exists in another stitch somewhere. For now we will skip these operations
		// TODO: If only one vertex is involved in another stitch somewhere else, we could constrain that vertex to be kept instead
		TStaticArray<int32, 4> SeamEdgeVertices;
		SeamEdgeVertices[0] = SeamEdge.Stitches[0][0];
		SeamEdgeVertices[1] = SeamEdge.Stitches[0][1];
		SeamEdgeVertices[2] = SeamEdge.Stitches[1][0];
		SeamEdgeVertices[3] = SeamEdge.Stitches[1][1];

		for (int32 InnerSeamID = 0; InnerSeamID < Seams.Num(); ++InnerSeamID)
		{
			const TArray<FIntVector2>& InnerSeam = Seams[InnerSeamID];
			for (int32 InnerStitchID = 0; InnerStitchID < InnerSeam.Num(); ++InnerStitchID)
			{
				if (InnerSeamID == SeamID)
				{
					if (InnerStitchID == StitchID || InnerStitchID == StitchID + 1)
					{
						// Don't check against adjacent stitches
						continue;
					}
				}

				const FIntVector2& InnerStitch = InnerSeam[InnerStitchID];

				for (const int32& SeamVertex : SeamEdgeVertices)
				{
					if (SeamVertex == InnerStitch[0] || SeamVertex == InnerStitch[1])
					{
						FSeamCollapseParameters Constraints;
						Constraints.bCanCollapse = false;
						return Constraints;
					}
				}
			}
		}

		// Now check for conditions that might prevent one or the other vertex from being deleted

		bool bStitchIsConstrained = false;
		bool bNextStitchIsConstrained = false;

		// check if stitch is at the beginning or end of the seam
		if (StitchID == 0)
		{
			bStitchIsConstrained = true;
		}
		if (StitchID + 1 == Seams[SeamID].Num() - 1)
		{
			bNextStitchIsConstrained = true;
		}

		// check for vertices that connect two seam sides -- these were previously added by creating a stitch with the same vertex twice
		if (!bStitchIsConstrained && (SeamEdge.Stitches[0][0] == SeamEdge.Stitches[0][1]))
		{
			bStitchIsConstrained = true;
		}
		if (!bNextStitchIsConstrained &&(SeamEdge.Stitches[1][0] == SeamEdge.Stitches[1][1]))
		{
			bNextStitchIsConstrained = true;
		}

		// check if any vertex is at a sharp corner

		for (int32 Side = 0; Side < 2; ++Side)
		{
			if (StitchID > 0)
			{
				const FSeamEdge PrevSeamEdge(Seams[SeamID][StitchID - 1], Seams[SeamID][StitchID]);

				check(PrevSeamEdge.Stitches[1][Side] == SeamEdge.Stitches[0][Side]);
				check(PrevSeamEdge.Stitches[1][Side] == SeamEdge.Stitches[0][Side])

				const FVector3d A = Mesh.GetVertex(PrevSeamEdge.Stitches[0][Side]);
				const FVector3d B = Mesh.GetVertex(PrevSeamEdge.Stitches[1][Side]);
				const FVector3d C = Mesh.GetVertex(SeamEdge.Stitches[1][Side]);

				bStitchIsConstrained = IsSharpCorner(A, B, C, CosAngleCornerTheshold);
			}

			if (StitchID < Seams[SeamID].Num() - 2)
			{
				const FSeamEdge NextSeamEdge(Seams[SeamID][StitchID + 1], Seams[SeamID][StitchID + 2]);

				check(SeamEdge.Stitches[1][Side] == NextSeamEdge.Stitches[0][Side]);
				check(NextSeamEdge.Stitches[0][Side] == SeamEdge.Stitches[1][Side])

				const FVector3d A = Mesh.GetVertex(SeamEdge.Stitches[0][Side]);
				const FVector3d B = Mesh.GetVertex(SeamEdge.Stitches[1][Side]);
				const FVector3d C = Mesh.GetVertex(NextSeamEdge.Stitches[1][Side]);

				bNextStitchIsConstrained = IsSharpCorner(A, B, C, CosAngleCornerTheshold);
			}
		}

		if (bStitchIsConstrained && bNextStitchIsConstrained)
		{
			FSeamCollapseParameters Constraints;
			Constraints.bCanCollapse = false;
			return Constraints;
		}

		FSeamCollapseParameters Constraints;
		Constraints.bCanCollapse = true;

		if (bStitchIsConstrained && !bNextStitchIsConstrained)
		{
			Constraints.KeepStitchIndex = StitchID;
			Constraints.DeleteStitchIndex = StitchID+1;
			Constraints.NewPositionEdgeParameter = 0.0;		// "Keep" vertex should stay where it is
		}
		else if (!bStitchIsConstrained && bNextStitchIsConstrained)
		{
			Constraints.KeepStitchIndex = StitchID+1;
			Constraints.DeleteStitchIndex = StitchID;
			Constraints.NewPositionEdgeParameter = 0.0;	    // "Keep" vertex should stay where it is
		}
		else
		{
			// unconstrained
			Constraints.KeepStitchIndex = StitchID;
			Constraints.DeleteStitchIndex = StitchID + 1;
			Constraints.NewPositionEdgeParameter = 0.5;		// Collapse to the edge midpoint
		}


		// Check if either collapse would fail in FDynamicMesh3::CollapseEdge
		for (int32 Side = 0; Side < 2; ++Side)
		{
			const int KeepVertexIndex = Seams[SeamID][Constraints.KeepStitchIndex][Side];
			const int DeleteVertexIndex = Seams[SeamID][Constraints.DeleteStitchIndex][Side];

			const EMeshResult CanCollapsePreview = Mesh.CanCollapseEdge(KeepVertexIndex, DeleteVertexIndex, Constraints.NewPositionEdgeParameter);
			if (CanCollapsePreview != EMeshResult::Ok)
			{
				Constraints.bCanCollapse = false;
				break;
			}
		}

		return Constraints;
	}

	static bool CanSplitSeamEdge(int32 SeamID, int32 StitchID, const TArray<TArray<FIntVector2>>& Seams)
	{
		using namespace UE::Geometry;

		bool bCanSplit = true;

		const FSeamEdge SeamEdge(Seams[SeamID][StitchID], Seams[SeamID][StitchID + 1]);

		// Check if any vertex exists in another stitch somewhere. For now we will skip these operations
		// TODO: We could probably enable splits if we are very careful about handling mesh edges that are in more than one seam
		TStaticArray<int32, 4> SeamEdgeVertices;
		SeamEdgeVertices[0] = SeamEdge.Stitches[0][0];
		SeamEdgeVertices[1] = SeamEdge.Stitches[0][1];
		SeamEdgeVertices[2] = SeamEdge.Stitches[1][0];
		SeamEdgeVertices[3] = SeamEdge.Stitches[1][1];

		for (int32 InnerSeamID = 0; InnerSeamID < Seams.Num() && bCanSplit; ++InnerSeamID)
		{
			const TArray<FIntVector2>& InnerSeam = Seams[InnerSeamID];
			for (int32 InnerStitchID = 0; InnerStitchID < InnerSeam.Num() && bCanSplit; ++InnerStitchID)
			{
				if (InnerSeamID == SeamID)
				{
					if (InnerStitchID == StitchID || InnerStitchID == StitchID + 1)
					{
						// Don't check against adjacent stitches
						continue;
					}
				}

				const FIntVector2& InnerStitch = InnerSeam[InnerStitchID];

				for (const int32& SeamVertex : SeamEdgeVertices)
				{
					if (SeamVertex == InnerStitch[0] || SeamVertex == InnerStitch[1])
					{
						bCanSplit = false;
						break;
					}
				}
			}
		}


		return bCanSplit;
	}


	static void FindCoincidentBoundaryVertices(const UE::Geometry::FDynamicMesh3& Mesh, TArray<FIntVector2>& Pairs)
	{
		using namespace UE::Geometry;

		constexpr double ProximityTolerance = FMathf::ZeroTolerance;

		TSet<int32> BoundaryVertices;
		for (int32 EdgeID : Mesh.BoundaryEdgeIndicesItr())
		{
			const UE::Geometry::FIndex2i Vertices = Mesh.GetEdgeV(EdgeID);
			BoundaryVertices.Add(Vertices[0]);
			BoundaryVertices.Add(Vertices[1]);
		}

		//
		// Create a spatial hash to speed up matching vertex search (this setup code was mostly copied from FMergeCoincidentMeshEdges)
		//
		
		// use denser grid as vertex count increases
		const int HashN = (Mesh.TriangleCount() < 100000) ? 64 : 128;
		const UE::Geometry::FAxisAlignedBox3d Bounds = Mesh.GetBounds(true);
		const double CellSize = FMath::Max(FMathd::ZeroTolerance, Bounds.MaxDim() / (double)HashN);

		UE::Geometry::FPointSetAdapterd BoundaryVertAdapter;
		BoundaryVertAdapter.MaxPointID = [&Mesh]() { return Mesh.MaxVertexID(); };
		BoundaryVertAdapter.PointCount = [&BoundaryVertices]() { return BoundaryVertices.Num(); };
		BoundaryVertAdapter.IsPoint = [&Mesh](int Idx) { return Mesh.IsVertex(Idx) && Mesh.IsBoundaryVertex(Idx); };
		BoundaryVertAdapter.GetPoint = [&Mesh](int Idx) { return Mesh.GetVertex(Idx); };
		BoundaryVertAdapter.HasNormals = [] { return false; };
		BoundaryVertAdapter.GetPointNormal = [](int Idx) { return FVector3f::UnitY(); };

		FPointSetHashtable BoundaryVertsHash(&BoundaryVertAdapter);
		BoundaryVertsHash.Build(CellSize, Bounds.Min);
		const double UseMergeSearchTol = FMathd::Min(CellSize, 2 * ProximityTolerance);

		// Now look for coincident vertices
		
		TSet<FIntVector2> PairSet;

		for (const int32 VertexAIndex : BoundaryVertices)
		{
			const FVector3d VertexAPosition = Mesh.GetVertex(VertexAIndex);

			TArray<int> NearbyVertices;
			BoundaryVertsHash.FindPointsInBall(VertexAPosition, UseMergeSearchTol, NearbyVertices);

			for (const int32 VertexBIndex : NearbyVertices)
			{
				if (VertexAIndex == VertexBIndex)
				{
					continue;
				}

				if (Mesh.FindEdge(VertexAIndex, VertexBIndex) != UE::Geometry::FDynamicMesh3::InvalidID)
				{
					continue;
				}

				const FVector3d VertexBPosition = Mesh.GetVertex(VertexBIndex);
				const double DistSqr = FVector3d::DistSquared(VertexAPosition, VertexBPosition);

				if (DistSqr < ProximityTolerance * ProximityTolerance)
				{
					const FIntVector2 SortedPair = VertexAIndex < VertexBIndex ? FIntVector2{ VertexAIndex, VertexBIndex } : FIntVector2{ VertexBIndex, VertexAIndex };
					PairSet.Add(SortedPair);
				}
			}
		}

		Pairs = PairSet.Array();
	}


	static bool EdgeIsInSelectedTriangleSet(const UE::Geometry::FDynamicMesh3& Mesh, int32 EdgeIndex, const TSet<int32>& SelectedTriangles)
	{
		if (!Mesh.IsEdge(EdgeIndex))
		{
			return false;
		}

		const UE::Geometry::FIndex2i IncidentTriangles = Mesh.GetEdgeT(EdgeIndex);
		for (const int32 Tri : {IncidentTriangles[0], IncidentTriangles[1]})
		{
			if (Mesh.IsTriangle(Tri) && SelectedTriangles.Contains(Tri))
			{
				return true;
			}
		}

		return false;
	};

	static void RemeshSeams(UE::Geometry::FDynamicMesh3& Mesh, TArray<TArray<FIntVector2>>& Seams, 
		double TargetEdgeLength, 
		const UE::Geometry::FDynamicMeshWeightAttribute* const DensityMapLayer,
		const FVector2f& DensityLowHigh,
		const TSet<int32>* SelectedTriangles)
	{
		using namespace UE::Geometry;

		// constants pulled from FRemesher::SetTargetEdgeLength
		const double MinLength = 0.66 * TargetEdgeLength;
		const double MaxLength = 1.33 * TargetEdgeLength;

		const bool bUsingSelectedTriangles = SelectedTriangles && !SelectedTriangles->IsEmpty();

		for (int32 SeamID = 0; SeamID < Seams.Num(); ++SeamID)
		{
			TArray<FIntVector2>& Seam = Seams[SeamID];

			for (int32 StitchID = 0; StitchID < Seam.Num() - 1; ++StitchID)
			{
				const int32 SideAVertexA = Seam[StitchID][0];
				const int32 SideAVertexB = Seam[StitchID + 1][0];

				const int32 EdgeSideA = Mesh.FindEdge(SideAVertexA, SideAVertexB);
				if (EdgeSideA == UE::Geometry::FDynamicMesh3::InvalidID)
				{
					continue;
				}

				if (bUsingSelectedTriangles && !EdgeIsInSelectedTriangleSet(Mesh, EdgeSideA, *SelectedTriangles))
				{
					continue;
				}

				double EdgeALength = (Mesh.GetVertex(SideAVertexA) - Mesh.GetVertex(SideAVertexB)).Length();

				if (DensityMapLayer)
				{
					float WeightValueA;
					DensityMapLayer->GetValue(SideAVertexA, &WeightValueA);
					WeightValueA = DensityLowHigh[0] * (1.0 - WeightValueA) + DensityLowHigh[1] * WeightValueA;

					float WeightValueB;
					DensityMapLayer->GetValue(SideAVertexB, &WeightValueB);
					WeightValueB = DensityLowHigh[0] * (1.0 - WeightValueB) + DensityLowHigh[1] * WeightValueB;

					EdgeALength = EdgeScalingFunction(WeightValueA, WeightValueB) * EdgeALength;
				}
				else
				{
					EdgeALength = EdgeScalingFunction(DensityLowHigh[0], DensityLowHigh[0]) * EdgeALength;
				}
				

				const int32 SideBVertexA = Seam[StitchID][1];
				const int32 SideBVertexB = Seam[StitchID + 1][1];

				const int32 EdgeSideB = Mesh.FindEdge(SideBVertexA, SideBVertexB);
				if (EdgeSideB == UE::Geometry::FDynamicMesh3::InvalidID)
				{
					continue;
				}

				if (bUsingSelectedTriangles && !EdgeIsInSelectedTriangleSet(Mesh, EdgeSideB, *SelectedTriangles))
				{
					continue;
				}

				double EdgeBLength = (Mesh.GetVertex(SideBVertexA) - Mesh.GetVertex(SideBVertexB)).Length();

				if (DensityMapLayer)
				{
					float WeightValueA;
					DensityMapLayer->GetValue(SideBVertexA, &WeightValueA);
					WeightValueA = DensityLowHigh[0] * (1.0 - WeightValueA) + DensityLowHigh[1] * WeightValueA;

					float WeightValueB;
					DensityMapLayer->GetValue(SideBVertexB, &WeightValueB);
					WeightValueB = DensityLowHigh[0] * (1.0 - WeightValueB) + DensityLowHigh[1] * WeightValueB;

					EdgeBLength = EdgeScalingFunction(WeightValueA, WeightValueB) * EdgeBLength;
				}
				else
				{
					EdgeBLength = EdgeScalingFunction(DensityLowHigh[0], DensityLowHigh[0]) * EdgeBLength;
				}

				if ((EdgeALength < MinLength) && (EdgeBLength < MinLength))
				{
					//
					// Collapse
					//

					const FSeamCollapseParameters CollapseConstraints = GetSeamEdgeCollapseParameters(SeamID, StitchID, Mesh, Seams);
					
					if (!CollapseConstraints.bCanCollapse)
					{
						continue;
					}

					const int32 PatternAKeepVertex = Seam[CollapseConstraints.KeepStitchIndex][0];
					const int32 PatternADeleteVertex = Seam[CollapseConstraints.DeleteStitchIndex][0];
					const int32 PatternBKeepVertex = Seam[CollapseConstraints.KeepStitchIndex][1];
					const int32 PatternBDeleteVertex = Seam[CollapseConstraints.DeleteStitchIndex][1];

					FDynamicMesh3::FEdgeCollapseInfo CollapseInfoA;
					const EMeshResult ResultA = Mesh.CollapseEdge(PatternAKeepVertex, PatternADeleteVertex, CollapseConstraints.NewPositionEdgeParameter, CollapseInfoA);

					FDynamicMesh3::FEdgeCollapseInfo CollapseInfoB;
					const EMeshResult ResultB = Mesh.CollapseEdge(PatternBKeepVertex, PatternBDeleteVertex, CollapseConstraints.NewPositionEdgeParameter, CollapseInfoB);

					check(ResultA == EMeshResult::Ok && ResultB == EMeshResult::Ok)

					Seam.RemoveAt(CollapseConstraints.DeleteStitchIndex);
				}
				else if ((EdgeALength > MaxLength) && (EdgeBLength > MaxLength))
				{
					//
					// Split
					//


					const bool bCanSplit = CanSplitSeamEdge(SeamID, StitchID, Seams);

					if (!bCanSplit)
					{
						continue;
					}

					FDynamicMesh3::FEdgeSplitInfo SplitInfoA;
					const EMeshResult ResultA = Mesh.SplitEdge(SideAVertexA, SideAVertexB, SplitInfoA);

					if (ResultA == EMeshResult::Ok)
					{
						if (Mesh.FindEdge(SideBVertexA, SideBVertexB) == UE::Geometry::FDynamicMesh3::InvalidID)
						{
							// Don't split the same edge twice
							continue;
						}

						FDynamicMesh3::FEdgeSplitInfo SplitInfoB;
						const EMeshResult ResultB = Mesh.SplitEdge(SideBVertexA, SideBVertexB, SplitInfoB);
						check(ResultB == EMeshResult::Ok);

						const FIntVector2 NewStitch{ SplitInfoA.NewVertex, SplitInfoB.NewVertex };

						Seam.Insert(NewStitch, StitchID+1);
					}
				}
			}
		}
	}


	//
	// Boundary remeshing
	//

	struct FEdgeCollapseParameters
	{
		bool bCanCollapse = false;
		int32 KeepVertexIndex = -1;
		int32 DeleteVertexIndex = -1;
		double NewPositionEdgeParameter = 0.5;
	};

	static FEdgeCollapseParameters GetBoundaryEdgeCollapseParameters(const UE::Geometry::FDynamicMesh3& Mesh, const TArray<TArray<FIntVector2>>& Seams, const UE::Geometry::FIndex2i& EdgeVerts, double CosAngleThreshold = 0.5)
	{
		using namespace UE::Geometry;

		auto IsSeamVertex = [](const TArray<TArray<FIntVector2>>& Seams, int VertexIndex) -> bool
		{
			for (const TArray<FIntVector2>& Seam : Seams)
			{
				for (int32 StitchID = 0; StitchID < Seam.Num(); ++StitchID)
				{
					const FIntVector2& Stitch = Seam[StitchID];
					if ((int)Stitch[0] == VertexIndex || (int)Stitch[1] == VertexIndex)
					{
						return true;
					}
				}
			}

			return false;
		};

		auto FindAdjacentBoundaryVertex = [](const UE::Geometry::FDynamicMesh3& Mesh, int VertexID, int ExcludedVertexID) -> int
		{
			for (int EdgeID : Mesh.VtxEdgesItr(VertexID))
			{
				if (!Mesh.IsBoundaryEdge(EdgeID))
				{
					continue;
				}

				const UE::Geometry::FIndex2i Edge = Mesh.GetEdge(EdgeID).Vert;
				const int TestVertexID = Edge[0] == VertexID ? Edge[1] : Edge[0];

				if (TestVertexID != ExcludedVertexID)
				{
					return TestVertexID;
				}
			}

			return UE::Geometry::FDynamicMesh3::InvalidID;
		};


		const int VertexA = EdgeVerts[0];
		const int VertexB = EdgeVerts[1];

		// Check for conditions that might prevent one or the other vertex from being deleted
		bool bVertexIsConstrained = false;
		bool bNextVertexIsConstrained = false;

		// Check if either vertex is on a seam

		if (IsSeamVertex(Seams, VertexA))
		{
			bVertexIsConstrained = true;
		}
	
		if (IsSeamVertex(Seams, VertexB))
		{
			bNextVertexIsConstrained = true;
		}

		// check if either vertex is at a sharp boundary corner

		int OtherVertex = FindAdjacentBoundaryVertex(Mesh, VertexA, VertexB);
		if ((OtherVertex != FDynamicMesh3::InvalidID) && !bVertexIsConstrained)
		{
			const FVector3d A = Mesh.GetVertex(OtherVertex);
			const FVector3d B = Mesh.GetVertex(VertexA);
			const FVector3d C = Mesh.GetVertex(VertexB);

			bVertexIsConstrained = IsSharpCorner(A, B, C, CosAngleThreshold);
		}

		OtherVertex = FindAdjacentBoundaryVertex(Mesh, VertexB, VertexA);
		if ((OtherVertex != FDynamicMesh3::InvalidID) && !bNextVertexIsConstrained)
		{
			const FVector3d A = Mesh.GetVertex(VertexA);
			const FVector3d B = Mesh.GetVertex(VertexB);
			const FVector3d C = Mesh.GetVertex(OtherVertex);

			bNextVertexIsConstrained = IsSharpCorner(A, B, C, CosAngleThreshold);
		}

		if (bVertexIsConstrained && bNextVertexIsConstrained)
		{
			FEdgeCollapseParameters Constraints;
			Constraints.bCanCollapse = false;
			return Constraints;
		}

		FEdgeCollapseParameters Constraints;
		Constraints.bCanCollapse = true;

		if (bVertexIsConstrained && !bNextVertexIsConstrained)
		{
			Constraints.KeepVertexIndex = VertexA;
			Constraints.DeleteVertexIndex = VertexB;
			Constraints.NewPositionEdgeParameter = 0.0;		// "Keep" vertex should stay where it is
		}
		else if (!bVertexIsConstrained && bNextVertexIsConstrained)
		{
			Constraints.KeepVertexIndex = VertexB;
			Constraints.DeleteVertexIndex = VertexA;
			Constraints.NewPositionEdgeParameter = 0.0;	    // "Keep" vertex should stay where it is
		}
		else
		{
			// unconstrained
			Constraints.KeepVertexIndex = VertexA;
			Constraints.DeleteVertexIndex = VertexB;
			Constraints.NewPositionEdgeParameter = 0.5;		// Collapse to the edge midpoint
		}

		return Constraints;
	}

	static void RemeshBoundaries(UE::Geometry::FDynamicMesh3& Mesh, 
		const TArray<TArray<FIntVector2>>& Seams, 
		double TargetEdgeLength, 
		const UE::Geometry::FDynamicMeshWeightAttribute* const DensityMapLayer,
		const FVector2f& DensityLowHigh, 
		const TSet<int32>* SelectedTriangles)
	{
		using namespace UE::Geometry;

		auto IsSeamEdge = [](const TArray<TArray<FIntVector2>>& Seams, int EdgeVertexA, int EdgeVertexB)
		{
			for (const TArray<FIntVector2>& Seam : Seams)
			{
				for (int32 StitchID = 0; StitchID < Seam.Num() - 1; ++StitchID)
				{
					const FIntVector2& Stitch = Seam[StitchID];
					const FIntVector2& NextStitch = Seam[StitchID + 1];

					for (int32 Side = 0; Side < 2; ++Side)
					{
						if ((int)Stitch[Side] == EdgeVertexA && (int)NextStitch[Side] == EdgeVertexB)
						{
							return true;
						}

						if ((int)Stitch[Side] == EdgeVertexB && (int)NextStitch[Side] == EdgeVertexA)
						{
							return true;
						}
					}
				}
			}

			return false;
		};

		// constants pulled from FRemesher::SetTargetEdgeLength
		const double MinLength = 0.66 * TargetEdgeLength;
		const double MaxLength = 1.33 * TargetEdgeLength;

		const bool bUsingSelectedTriangles = SelectedTriangles &&!SelectedTriangles->IsEmpty();

		// Get the set of boundary edges up front and then check if they get invalidated later. 
		// If we process edges inside this loop it tends to collapse a bunch of sequential edges, which can lead to very high-valence vertices
		TArray<FIndex2i> BoundaryEdges;
		for (const int EdgeID : Mesh.BoundaryEdgeIndicesItr())
		{
			BoundaryEdges.Add(Mesh.GetEdge(EdgeID).Vert);
		}

		for (const FIndex2i& EdgeVerts : BoundaryEdges)
		{
			const int EdgeID = Mesh.FindEdge(EdgeVerts[0], EdgeVerts[1]);
			if (EdgeID == FDynamicMesh3::InvalidID || !Mesh.IsBoundaryEdge(EdgeID))
			{
				continue;
			}

			if (IsSeamEdge(Seams, EdgeVerts[0], EdgeVerts[1]))
			{
				continue;
			}

			if (bUsingSelectedTriangles && !EdgeIsInSelectedTriangleSet(Mesh, EdgeID, *SelectedTriangles))
			{
				continue;
			}

			double EdgeLength = (Mesh.GetVertex(EdgeVerts[0]) - Mesh.GetVertex(EdgeVerts[1])).Length();

			if (DensityMapLayer)
			{
				float WeightValueA;
				DensityMapLayer->GetValue(EdgeVerts[0], &WeightValueA);
				WeightValueA = DensityLowHigh[0] * (1.0 - WeightValueA) + DensityLowHigh[1] * WeightValueA;

				float WeightValueB;
				DensityMapLayer->GetValue(EdgeVerts[1], &WeightValueB);
				WeightValueB = DensityLowHigh[0] * (1.0 - WeightValueB) + DensityLowHigh[1] * WeightValueB;

				EdgeLength = EdgeScalingFunction(WeightValueA, WeightValueB) * EdgeLength;
			}
			else
			{
				EdgeLength = EdgeScalingFunction(DensityLowHigh[0], DensityLowHigh[0]) * EdgeLength;
			}


			if (EdgeLength < MinLength)
			{
				const UE::Chaos::ClothAsset::Private::FEdgeCollapseParameters CollapseParams = Private::GetBoundaryEdgeCollapseParameters(Mesh, Seams, EdgeVerts);

				if (CollapseParams.bCanCollapse)
				{
					FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
					EMeshResult CollapseResult = Mesh.CollapseEdge(CollapseParams.KeepVertexIndex, CollapseParams.DeleteVertexIndex, CollapseParams.NewPositionEdgeParameter, CollapseInfo);
				}
			}
			else if (EdgeLength > MaxLength)
			{
				FDynamicMesh3::FEdgeSplitInfo SplitInfo;
				EMeshResult SplitResult = Mesh.SplitEdge(EdgeVerts[0], EdgeVerts[1], SplitInfo);
			}
		};
	}

	/**
	* Custom subclass of FRestrictedSubRegionRemesher that will update its own constraints. Typically users of FRestrictedSubRegionRemesher will interleave
	* calls to BasicRemeshPass() with constraint updates, but this class will handle it internally.
	*/
	class FClothSubRegionRemesher : public UE::Geometry::FRestrictedSubRegionRemesher
	{
	public:
		FClothSubRegionRemesher(UE::Geometry::FDynamicMesh3* Mesh, const TSet<int>& InTriangleROI) :
			UE::Geometry::FRestrictedSubRegionRemesher(Mesh, InTriangleROI)
		{
		}
	private:
		virtual void BasicRemeshPass() override
		{
			UpdateROI();
			// Reconstrain ROI boundaries
			UE::Geometry::FMeshConstraints EditConstraints;
			if (GetConstraints())
			{
				EditConstraints = *GetConstraints();
			}
			constexpr bool bAllowSplits = false;
			constexpr bool bAllowSmoothing = false;
			UE::Geometry::FMeshConstraintsUtil::ConstrainROIBoundariesInEdgeROI(EditConstraints, *GetMesh(), GetCurrentEdgeROI(), GetCurrentTriangleROI(), bAllowSplits, bAllowSmoothing);
			SetExternalConstraints(EditConstraints);
			UE::Geometry::FRestrictedSubRegionRemesher::BasicRemeshPass();
		}
	};

	/**
	* Custom RemeshOp that uses FClothSubRegionRemesher
	*/
	class FClothRemeshOp : public UE::Geometry::FRemeshMeshOp
	{
	public:
		FClothRemeshOp(const TSet<int32>& TriangleROI) :
			TriangleROI(TriangleROI)
		{
		};
		TSet<int32> TriangleROI;
	private:
		virtual TUniquePtr<UE::Geometry::FRemesher> CreateRemesher(ERemeshType Type, UE::Geometry::FDynamicMesh3* TargetMesh) override
		{
			if (!TriangleROI.IsEmpty())
			{
				return MakeUnique<FClothSubRegionRemesher>(TargetMesh, TriangleROI);
			}
			return UE::Geometry::FRemeshMeshOp::CreateRemesher(Type, TargetMesh);
		}
	};

	//
	// Remeshing away from Seams/Boundaries
	// 
	struct FRemeshParams
	{
		double TargetEdgeLength = 100;
		int32 Iterations = 10;
		float SmoothingRate = 0.25;
		bool bUniformSmoothing = true;
		const TSet<int32>* FaceSelection = nullptr;
		FString DensityMapName;
		FVector2f DensityLowHigh = FVector2f(0,1);
		UE::Geometry::FCompactMaps* CompactMaps = nullptr;
	};

	static bool Remesh(UE::Geometry::FDynamicMesh3& Mesh, 
		const FRemeshParams& Params,
		const TArray<TArray<FIntVector2>>& Seams)
	{
		using namespace UE::Geometry;

		//
		// These consts control overall remeshing behavior and are analogs of the properties exposed to the user in the actual Remesh tool
		// (i.e. things we might wish to add to the node properties in the future)
		//

		constexpr bool bReprojectToInputMesh = true;
		constexpr bool bDiscardAttributes = false;
		constexpr bool bUseFullRemeshPasses = true;
		constexpr bool bAllowFlips = true;
		constexpr bool bAllowSplits = true;
		constexpr bool bAllowCollapses = true;
		constexpr bool bPreventNormalFlips = true;
		constexpr bool bPreventTinyTriangles = true;
		constexpr bool bAutoCompact = true;
		constexpr bool bCoarsenBoundaries = false;

		const ERemeshSmoothingType SmoothingType = Params.bUniformSmoothing ? ERemeshSmoothingType::Uniform : ERemeshSmoothingType::MeanValue;

		// Mesh seam behavior
		// Here we are talking UV, Normal, and Color seams, not Cloth Seams
		// This controls bAllowSeamCollapse and bAllowSeamSmoothing on those overlay seams
		// NOTE: These seams are not affected by bReprojectConstraints
		constexpr bool bPreserveSharpEdges = false;

		// Mesh boundaries
		const EEdgeRefineFlags MeshBoundaryConstraint = bCoarsenBoundaries ? EEdgeRefineFlags::NoFlip : EEdgeRefineFlags::FullyConstrained;

		// Group ID boundaries
		const EEdgeRefineFlags PolyGroupBoundaryConstraint = bCoarsenBoundaries ? EEdgeRefineFlags::NoFlip : EEdgeRefineFlags::FullyConstrained;

		// Material ID boundaries
		const EEdgeRefineFlags MaterialBoundaryConstraint = bCoarsenBoundaries ? EEdgeRefineFlags::NoFlip : EEdgeRefineFlags::FullyConstrained;

		// Whether to move boundary vertices back onto the poly-line defined by the original boundary in case of collapse
		const bool bReprojectConstraints = bCoarsenBoundaries;

		// Seam "corners" are held fixed. We use this angle threshold to determine what constitutes a seam corner
		constexpr float SeamCornerThresholdAngleDegrees = 45.0;

		TSet<int32> EmptySelection;
		FClothRemeshOp RemeshOp(Params.FaceSelection? *Params.FaceSelection: EmptySelection);

		TSharedPtr<FDynamicMesh3> SourceMesh = MakeShared<FDynamicMesh3>(MoveTemp(Mesh));
		TSharedPtr<FDynamicMeshAABBTree3> SourceSpatial;
		if (bReprojectToInputMesh)
		{
			// acceleration structure is only used for reprojecting
			SourceSpatial = MakeShared<FDynamicMeshAABBTree3>(SourceMesh.Get(), true);
		}

		RemeshOp.OriginalMesh = SourceMesh;
		RemeshOp.OriginalMeshSpatial = SourceSpatial;

		RemeshOp.bDiscardAttributes = bDiscardAttributes;
		RemeshOp.RemeshType = (bUseFullRemeshPasses) ? ERemeshType::FullPass : ERemeshType::Standard;
		RemeshOp.RemeshIterations = Params.Iterations;
		RemeshOp.MaxRemeshIterations = Params.Iterations;
		RemeshOp.ExtraProjectionIterations = 0;		// unused for regular remeshing
		RemeshOp.TriangleCountHint = 0;				// unused for regular remeshing
		RemeshOp.SmoothingStrength = FMath::Clamp(Params.SmoothingRate, 0.0f, 1.0f);
		RemeshOp.SmoothingType = SmoothingType;

		RemeshOp.TargetEdgeLength = Params.TargetEdgeLength;
		RemeshOp.bPreserveSharpEdges = bPreserveSharpEdges;
		RemeshOp.bFlips = bAllowFlips;
		RemeshOp.bSplits = bAllowSplits;
		RemeshOp.bCollapses = bAllowCollapses;
		RemeshOp.bPreventNormalFlips = bPreventNormalFlips;
		RemeshOp.bPreventTinyTriangles = bPreventTinyTriangles;
		RemeshOp.MeshBoundaryConstraint = MeshBoundaryConstraint;
		RemeshOp.GroupBoundaryConstraint = PolyGroupBoundaryConstraint;
		RemeshOp.MaterialBoundaryConstraint = MaterialBoundaryConstraint;
		RemeshOp.bReproject = bReprojectToInputMesh;
		RemeshOp.ProjectionTarget = SourceMesh.Get();
		RemeshOp.ProjectionTargetSpatial = SourceSpatial.Get();
		RemeshOp.bReprojectConstraints = bReprojectConstraints;
		RemeshOp.BoundaryCornerAngleThreshold = SeamCornerThresholdAngleDegrees;
		RemeshOp.TargetMeshLocalToWorld = FTransformSRT3d::Identity();
		RemeshOp.ToolMeshLocalToWorld = FTransformSRT3d::Identity();
		RemeshOp.bUseWorldSpace = false;
		RemeshOp.bParallel = true;

		// RemeshOp makes a copy of the mesh to operate on, so we can't just pass the FDynamicMeshWeightAttribute pointer into CustomEdgeLengthScaleF
		int32 DensityMapLayerIndex = -1;
		bool bFoundDensityMapLayer = false;
		if (!Params.DensityMapName.IsEmpty() && SourceMesh->HasAttributes())
		{
			for (int32 WeightLayerIndex = 0; WeightLayerIndex < SourceMesh->Attributes()->NumWeightLayers(); ++WeightLayerIndex)
			{
				if (SourceMesh->Attributes()->GetWeightLayer(WeightLayerIndex)->GetName() == Params.DensityMapName)
				{
					DensityMapLayerIndex = WeightLayerIndex;
					bFoundDensityMapLayer = true;
					RemeshOp.bNeedsWeightMaps = true;
					break;
				}
			}
		}

		RemeshOp.CustomEdgeLengthScaleF = [bFoundDensityMapLayer, DensityMapLayerIndex, &Params](const FDynamicMesh3& Mesh, int VertexA, int VertexB) -> double
		{
			if (bFoundDensityMapLayer)
			{
				check(Mesh.HasAttributes());
				const FDynamicMeshWeightAttribute* const DensityMapLayer = Mesh.Attributes()->GetWeightLayer(DensityMapLayerIndex);
				check(DensityMapLayer);

				float WeightValueA;
				DensityMapLayer->GetValue(VertexA, &WeightValueA);
				WeightValueA = Params.DensityLowHigh[0] * (1 - WeightValueA) + Params.DensityLowHigh[1] * WeightValueA;

				float WeightValueB;
				DensityMapLayer->GetValue(VertexB, &WeightValueB);
				WeightValueB = Params.DensityLowHigh[0] * (1 - WeightValueB) + Params.DensityLowHigh[1] * WeightValueB;

				return EdgeScalingFunction(WeightValueA, WeightValueB);
			}
			else
			{
				return EdgeScalingFunction(Params.DensityLowHigh[0], Params.DensityLowHigh[0]);
			}
		};

		// Set up constraints for Cloth Seam edges
		UE::Geometry::FMeshConstraints Constraints;
		for (const TArray<FIntVector2>& Seam : Seams)
		{
			if (Seam.Num() == 1)
			{
				for (int32 Side = 0; Side < 2; ++Side)
				{
					constexpr bool bCannotDelete = true;
					constexpr bool bCanMove = false;
					FVertexConstraint VertexConstraint(bCannotDelete, bCanMove);
					Constraints.SetOrCombineVertexConstraint(Seam[0][Side], VertexConstraint);
				}
			}
			else
			{
				for (int32 StitchIndex = 0; StitchIndex < Seam.Num() - 1; ++StitchIndex)
				{
					for (int32 Side = 0; Side < 2; ++Side)
					{
						const int EdgeID = SourceMesh->FindEdge(Seam[StitchIndex][Side], Seam[StitchIndex + 1][Side]);
						check(EdgeID != FDynamicMesh3::InvalidID);
						FEdgeConstraint EdgeConstraint(EEdgeRefineFlags::FullyConstrained);
						Constraints.SetOrUpdateEdgeConstraint(EdgeID, EdgeConstraint);

						constexpr bool bCannotDelete = true;
						constexpr bool bCanMove = false;
						FVertexConstraint VertexConstraint(bCannotDelete, bCanMove);
						Constraints.SetOrCombineVertexConstraint(Seam[StitchIndex][Side], VertexConstraint);
						Constraints.SetOrCombineVertexConstraint(Seam[StitchIndex + 1][Side], VertexConstraint);
					}
				}
			}
		}
		RemeshOp.SetUserSpecifiedConstraints(Constraints);

		constexpr FProgressCancel* Progress = nullptr;		// Don't allow cancel or report progress for now
		RemeshOp.CalculateResult(Progress);

		if (RemeshOp.GetResultInfo().Result == EGeometryResultType::Success)
		{
			TUniquePtr<FDynamicMesh3> ResultMesh = RemeshOp.ExtractResult();
			Mesh = MoveTemp(*ResultMesh);
		}
		else
		{
			return false;
		}

		// compact the input mesh if enabled
		if (bAutoCompact)
		{
			Mesh.CompactInPlace(Params.CompactMaps);
		}

		return true;
	}


	static bool Simplify(UE::Geometry::FDynamicMesh3& Mesh, int TargetVertexCount, bool bCoarsenBoundaries, UE::Geometry::FCompactMaps* CompactMaps = nullptr)
	{
		using namespace UE::Geometry;

		//
		// These consts control overall remeshing behavior and are analogs of the properties exposed to the user in the actual Simplify tool
		// (i.e. things we might wish to add to the node properties in the future)
		//

		constexpr ESimplifyType SimplifierType = ESimplifyType::Attribute;
		constexpr bool bDiscardAttributes = false;
		constexpr bool bPreventNormalFlips = true;
		constexpr bool bPreserveSharpEdges = false;
		constexpr bool bPreventTinyTriangles = false;
		constexpr bool bReproject = true;
		constexpr bool bAutoCompact = true;
		constexpr bool bGeometricConstraint = false;

		// Angle threshold in degrees used for testing if two triangles should be considered coplanar, or two lines collinear */
		constexpr float MinimalAngleThreshold = 0.01f;

		// Note PolyEdgeAngleTolerance is very similar to MinimalAngleThreshold, but not redundant b/c the useful ranges are very different (MinimalAngleThreshold should generally be kept very small)
		// Threshold angle change (in degrees) along a polygroup edge, above which a vertex must be added
		constexpr float PolyEdgeAngleTolerance = 0.1f;


		FSimplifyMeshOp Op;

		Op.bDiscardAttributes = bDiscardAttributes;
		Op.bResultMustHaveAttributesEnabled = true;
		Op.bPreventNormalFlips = bPreventNormalFlips;
		Op.bPreserveSharpEdges = bPreserveSharpEdges;
		Op.bAllowSeamCollapse = !bPreserveSharpEdges;
		Op.bPreventTinyTriangles = bPreventTinyTriangles;
		Op.bReproject = bReproject;
		Op.SimplifierType = SimplifierType;
		Op.MinimalPlanarAngleThresh = MinimalAngleThreshold;

		Op.TargetMode = ESimplifyTargetType::VertexCount;
		Op.TargetCount = TargetVertexCount;

		Op.MeshBoundaryConstraint = bCoarsenBoundaries ? EEdgeRefineFlags::CollapseOnly : EEdgeRefineFlags::FullyConstrained;
		Op.GroupBoundaryConstraint = EEdgeRefineFlags::CollapseOnly;
		Op.MaterialBoundaryConstraint = EEdgeRefineFlags::CollapseOnly;

		Op.bGeometricDeviationConstraint = bGeometricConstraint;
		Op.GeometricTolerance = 0.0f;
		Op.PolyEdgeAngleTolerance = PolyEdgeAngleTolerance;

		TSharedPtr<FDynamicMesh3> SourceMesh = MakeShared<FDynamicMesh3>(MoveTemp(Mesh));
		TSharedPtr<FDynamicMeshAABBTree3> SourceSpatial;
		if (bReproject)
		{
			// acceleration structure is only used for reprojecting
			SourceSpatial = MakeShared<FDynamicMeshAABBTree3>(SourceMesh.Get(), true);
		}
		Op.OriginalMesh = SourceMesh;
		Op.OriginalMeshSpatial = SourceSpatial;

		IMeshReductionManagerModule& MeshReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
		Op.MeshReduction = MeshReductionModule.GetStaticMeshReductionInterface();

		constexpr FProgressCancel* Progress = nullptr;		// Don't allow cancel or report progress for now
		Op.CalculateResult(Progress);

		if (Op.GetResultInfo().Result == EGeometryResultType::Success)
		{
			TUniquePtr<FDynamicMesh3> ResultMesh = Op.ExtractResult();
			Mesh = MoveTemp(*ResultMesh);
		}
		else
		{
			return false;
		}

		// compact the input mesh if enabled
		if (bAutoCompact)
		{
			Mesh.CompactInPlace(CompactMaps);
		}

		return true;
	}

	static void EmptySimSelections(const TSharedRef<FManagedArrayCollection>& ClothCollection)
	{
		FCollectionClothSelectionFacade SelectionFacade(ClothCollection);

		const TArray<FName> SelectionNames = SelectionFacade.GetNames();
		for (const FName& SelectionName : SelectionNames)
		{
			const FName GroupName = SelectionFacade.GetSelectionGroup(SelectionName);
			if (GroupName == ClothCollectionGroup::SimVertices3D || GroupName == ClothCollectionGroup::SimVertices2D || GroupName == ClothCollectionGroup::SimFaces)
			{
				TSet<int32>& SelectionSet = SelectionFacade.GetSelectionSet(SelectionName);
				SelectionSet.Empty();
			}
		}
	}

	static void EmptySprings(const TSharedRef<FManagedArrayCollection>& ClothCollection)
	{
		::Chaos::Softs::FEmbeddedSpringFacade SpringFacade(ClothCollection.Get(), ClothCollectionGroup::SimVertices3D);
		SpringFacade.SetNumSpringConstraints(0);
	}

	static void RebuildTopologyDependentSimData(const TSharedRef<const FManagedArrayCollection>& InClothCollection,
		const TSharedRef<FManagedArrayCollection>& OutClothCollection)
	{
		using namespace UE::Chaos::ClothAsset;
		using namespace ::Chaos::Softs;

		FCollectionClothConstFacade InClothFacade(InClothCollection);
		FCollectionClothFacade OutClothFacade(OutClothCollection);
		FCollectionClothSelectionFacade OutSelectionFacade(OutClothCollection);
		OutSelectionFacade.DefineSchema();

		// Check that weight maps and skinning info have been interpolated over
		for (const FName& InWeightMapName : InClothFacade.GetWeightMapNames())
		{
			const FName* FoundName = Algo::Find(OutClothFacade.GetWeightMapNames(), InWeightMapName);
			checkf(FoundName, TEXT("Weight map %s was not copied to the output cloth collection"), *InWeightMapName.ToString());
		}
		if (InClothFacade.GetSimBoneIndices().Num() > 0 && OutClothFacade.GetNumSimVertices3D() > 0)
		{
			checkf(OutClothFacade.GetSimBoneIndices().Num() > 0, TEXT("Skinning bone indices not copied to the sim mesh of the output cloth collection"));
		}
		if (InClothFacade.GetSimBoneWeights().Num() > 0 && OutClothFacade.GetNumSimVertices3D() > 0)
		{
			checkf(OutClothFacade.GetSimBoneWeights().Num() > 0, TEXT("Skinning bone weights not copied to the sim mesh of the output cloth collection"));
		}

		FCollectionPropertyConstFacade InProperties(InClothCollection);

		// Reconstruct KinematicVertexSet
		const FName MaxDistanceName = GET_MEMBER_NAME_CHECKED(FChaosClothAssetSimulationMaxDistanceConfigNode, MaxDistance);
		const FName KinematicVertices3DName = GET_MEMBER_NAME_CHECKED(FChaosClothAssetSimulationMaxDistanceConfigNode, KinematicVertices3D);
		if (InProperties.GetKeyNameIndex(MaxDistanceName) != INDEX_NONE)
		{
			const FName MaxDistanceMapName(InProperties.GetStringValue(MaxDistanceName, MaxDistanceName.ToString()));
			OutSelectionFacade.FindOrAddSelectionSet(KinematicVertices3DName, ClothCollectionGroup::SimVertices3D) =
				FClothGeometryTools::GenerateKinematicVertices3D(OutClothCollection, MaxDistanceMapName, InProperties.GetWeightedFloatValue(MaxDistanceName, FVector2f(0.f, 1.f)), NAME_None);
		}

		// Reconstruct collision spheres
		const FName SelfCollisionSphereStiffnessName = GET_MEMBER_NAME_CHECKED(FChaosClothAssetSimulationSelfCollisionSpheresConfigNode, SelfCollisionSphereStiffness);
		const FName SelfCollisionSphereRadiusName = GET_MEMBER_NAME_CHECKED(FChaosClothAssetSimulationSelfCollisionSpheresConfigNode, SelfCollisionSphereRadius);
		const FName SelfCollisionSphereRadiusCullMultiplierName = GET_MEMBER_NAME_CHECKED(FChaosClothAssetSimulationSelfCollisionSpheresConfigNode, SelfCollisionSphereRadiusCullMultiplier);
		const FName SelfCollisionSphereSetNameName = GET_MEMBER_NAME_CHECKED(FChaosClothAssetSimulationSelfCollisionSpheresConfigNode, SelfCollisionSphereSetName);

		if (InProperties.GetKeyNameIndex(SelfCollisionSphereStiffnessName) != INDEX_NONE)
		{
			const float SelfCollisionSphereRadius = InProperties.GetValue<float>(SelfCollisionSphereRadiusName);
			const float SelfCollisionSphereRadiusCullMultiplier = InProperties.GetValue<float>(SelfCollisionSphereRadiusCullMultiplierName);
			const float CullDiameterSq = FMath::Square(SelfCollisionSphereRadius * SelfCollisionSphereRadiusCullMultiplier * 2.f);

			if (OutClothFacade.IsValid() && CullDiameterSq > 0.f)
			{
				TConstArrayView<FVector3f> SimPositions = OutClothFacade.GetSimPosition3D();
				TSet<int32> VertexSet;
				FClothGeometryTools::SampleVertices(SimPositions, CullDiameterSq, VertexSet);

				const FName SelectionSetName(*InProperties.GetStringValue(SelfCollisionSphereSetNameName, SelfCollisionSphereSetNameName.ToString()));
				OutSelectionFacade.FindOrAddSelectionSet(SelectionSetName, ClothCollectionGroup::SimVertices3D) = VertexSet;
			}
		}

		// Reconstruct long-range attachments

		// v1 (weight map)
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FName FixedEndWeightMapName = GET_MEMBER_NAME_CHECKED(FChaosClothAssetSimulationLongRangeAttachmentConfigNode, FixedEndWeightMap);
		static const FName UseGeodesicTethersName(TEXT("UseGeodesicTethers"), EFindName::FNAME_Find);
		check(UseGeodesicTethersName != NAME_None);
		if (InProperties.GetKeyNameIndex(FixedEndWeightMapName) != INDEX_NONE)
		{
			const bool bUseGeodesicTethers = InProperties.GetValue<bool>(UseGeodesicTethersName);
			const FName FixedEndWeightMap(InProperties.GetStringValue(FixedEndWeightMapName));

			UE::Chaos::ClothAsset::FClothEngineTools::GenerateTethers(OutClothCollection, FixedEndWeightMap, bUseGeodesicTethers);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// v2 (vertex set)
		const FName FixedEndSetName = GET_MEMBER_NAME_CHECKED(FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2, FixedEndSet);
		if (InProperties.GetKeyNameIndex(FixedEndSetName) != INDEX_NONE)
		{
			// Regenerate using the Kinematic vertices.
			const bool bUseGeodesicTethers = InProperties.GetValue<bool>(UseGeodesicTethersName);
			UE::Chaos::ClothAsset::FClothEngineTools::GenerateTethersFromSelectionSet(OutClothCollection, KinematicVertices3DName, bUseGeodesicTethers);
		}
	}


	static void EmptyRenderSelections(const TSharedRef<FManagedArrayCollection>& ClothCollection)
	{
		using namespace UE::Chaos::ClothAsset;

		FCollectionClothSelectionFacade SelectionFacade(ClothCollection);

		const TArray<FName> SelectionNames = SelectionFacade.GetNames();
		for (const FName& SelectionName : SelectionNames)
		{
			const FName GroupName = SelectionFacade.GetSelectionGroup(SelectionName);
			if (GroupName == ClothCollectionGroup::RenderVertices || GroupName == ClothCollectionGroup::RenderFaces)
			{
				TSet<int32>& SelectionSet = SelectionFacade.GetSelectionSet(SelectionName);
				SelectionSet.Empty();
			}
		}
	}

	// Grab the seam information from the input collection
	// Seams are comprised of a set of Stitches. Each Stitch is simply a pair of vertex indices indicating vertices that should be welded to form the 3D mesh.
	// Stitches are given in random order within the Seam. To make remeshing them easier, we will find connected strips of stitches and store them in sequential order.
	// So the vertices in Stitch N are connected to the vertices in Stitch N+1.
	static void GetSimMeshSeams(const TSharedRef<const FManagedArrayCollection>& ClothCollection, const FCollectionClothConstFacade& ClothFacade, const UE::Geometry::FDynamicMesh3& Mesh2D, TArray<TArray<FIntVector2>>& OutSeams)
	{
		for (int32 SeamIndex = 0; SeamIndex < ClothFacade.GetNumSeams(); ++SeamIndex)
		{
			FCollectionClothSeamConstFacade SeamFacade = ClothFacade.GetSeam(SeamIndex);
			FClothGeometryTools::BuildConnectedSeams2D(ClothCollection, SeamIndex, Mesh2D, OutSeams);

			// Check seams are valid
			for (int32 SeamID = 0; SeamID < OutSeams.Num(); ++SeamID)
			{
				const TArray<FIntVector2>& SubSeam = OutSeams[SeamID];
				for (int32 StitchID = 0; StitchID < SubSeam.Num() - 1; ++StitchID)
				{
					const int32 NextStitchID = StitchID + 1;
					for (int32 Side = 0; Side < 2; ++Side)
					{
						const int32 StitchVert = SubSeam[StitchID][Side];
						const int32 NextStitchVert = SubSeam[NextStitchID][Side];

						if (StitchVert == NextStitchVert)
						{
							continue;
						}

						const int32 FoundEdge = Mesh2D.FindEdge(StitchVert, NextStitchVert);

						// This would indicate a problem in BuildConnectedSeams
						checkf(FoundEdge != UE::Geometry::FDynamicMesh3::InvalidID, TEXT("Could not find a mesh edge between sequential seam vertices %d, %d"), StitchVert, NextStitchVert);
					}
				}
			}
		}


		// For remeshing purposes, we also want to find and constrain any vertices that connect the two seam sides together
		// (This is only relevant for "internal" seams, which connect vertices within the same pattern)
		for (TArray<FIntVector2>& Seam : OutSeams)
		{
			if (Seam.Num() == 0)
			{
				continue;
			}

			for (const FIntVector2& EndStitch : { Seam[0], Seam.Last() })
			{
				for (const int NeighborA : Mesh2D.VtxVerticesItr(EndStitch[0]))
				{
					bool bMatchFound = false;

					for (const int NeighborB : Mesh2D.VtxVerticesItr(EndStitch[1]))
					{
						if (NeighborA == NeighborB)
						{
							if (Seam.Contains(FIntVector2(NeighborA, NeighborB)))
							{
								continue;
							}

							if (EndStitch == Seam[0])
							{
								Seam.Insert(FIntVector2(NeighborA, NeighborB), 0);
							}
							else
							{
								Seam.Add(FIntVector2(NeighborA, NeighborB));
							}

							bMatchFound = true;
							break;
						}
					}

					if (bMatchFound)
					{
						break;
					}
				}
			}
		}
	}

	struct FRemeshSimMeshParams
	{
		int32 Iterations = 10;
		double Smoothing = 0.25;
		const TSet<int32>* FaceSelection = nullptr;
		FString DensityMapName;
		FVector2f DensityLowHigh = { 0,1 };
	};

	struct FVertexProjectionData
	{
		UE::Geometry::FIndex3i GlobalTriangle;  // Global sim vertex 3D indices from ProjectionTarget3D
		FVector3d BaryCoords;
	};

	static void ReprojectAccessoryMeshes(
		const TSharedRef<const FManagedArrayCollection>& InClothCollection,
		const TSharedRef<FManagedArrayCollection>& OutClothCollection,
		const TMap<int32, FVertexProjectionData>& VertexProjectionDataMap,
		const TMap<int, int32>& DynamicMeshToClothVertexMap)
	{
		using namespace UE::Chaos::ClothAsset;

		FCollectionClothConstFacade InClothFacade(InClothCollection);
		FCollectionClothFacade OutClothFacade(OutClothCollection);

		const int32 NumAccessoryMeshes = InClothFacade.GetNumSimAccessoryMeshes();
		if (NumAccessoryMeshes == 0)
		{
			return;
		}

		// Clear stale data that was CopyTo'd with the old vertex count
		OutClothFacade.SetNumSimAccessoryMeshes(0);

		const int32 NumOutputVertices = OutClothFacade.GetNumSimVertices3D();

		for (int32 MeshIdx = 0; MeshIdx < NumAccessoryMeshes; ++MeshIdx)
		{
			FCollectionClothSimAccessoryMeshConstFacade InAccessory = InClothFacade.GetSimAccessoryMesh(MeshIdx);

			const TConstArrayView<FVector3f> InPositions = InAccessory.GetSimAccessoryMeshPosition3D();
			const TConstArrayView<FVector3f> InNormals = InAccessory.GetSimAccessoryMeshNormal();
			const TConstArrayView<TArray<int32>> InBoneIndices = InAccessory.GetSimAccessoryMeshBoneIndices();
			const TConstArrayView<TArray<float>> InBoneWeights = InAccessory.GetSimAccessoryMeshBoneWeights();

			const bool bHasSkinning = InBoneIndices.Num() > 0 && InBoneWeights.Num() > 0 &&
				InBoneIndices.Num() == InPositions.Num() && InBoneWeights.Num() == InPositions.Num();

			TArray<FVector3f> OutPositions;
			TArray<FVector3f> OutNormals;
			TArray<TArray<int32>> OutBoneIndices;
			TArray<TArray<float>> OutBoneWeights;

			OutPositions.SetNumZeroed(NumOutputVertices);
			OutNormals.SetNumZeroed(NumOutputVertices);
			OutBoneIndices.SetNum(NumOutputVertices);
			OutBoneWeights.SetNum(NumOutputVertices);

			for (const TPair<int, int32>& Pair : DynamicMeshToClothVertexMap)
			{
				const int32 VID = Pair.Key;
				const int32 OutVertIdx = Pair.Value;

				const FVertexProjectionData* ProjData = VertexProjectionDataMap.Find(VID);
				if (!ProjData)
				{
					continue;
				}

				const int32 G0 = ProjData->GlobalTriangle[0];
				const int32 G1 = ProjData->GlobalTriangle[1];
				const int32 G2 = ProjData->GlobalTriangle[2];

				if (!InPositions.IsValidIndex(G0) || !InPositions.IsValidIndex(G1) || !InPositions.IsValidIndex(G2) || !OutPositions.IsValidIndex(OutVertIdx))
				{
					continue;
				}

				const FVector3d& B = ProjData->BaryCoords;

				// Interpolate Position3D
				OutPositions[OutVertIdx] = FVector3f(
					(float)B[0] * InPositions[G0] +
					(float)B[1] * InPositions[G1] +
					(float)B[2] * InPositions[G2]);

				// Interpolate Normal
				FVector3f InterpNormal =
					(float)B[0] * InNormals[G0] +
					(float)B[1] * InNormals[G1] +
					(float)B[2] * InNormals[G2];
				InterpNormal.Normalize();
				OutNormals[OutVertIdx] = InterpNormal;

				// Interpolate skinning data
				if (bHasSkinning)
				{
					auto MakeBoneWeights = [](const TArray<int32>& Indices, const TArray<float>& Weights)
					{
						check(Indices.Num() == Weights.Num());

						TArray<FBoneIndexType> BoneIndexTypes;
						BoneIndexTypes.Reserve(Indices.Num());
						for (const int32 Idx : Indices)
						{
							BoneIndexTypes.Add(static_cast<FBoneIndexType>(Idx));
						}
						return UE::AnimationCore::FBoneWeights::Create(BoneIndexTypes.GetData(), Weights.GetData(), Indices.Num());
					};

					const UE::AnimationCore::FBoneWeights BlendedWeights = UE::AnimationCore::FBoneWeights::Blend(
						MakeBoneWeights(InBoneIndices[G0], InBoneWeights[G0]),
						MakeBoneWeights(InBoneIndices[G1], InBoneWeights[G1]),
						MakeBoneWeights(InBoneIndices[G2], InBoneWeights[G2]),
						(float)B[0], (float)B[1], (float)B[2]);

					TArray<int32>& OutIndices = OutBoneIndices[OutVertIdx];
					TArray<float>& OutWeights = OutBoneWeights[OutVertIdx];
					OutIndices.Reserve(BlendedWeights.Num());
					OutWeights.Reserve(BlendedWeights.Num());
					for (int32 i = 0; i < BlendedWeights.Num(); ++i)
					{
						OutIndices.Add(static_cast<int32>(BlendedWeights[i].GetBoneIndex()));
						OutWeights.Add(BlendedWeights[i].GetWeight());
					}
				}
			}

			FCollectionClothSimAccessoryMeshFacade OutAccessory = OutClothFacade.AddGetSimAccessoryMesh();
			OutAccessory.Initialize(InAccessory.GetSimAccessoryMeshName(), OutPositions, OutNormals, OutBoneIndices, OutBoneWeights);
		}
	}

	static void RemeshSimMesh(
		const TSharedRef<const FManagedArrayCollection>& ClothCollection,
		const FRemeshSimMeshParams& Params,
		const TSharedRef<FManagedArrayCollection>& OutClothCollection,
		TOptional<FText>& OutWarningMsg)
	{
		using namespace UE::Geometry;
		using namespace UE::Chaos::ClothAsset;

		FCollectionClothConstFacade InClothFacade(ClothCollection);

		if (InClothFacade.GetNumSimPatterns() == 0)
		{
			FClothGeometryTools::DeleteSimMesh(OutClothCollection);
			return;
		}

		// Convert input patterns to a DynamicMesh

		FClothPatternToDynamicMesh Converter;

		FDynamicMesh3 Mesh2D;
		Converter.Convert(ClothCollection, INDEX_NONE, EClothPatternVertexType::Sim2D, Mesh2D);

		const double TotalArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(Mesh2D).Y;
		checkf(TotalArea > 0, TEXT("FChaosClothAssetRemeshNode::RemeshSimMesh: Expected 2D Sim mesh to have a positive area"));
		const int TriangleCount = Mesh2D.TriangleCount();

		// Copy pattern IDs into polygroup layer

		Mesh2D.EnableAttributes();
		const int32 PatternIndexLayerID = Mesh2D.Attributes()->NumPolygroupLayers();
		Mesh2D.Attributes()->SetNumPolygroupLayers(PatternIndexLayerID + 1);
		FDynamicMeshPolygroupAttribute* const PatternIndexLayer = Mesh2D.Attributes()->GetPolygroupLayer((int)PatternIndexLayerID);
		check(Mesh2D.TriangleCount() == InClothFacade.GetNumSimFaces());
		for (int32 FaceIndex = 0; FaceIndex < InClothFacade.GetNumSimFaces(); ++FaceIndex)
		{
			const int32 PatternID = InClothFacade.FindSimPatternByFaceIndex(FaceIndex);
			PatternIndexLayer->SetValue(FaceIndex, PatternID);
		}

		// Look for the DensityMap weight layer
		const FDynamicMeshWeightAttribute* DensityMapLayer = nullptr;
		if (Mesh2D.HasAttributes())
		{
			for (int32 WeightLayerIndex = 0; WeightLayerIndex < Mesh2D.Attributes()->NumWeightLayers(); ++WeightLayerIndex)
			{
				if (Mesh2D.Attributes()->GetWeightLayer(WeightLayerIndex)->GetName() == Params.DensityMapName)
				{
					DensityMapLayer = Mesh2D.Attributes()->GetWeightLayer(WeightLayerIndex);
					break;
				}
			}
		}


		TArray<TArray<FIntVector2>> Seams;
		Private::GetSimMeshSeams(ClothCollection, InClothFacade, Mesh2D, Seams);


		// Remesh seams

		const double TargetEdgeLength = TMeshQueries<FDynamicMesh3>::AverageEdgeLength(Mesh2D);

		for (int32 ResampleIter = 0; ResampleIter < Params.Iterations; ++ResampleIter)
		{
			UE::Chaos::ClothAsset::Private::RemeshSeams(Mesh2D, Seams, TargetEdgeLength, DensityMapLayer, Params.DensityLowHigh, Params.FaceSelection);
		}

		// Remesh boundaries

		for (int32 ResampleIter = 0; ResampleIter < Params.Iterations; ++ResampleIter)
		{
			UE::Chaos::ClothAsset::Private::RemeshBoundaries(Mesh2D, Seams, TargetEdgeLength, DensityMapLayer, Params.DensityLowHigh, Params.FaceSelection);
		}

		// Do the remeshing of the rest of the mesh

		UE::Geometry::FCompactMaps CompactMaps;
		constexpr bool bUniformSmoothing = true;

		FRemeshParams RemeshParams;
		RemeshParams.TargetEdgeLength = TargetEdgeLength;
		RemeshParams.Iterations = Params.Iterations;
		RemeshParams.SmoothingRate = Params.Smoothing;
		RemeshParams.bUniformSmoothing = bUniformSmoothing;
		RemeshParams.DensityMapName = Params.DensityMapName;
		RemeshParams.DensityLowHigh = Params.DensityLowHigh;
		RemeshParams.CompactMaps = &CompactMaps;
		RemeshParams.FaceSelection = Params.FaceSelection;

		UE::Chaos::ClothAsset::Private::Remesh(Mesh2D, RemeshParams, Seams);

		// Update stitches
		for (TArray<FIntVector2>& Seam : Seams)
		{
			for (FIntVector2& Stitch : Seam)
			{
				Stitch[0] = CompactMaps.GetVertexMapping(Stitch[0]);
				Stitch[1] = CompactMaps.GetVertexMapping(Stitch[1]);
				checkf(Stitch[0] != FDynamicMesh3::InvalidID, TEXT("Stitch vertex %d was deleted by remeshing"), Stitch[0]);
				checkf(Stitch[1] != FDynamicMesh3::InvalidID, TEXT("Stitch vertex %d was deleted by remeshing"), Stitch[1]);
			}
		}


		// Project the 3D vertices onto the input 3D mesh

		// For each 2D vertex, we will find the closest triangle on the input 2D mesh, then look up that triangle on the input 3D mesh to get the final 3D location.
		// We will do this pattern-by-pattern to handle issues where the patterns overlap in 2D space.

		const FDynamicMeshPolygroupAttribute* const NewPatternIndexLayer = Mesh2D.Attributes()->GetPolygroupLayer((int)PatternIndexLayerID);
		check(NewPatternIndexLayer);

		FDynamicMesh3 Mesh3D;
		Mesh3D.Copy(Mesh2D);

		TMap<int32, FVertexProjectionData> VertexProjectionDataMap;

		TMap<int32, TSet<int32>> PatternVertexIDs;
		for (int32 TID : Mesh2D.TriangleIndicesItr())
		{
			const int32 PatternID = NewPatternIndexLayer->GetValue(TID);
			if (!PatternVertexIDs.Contains(PatternID))
			{
				PatternVertexIDs.Add(PatternID, TSet<int32>());
			}

			const FIndex3i Tri = Mesh2D.GetTriangle(TID);
			PatternVertexIDs[PatternID].Add(Tri[0]);
			PatternVertexIDs[PatternID].Add(Tri[1]);
			PatternVertexIDs[PatternID].Add(Tri[2]);
		}

		for (int32 PatternID = 0; PatternID < InClothFacade.GetNumSimPatterns(); ++PatternID)
		{
			if (!PatternVertexIDs.Contains(PatternID))
			{
				continue;
			}

			FDynamicMesh3 ProjectionTarget2D;
			Converter.Convert(ClothCollection, PatternID, EClothPatternVertexType::Sim2D, ProjectionTarget2D);
			FDynamicMesh3 ProjectionTarget3D;
			Converter.Convert(ClothCollection, PatternID, EClothPatternVertexType::Sim3D, ProjectionTarget3D);
			const FClothPatternToDynamicMeshMappingSupport MappingSupport3D(ProjectionTarget3D);

			if (ProjectionTarget2D.TriangleCount() < ProjectionTarget3D.TriangleCount())
			{
				OutWarningMsg = LOCTEXT("Sim2DDegenerateTriangle", "2D Sim mesh has degenerate triangles. Remeshing may produce unexpected results");
			}
			else if (ProjectionTarget3D.TriangleCount() < ProjectionTarget2D.TriangleCount())
			{
				OutWarningMsg = LOCTEXT("Sim3DDegenerateTriangle", "3D Sim mesh has degenerate triangles. Remeshing may produce unexpected results");
			}

			TArray<int32> ValidTriangles;
			for (int32 TriangleID = 0; TriangleID < FMath::Min(ProjectionTarget2D.MaxTriangleID(), ProjectionTarget3D.MaxTriangleID()); ++TriangleID)
			{
				if (ProjectionTarget2D.IsTriangle(TriangleID) && ProjectionTarget3D.IsTriangle(TriangleID))
				{
					ValidTriangles.Add(TriangleID);
				}
			}
			TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3> ProjectionTargetSpatial = MakeShared<UE::Geometry::FDynamicMeshAABBTree3>(&ProjectionTarget2D, /*bAutoBuild=*/false);
			ProjectionTargetSpatial->Build(ValidTriangles);

			const MeshIntersection::FIntersectionsQueryResult Result = ProjectionTargetSpatial->FindAllSelfIntersections(true, UE::Geometry::IMeshSpatial::FQueryOptions(), [](FIntrTriangle3Triangle3d& IntersectionInfo) -> bool
			{
				const FTriangle3d T03D = IntersectionInfo.GetTriangle0();
				const FTriangle2d T02D{ FVector2d(T03D.V[0]), FVector2d(T03D.V[1]), FVector2d(T03D.V[2]) };
				const FTriangle3d T13D = IntersectionInfo.GetTriangle1();
				const FTriangle2d T12D{ FVector2d(T13D.V[0]), FVector2d(T13D.V[1]), FVector2d(T13D.V[2]) };

				FIntrTriangle2Triangle2d Intersection2D(T02D, T12D);
				if (Intersection2D.Find())
				{
					IntersectionInfo.SetReportCoplanarIntersection(true);
					return IntersectionInfo.Find();
				}
				return false;
			});

			if ((Result.Points.Num() > 0) || (Result.Polygons.Num() > 0) || (Result.Segments.Num() > 0))
			{
				OutWarningMsg = LOCTEXT("SelfIntersecting2DPattern", "2D Sim mesh has one or more self-intersecting patterns. Remeshing may produce unexpected results");
			}
			

			for (const int32 VID : PatternVertexIDs[PatternID])
			{
				const FVector3d SrcVert = Mesh2D.GetVertex(VID);

				double Distance;
				const int NearestTriangle = ProjectionTargetSpatial->FindNearestTriangle(SrcVert, Distance);

				if (!ProjectionTarget2D.IsTriangle(NearestTriangle))
				{
					UE_LOGF(LogChaosClothAssetDataflowNodes, Verbose, "UE::Chaos::ClothAsset::RemeshNode: Failed to find a nearest triangle on the input 2D Sim mesh. Ensure the 2D sim mesh has no empty or degenerate triangles before remeshing");
					continue;
				}

				const FDistPoint3Triangle3d Dist = TMeshQueries<FDynamicMesh3>::TriangleDistance(ProjectionTarget2D, NearestTriangle, SrcVert);
				const FVector3d Bary = Dist.TriangleBaryCoords;

				if (!ProjectionTarget3D.IsTriangle(NearestTriangle))
				{
					UE_LOGF(LogChaosClothAssetDataflowNodes, Verbose, "UE::Chaos::ClothAsset::RemeshNode: A triangle ID that is valid on the 2D Sim mesh was not found on the 3D sim mesh. Check Simulation faces");
					continue;
				}

				const FIndex3i Triangle3D = ProjectionTarget3D.GetTriangle(NearestTriangle);
				if (!ProjectionTarget3D.IsVertex(Triangle3D[0]) || !ProjectionTarget3D.IsVertex(Triangle3D[1]) || !ProjectionTarget3D.IsVertex(Triangle3D[2]))
				{
					UE_LOGF(LogChaosClothAssetDataflowNodes, Verbose, "UE::Chaos::ClothAsset::RemeshNode: A triangle ID is valid on the 3D Sim mesh has vertex indices that are not valid. Check Simulation faces");
					continue;
				}

				const FVector3d InterpolatedPoint = ProjectionTarget3D.GetTriBaryPoint(NearestTriangle, Bary[0], Bary[1], Bary[2]);

				Mesh3D.SetVertex(VID, InterpolatedPoint);

				const FIndex3i GlobalTriangle(
					MappingSupport3D.GetOriginalVertexID(Triangle3D[0]),
					MappingSupport3D.GetOriginalVertexID(Triangle3D[1]),
					MappingSupport3D.GetOriginalVertexID(Triangle3D[2]));
				VertexProjectionDataMap.Add(VID, { GlobalTriangle, Bary });
			}
		}

		// Build the output cloth sim mesh

		constexpr bool bAppendToExistingMesh = false;
		constexpr bool bTransferWeightMaps = true;
		constexpr bool bTransferSimSkinningData = true;
		TMap<int, int32> DynamicMeshToClothVertexMap;
		FClothGeometryTools::BuildSimMeshFromDynamicMeshes(OutClothCollection, Mesh2D, Mesh3D, PatternIndexLayerID, bTransferWeightMaps, bTransferSimSkinningData, bAppendToExistingMesh, DynamicMeshToClothVertexMap);

		// Reproject accessory mesh data before the seam rebuild, while DynamicMeshToClothVertexMap is still valid.
		// Seam initialization modifies SimVertices3D (splits boundary vertices), which invalidates the map.
		Private::ReprojectAccessoryMeshes(ClothCollection, OutClothCollection, VertexProjectionDataMap, DynamicMeshToClothVertexMap);

		// Re-apply the seam info from the input sim mesh. This will create a new Seam for each set of connected stitches

		FCollectionClothFacade OutClothFacade(OutClothCollection);
		for (int32 SeamIndex = 0; SeamIndex < Seams.Num(); ++SeamIndex)
		{
			const TArray<FIntVector2>& Seam = Seams[SeamIndex];

			TArray<FIntVector2> NewSeam;
			for (const FIntVector2& Stitch : Seam)
			{
				if (Stitch[0] == Stitch[1])
				{
					continue;
				}

				FIntVector2 NewStitch;
				NewStitch[0] = DynamicMeshToClothVertexMap[Stitch[0]];
				NewStitch[1] = DynamicMeshToClothVertexMap[Stitch[1]];
				NewSeam.Add(NewStitch);
			}

			FCollectionClothSeamFacade NewSeamFacade = OutClothFacade.AddGetSeam();
			NewSeamFacade.Initialize(NewSeam);
		}
	}

	struct FRemeshRenderMeshParams
	{
		EChaosClothAssetRemeshMethod RemeshMethod = EChaosClothAssetRemeshMethod::Remesh;
		bool bRemeshSeams = false;
		int32 SeamRemeshIterations = 1;
		int32 Iterations = 10;
		double Smoothing = 0.25;
		const TSet<int32>* FaceSelection = nullptr;
		FString DensityMapName;
		FVector2f DensityLowHigh;
		int32 TargetPercent = 100;
	};

	static void RemeshRenderMesh(const TSharedRef<const FManagedArrayCollection>& ClothCollection,
		const FRemeshRenderMeshParams& Params,
		const TSharedRef<FManagedArrayCollection>& OutClothCollection)
	{
		using namespace UE::Geometry;
		using namespace UE::Chaos::ClothAsset;

		// Get the source mesh
		FClothPatternToDynamicMesh Converter;
		FDynamicMesh3 DynamicMesh;

		// NOTE: When applied to the Render mesh, this Convert function will assign PatternIDs to the MaterialID attribute of the DynamicMesh. 
		// After remeshing we will use the MatrialID attribute to determine which triangles should go into which output pattern.
		Converter.Convert(ClothCollection, INDEX_NONE, EClothPatternVertexType::Render, DynamicMesh);

		check(DynamicMesh.HasAttributes());

		const int InputMeshVertexCount = DynamicMesh.VertexCount();
		const int InputMeshTriangleCount = DynamicMesh.TriangleCount();

		const bool bHasUVs = (DynamicMesh.Attributes()->PrimaryUV() != nullptr);

		const double TargetEdgeLength = TMeshQueries<FDynamicMesh3>::AverageEdgeLength(DynamicMesh);

		TArray<TArray<FIntVector2>> Seams;

		if (Params.bRemeshSeams)
		{
			// Create pseudo-stitches based on boundary vertex proximity. These stitches aren't going to actually weld vertices together, but they will guide boundary remeshing.
			// The goal is to maintain a vertex pairing along boundaries in order to avoid holes opening up when the mesh deforms due to skinning.
			TArray<FIntVector2> Stitches;
			UE::Chaos::ClothAsset::Private::FindCoincidentBoundaryVertices(DynamicMesh, Stitches);

			FClothGeometryTools::BuildConnectedSeams(Stitches, DynamicMesh, Seams);

			// Add density map for for Render mesh
			const UE::Geometry::FDynamicMeshWeightAttribute* DensityMapLayer = nullptr;
			if (DynamicMesh.HasAttributes())
			{
				for (int32 WeightLayerIndex = 0; WeightLayerIndex < DynamicMesh.Attributes()->NumWeightLayers(); ++WeightLayerIndex)
				{
					if (DynamicMesh.Attributes()->GetWeightLayer(WeightLayerIndex)->GetName() == Params.DensityMapName)
					{
						DensityMapLayer = DynamicMesh.Attributes()->GetWeightLayer(WeightLayerIndex);
						break;
					}
				}
			}

			for (int RemeshPass = 0; RemeshPass < Params.SeamRemeshIterations; ++RemeshPass)
			{
				UE::Chaos::ClothAsset::Private::RemeshSeams(DynamicMesh, Seams, TargetEdgeLength, DensityMapLayer, Params.DensityLowHigh, Params.FaceSelection);
			}

			// Also remesh the open boundaries that are not constrained by seams
			for (int RemeshPass = 0; RemeshPass < Params.SeamRemeshIterations; ++RemeshPass)
			{
				UE::Chaos::ClothAsset::Private::RemeshBoundaries(DynamicMesh, Seams, TargetEdgeLength, DensityMapLayer, Params.DensityLowHigh, Params.FaceSelection);
			}
		}


		UE::Geometry::FCompactMaps CompactMaps;
		if (Params.RemeshMethod == EChaosClothAssetRemeshMethod::Remesh)
		{
			constexpr bool bUniformSmo1othing = false;	// uniform smoothing can distort the UV layer pretty badly

			FRemeshParams RemeshParams;
			RemeshParams.TargetEdgeLength = TargetEdgeLength;
			RemeshParams.Iterations = Params.Iterations;
			RemeshParams.SmoothingRate = Params.Smoothing;
			RemeshParams.bUniformSmoothing = bUniformSmo1othing;
			RemeshParams.DensityMapName = Params.DensityMapName;
			RemeshParams.DensityLowHigh = Params.DensityLowHigh;
			RemeshParams.CompactMaps = &CompactMaps;
			RemeshParams.FaceSelection = Params.FaceSelection;

			const bool bSuccess = UE::Chaos::ClothAsset::Private::Remesh(DynamicMesh, RemeshParams, Seams);
			check(bSuccess);
		}
		else
		{
			const bool bCoarsenBoundariesDuringSimplify = !Params.bRemeshSeams;
			const int TargetVertexCount = FMath::RoundToInt(static_cast<float>(Params.TargetPercent) / 100.0f * static_cast<float>(InputMeshVertexCount));
			UE::Chaos::ClothAsset::Private::Simplify(DynamicMesh, TargetVertexCount, bCoarsenBoundariesDuringSimplify, &CompactMaps);
		}

		// Collect outputs

		//
		// Normals
		//

		const bool bHasNormals = (DynamicMesh.Attributes()->PrimaryNormals() != nullptr);
		check(bHasNormals);

		TArray<FVector3f> Normals;
		Normals.SetNum(DynamicMesh.VertexCount());
		const FDynamicMeshNormalOverlay* const NormalOverlay = DynamicMesh.Attributes()->PrimaryNormals();
		for (const int TriangleIndex : DynamicMesh.TriangleIndicesItr())
		{
			const FIndex3i Tri = DynamicMesh.GetTriangle(TriangleIndex);

			for (int TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
			{
				const int VertexIndex = Tri[TriangleVertexIndex];

				// NOTE: This assumes one normal per vertex in the overlay (i.e. no "hard edges")
				Normals[VertexIndex] = NormalOverlay->GetElementAtVertex(TriangleIndex, VertexIndex);
			}
		}

		//
		// Tangents
		//

		TArray<FVector3f> TangentUs;
		TArray<FVector3f> TangentVs;
		{
			const bool bHasTangentUs = (DynamicMesh.Attributes()->PrimaryTangents() != nullptr);
			const bool bHasTangentVs = (DynamicMesh.Attributes()->PrimaryBiTangents() != nullptr);
			if (!bHasTangentUs || !bHasTangentVs)
			{
				FMeshTangentsf::ComputeDefaultOverlayTangents(DynamicMesh);
			}
			TangentUs.SetNumZeroed(DynamicMesh.VertexCount());
			TangentVs.SetNumZeroed(DynamicMesh.VertexCount());

			const FDynamicMeshNormalOverlay* const TangentUOverlay = DynamicMesh.Attributes()->PrimaryTangents();
			const FDynamicMeshNormalOverlay* const TangentVOverlay = DynamicMesh.Attributes()->PrimaryBiTangents();

			for (const int TriangleIndex : DynamicMesh.TriangleIndicesItr())
			{
				const FIndex3i Tri = DynamicMesh.GetTriangle(TriangleIndex);

				for (int TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
				{
					const int VertexIndex = Tri[TriangleVertexIndex];

					TangentUs[VertexIndex] += TangentUOverlay->GetElementAtVertex(TriangleIndex, VertexIndex);
					TangentVs[VertexIndex] += TangentVOverlay->GetElementAtVertex(TriangleIndex, VertexIndex);
				}
			}

			for (int32 VertexIndex = 0; VertexIndex < DynamicMesh.VertexCount(); ++VertexIndex)
			{
				TangentUs[VertexIndex].Normalize();
				TangentVs[VertexIndex].Normalize();
			}
		}

		//
		// UVs
		//

		TArray<TArray<FVector2f, TInlineAllocator<MAX_TEXCOORDS>>> UVs;
		if (bHasUVs)
		{
			UVs.SetNum(DynamicMesh.VertexCount());

			for (int32 UVChannel = 0; UVChannel < DynamicMesh.Attributes()->NumUVLayers(); ++UVChannel)
			{
				const FDynamicMeshUVOverlay* const UVOverlay = DynamicMesh.Attributes()->GetUVLayer(UVChannel);

				// Assume no seams in the dynamic mesh UV overlay
				checkSlow(!UVOverlay->HasInteriorSeamEdges());

				for (const int TriangleIndex : DynamicMesh.TriangleIndicesItr())
				{
					const FIndex3i Tri = DynamicMesh.GetTriangle(TriangleIndex);
					const FIndex3i OverlayTri = UVOverlay->GetTriangle(TriangleIndex);

					for (int TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
					{
						if (OverlayTri[TriangleVertexIndex] != FDynamicMesh3::InvalidID)
						{
							const int VertexIndex = Tri[TriangleVertexIndex];
							UVs[VertexIndex].SetNumZeroed(UVChannel + 1);
							UVs[VertexIndex][UVChannel] = UVOverlay->GetElement(OverlayTri[TriangleVertexIndex]);
						}
					}
				}
			}
		}

		//
		// Skin Weights
		//

		FDynamicMeshAttributeSet* Attributes = DynamicMesh.Attributes();
		check(Attributes);

		TArray<TArray<int32>> BoneIndices;
		TArray<TArray<float>> BoneWeights;
		BoneIndices.SetNum(DynamicMesh.VertexCount());
		BoneWeights.SetNum(DynamicMesh.VertexCount());

		for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& SkinWeightLayer : Attributes->GetSkinWeightsAttributes())
		{
			const FDynamicMeshVertexSkinWeightsAttribute* const SkinWeightAttribute = SkinWeightLayer.Value.Get();

			for (const int TriangleIndex : DynamicMesh.TriangleIndicesItr())
			{
				const FIndex3i Tri = DynamicMesh.GetTriangle(TriangleIndex);

				for (int TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
				{
					const int VertexIndex = Tri[TriangleVertexIndex];
					SkinWeightAttribute->GetValue(VertexIndex, BoneIndices[VertexIndex], BoneWeights[VertexIndex]);
				}
			}
		}

		//
		// Vertex Weights
		//

		TArray<TArray<float>> VertexWeights;
		VertexWeights.SetNum(Attributes->NumWeightLayers());

		TArray<FName> VertexWeightLayerNames;
		VertexWeightLayerNames.SetNum(Attributes->NumWeightLayers());

		for (int32 WeightLayerIndex = 0; WeightLayerIndex < Attributes->NumWeightLayers(); ++WeightLayerIndex)
		{
			const FDynamicMeshWeightAttribute* const WeightLayerAttribute = Attributes->GetWeightLayer(WeightLayerIndex);
			VertexWeightLayerNames[WeightLayerIndex] = WeightLayerAttribute->GetName();

			VertexWeights[WeightLayerIndex].SetNum(DynamicMesh.MaxVertexID());
			for (const int VertexIndex : DynamicMesh.VertexIndicesItr())
			{
				WeightLayerAttribute->GetValue(VertexIndex, &(VertexWeights[WeightLayerIndex][VertexIndex]));
			}
		}


		// Find the set of triangles per MaterialID

		TMap<int32, TArray<int32>> MaterialTriangles;

		const FDynamicMeshMaterialAttribute* const MaterialAttribute = Attributes->GetMaterialID();
		check(MaterialAttribute);

		for (const int32 TriangleID : DynamicMesh.TriangleIndicesItr())
		{
			const int32 MaterialID = MaterialAttribute->GetValue(TriangleID);
			if (!MaterialTriangles.Contains(MaterialID))
			{
				MaterialTriangles.Add(MaterialID);
			}
			MaterialTriangles[MaterialID].Add(TriangleID);
		}

		TArray<int32> MaterialIDs;
		MaterialTriangles.GetKeys(MaterialIDs);

		const int32 NumMaterials = MaterialIDs.Num();

		//
		// Populate output cloth collection
		// 

		FClothGeometryTools::DeleteRenderMesh(OutClothCollection);
		FCollectionClothFacade OutClothFacade(OutClothCollection);
		OutClothFacade.SetNumRenderPatterns(NumMaterials);

		TMap<int, FIntVector2> MeshVertexToPatternAndVertex;

		for (int32 DestPatternID = 0; DestPatternID < MaterialIDs.Num(); ++DestPatternID)
		{
			FCollectionClothRenderPatternFacade OutClothPatternFacade = OutClothFacade.GetRenderPattern(DestPatternID);
			check(OutClothPatternFacade.GetNumRenderFaces() == 0);
			check(OutClothPatternFacade.GetNumRenderVertices() == 0);

			const int32 SourceMaterialID = MaterialIDs[DestPatternID];
			check(MaterialTriangles.Contains(SourceMaterialID));
			const TArray<int32>& TriangleIDs = MaterialTriangles[SourceMaterialID];

			TSet<int32> VertexIndices;
			for (int32 PatternTriangleIndex = 0; PatternTriangleIndex < TriangleIDs.Num(); ++PatternTriangleIndex)
			{
				const int32 TInd = TriangleIDs[PatternTriangleIndex];
				const FIndex3i Tri = DynamicMesh.GetTriangle(TInd);
				VertexIndices.Add(Tri[0]);
				VertexIndices.Add(Tri[1]);
				VertexIndices.Add(Tri[2]);
			}
			const TArray<int32> SourceVertexIndicesArray = VertexIndices.Array();
			const int32 NumVerticesThisPattern = SourceVertexIndicesArray.Num();

			OutClothPatternFacade.SetNumRenderVertices(NumVerticesThisPattern);
			TArrayView<FVector3f> RenderPosition = OutClothPatternFacade.GetRenderPosition();
			TArrayView<FVector3f> RenderNormal = OutClothPatternFacade.GetRenderNormal();
			TArrayView<FVector3f> RenderTangentU = OutClothPatternFacade.GetRenderTangentU();
			TArrayView<FVector3f> RenderTangentV = OutClothPatternFacade.GetRenderTangentV();
			TArrayView<TArray<FVector2f>> RenderUVs = OutClothPatternFacade.GetRenderUVs();
			TArrayView<FLinearColor> RenderColor = OutClothPatternFacade.GetRenderColor();
			TArrayView<TArray<int32>> RenderBoneIndices = OutClothPatternFacade.GetRenderBoneIndices();
			TArrayView<TArray<float>> RenderBoneWeights = OutClothPatternFacade.GetRenderBoneWeights();

			TMap<int32, int32> SourceToDestVertexMap;
			for (int32 PatternVertexIndex = 0; PatternVertexIndex < NumVerticesThisPattern; ++PatternVertexIndex)
			{
				const int32 SourceVertexIndex = SourceVertexIndicesArray[PatternVertexIndex];

				SourceToDestVertexMap.Add(SourceVertexIndex, PatternVertexIndex);

				RenderPosition[PatternVertexIndex] = FVector3f(DynamicMesh.GetVertex(SourceVertexIndex));
				if (bHasUVs)
				{
					RenderUVs[PatternVertexIndex] = UVs[SourceVertexIndex];
				}
				if (bHasNormals)
				{
					RenderNormal[PatternVertexIndex] = Normals[SourceVertexIndex];
				}
				RenderTangentU[PatternVertexIndex] = TangentUs[SourceVertexIndex];
				RenderTangentV[PatternVertexIndex] = TangentVs[SourceVertexIndex];
				RenderColor[PatternVertexIndex] = FLinearColor::White;
				RenderBoneIndices[PatternVertexIndex] = BoneIndices[SourceVertexIndex];
				RenderBoneWeights[PatternVertexIndex] = BoneWeights[SourceVertexIndex];

				if (!MeshVertexToPatternAndVertex.Contains(SourceVertexIndex))
				{
					MeshVertexToPatternAndVertex.Add({ SourceVertexIndex, FIntVector2{DestPatternID, PatternVertexIndex} });
				}
			}

			OutClothPatternFacade.SetNumRenderFaces(TriangleIDs.Num());
			for (int32 PatternTriangleIndex = 0; PatternTriangleIndex < TriangleIDs.Num(); ++PatternTriangleIndex)
			{
				const int32 VertexOffset = OutClothPatternFacade.GetRenderVerticesOffset();
				TArrayView<FIntVector3> RenderIndices = OutClothPatternFacade.GetRenderIndices();

				const int32 TInd = TriangleIDs[PatternTriangleIndex];
				const FIndex3i SourceTri = DynamicMesh.GetTriangle(TInd);

				RenderIndices[PatternTriangleIndex][0] = VertexOffset + SourceToDestVertexMap[SourceTri[0]];
				RenderIndices[PatternTriangleIndex][1] = VertexOffset + SourceToDestVertexMap[SourceTri[1]];
				RenderIndices[PatternTriangleIndex][2] = VertexOffset + SourceToDestVertexMap[SourceTri[2]];
			}

			FCollectionClothConstFacade InClothFacade(ClothCollection);
			FCollectionClothRenderPatternConstFacade InPatternFacade = InClothFacade.GetRenderPattern(SourceMaterialID);
			OutClothPatternFacade.SetRenderMaterialSoftObjectPathName(InPatternFacade.GetRenderMaterialSoftObjectPathName());
		}


		if (VertexWeightLayerNames.Num() > 0)
		{
			TArray<int32> DynamicMeshToClothVertexMap;
			DynamicMeshToClothVertexMap.SetNum(DynamicMesh.MaxVertexID());
			for (int InGlobalVertexIndex = 0; InGlobalVertexIndex < DynamicMesh.MaxVertexID(); ++InGlobalVertexIndex)
			{
				const int32 PatternID = MeshVertexToPatternAndVertex[InGlobalVertexIndex][0];
				const int32 VertexID = MeshVertexToPatternAndVertex[InGlobalVertexIndex][1];
				const int32 ClothGlobalIndex = OutClothFacade.GetRenderPattern(PatternID).GetRenderVerticesOffset() + VertexID;
				DynamicMeshToClothVertexMap[InGlobalVertexIndex] = ClothGlobalIndex;
			}

			for (int32 WeightLayerID = 0; WeightLayerID < VertexWeightLayerNames.Num(); ++WeightLayerID)
			{
				const bool bAdded = OutClothFacade.AddUserDefinedAttribute<float>(VertexWeightLayerNames[WeightLayerID], ClothCollectionGroup::RenderVertices);
				ensure(bAdded);
				TArrayView<float> ClothRenderWeights = OutClothFacade.GetUserDefinedAttribute<float>(VertexWeightLayerNames[WeightLayerID], ClothCollectionGroup::RenderVertices);
				ensure(ClothRenderWeights.Num() == VertexWeights[WeightLayerID].Num());

				for (int32 MeshVertexIndex = 0; MeshVertexIndex < VertexWeights[WeightLayerID].Num(); ++MeshVertexIndex)
				{
					const float VertexWeight = VertexWeights[WeightLayerID][MeshVertexIndex];
					const int32 ClothVertexIndex = DynamicMeshToClothVertexMap[MeshVertexIndex];
					ClothRenderWeights[ClothVertexIndex] = VertexWeight;
				}
			}
		}


	}


}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////


FChaosClothAssetRemeshNode_v2::FChaosClothAssetRemeshNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&TriangleSelectionSim.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterInputConnection(&DensityMapSim.WeightMap);

	RegisterInputConnection(&TriangleSelectionRender.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterInputConnection(&DensityMapRender.WeightMap);

	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetRemeshNode_v2::GetSimSelectionFromContext(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection, const FString& SimDensityMap, TSet<int32>& OutSelection) const
{
	using namespace UE::Chaos::ClothAsset;

	OutSelection.Reset();

	switch (SelectionSourceSim)
	{
	case EChaosClothAssetRemeshSelectionSource::SelectionInput:
	{
		const FName SimTriangleSelectionName = FName(*GetValue(Context, &TriangleSelectionSim.StringValue));
		if (!FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, SimTriangleSelectionName, ClothCollectionGroup::SimFaces, OutSelection))
		{
			Context.Warning(
				FText::Format(LOCTEXT("SimSelection_NotFound", "Invalid input sim mesh selection \"{0}\" "),
					FText::FromName(SimTriangleSelectionName))
				, this);
		}
		break;
	}
	case EChaosClothAssetRemeshSelectionSource::DensityMapInput:
	{
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		const FName SimDensityMapName(SimDensityMap);
		if (SimDensityMapName != NAME_None && ClothFacade.HasWeightMap(SimDensityMapName))
		{
			TConstArrayView<float> WeightMap = ClothFacade.GetWeightMap(SimDensityMapName);

			const TConstArrayView<FIntVector3> SimIndices3D = ClothFacade.GetSimIndices3D();
			// Select a face if all 3 vertices density values are below the threshold
			for (int32 FaceIndex = 0; FaceIndex < SimIndices3D.Num(); ++FaceIndex)
			{
				const FIntVector3& Index = SimIndices3D[FaceIndex];
				if (WeightMap[Index[0]] < DensityMapThresholdSim &&
					WeightMap[Index[1]] < DensityMapThresholdSim &&
					WeightMap[Index[2]] < DensityMapThresholdSim)
				{
					OutSelection.Add(FaceIndex);
				}
			}
		}
		break;
	}
	}
}

void FChaosClothAssetRemeshNode_v2::GetRenderSelectionFromContext(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection, const FString& RenderDensityMap, TSet<int32>& OutSelection) const
{
	using namespace UE::Chaos::ClothAsset;

	OutSelection.Reset();

	switch (SelectionSourceRender)
	{
	case EChaosClothAssetRemeshSelectionSource::SelectionInput:
	{
		const FName RenderTriangleSelectionName = FName(*GetValue(Context, &TriangleSelectionRender.StringValue));
		if (!FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, RenderTriangleSelectionName, ClothCollectionGroup::RenderFaces, OutSelection))
		{
			Context.Warning(
				FText::Format(LOCTEXT("RenderSelection_NotFound", "Invalid input render mesh selection \"{0}\" "),
					FText::FromName(RenderTriangleSelectionName))
				, this);
		}
		break;
	}
	case EChaosClothAssetRemeshSelectionSource::DensityMapInput:
	{
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		const FName RenderDensityMapName(RenderDensityMap);
		if (RenderDensityMapName != NAME_None && ClothFacade.HasWeightMap(RenderDensityMapName))
		{
			TConstArrayView<float> WeightMap = ClothFacade.GetWeightMap(RenderDensityMapName);

			const TConstArrayView<FIntVector3> SimIndices3D = ClothFacade.GetRenderIndices();
			// Select a face if all 3 vertices density values are below the threshold
			for (int32 FaceIndex = 0; FaceIndex < SimIndices3D.Num(); ++FaceIndex)
			{
				const FIntVector3& Index = SimIndices3D[FaceIndex];
				if (WeightMap[Index[0]] < DensityMapThresholdRender &&
					WeightMap[Index[1]] < DensityMapThresholdRender &&
					WeightMap[Index[2]] < DensityMapThresholdRender)
				{
					OutSelection.Add(FaceIndex);
				}
			}
		}
		break;
	}
	}
}
void FChaosClothAssetRemeshNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothFacade ClothFacade(ClothCollection);

		const FString SimDensityMapName = GetValue<FString>(Context, &DensityMapSim.WeightMap);
		const FString RenderDensityMapName = GetValue<FString>(Context, &DensityMapRender.WeightMap);

		// Copy collection to output
		TSharedRef<FManagedArrayCollection> OutputClothCollection = MakeShared<FManagedArrayCollection>();

		if ((bRemeshSim || bRemeshRender) && ClothFacade.IsValid(EClothCollectionExtendedSchemas::RenderDeformer))
		{
			Context.Warning(
				LOCTEXT("InputHasDeformerDataDetails", "The input Cloth Collection has Proxy Deformer data that will be removed by the Remesh node. Default deformer bindings will be computed in the final asset. Consider placing ProxyDeformer Node after the Remesh Node.")
				, this, Out
			);

			// Don't copy proxy deformer data
			const TArray<FName> GroupsToSkip = TArray<FName>();
			TArray<TTuple<FName, FName>> AttributesToSkip;
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerNumInfluences, ClothCollectionGroup::RenderPatterns });
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerPositionBaryCoordsAndDist, ClothCollectionGroup::RenderVertices });
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerNormalBaryCoordsAndDist, ClothCollectionGroup::RenderVertices });
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerTangentBaryCoordsAndDist, ClothCollectionGroup::RenderVertices });
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerSimIndices3D, ClothCollectionGroup::RenderVertices });
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerWeight, ClothCollectionGroup::RenderVertices });
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerSkinningBlend, ClothCollectionGroup::RenderVertices });

			ClothCollection->CopyTo(&OutputClothCollection.Get(), GroupsToSkip, AttributesToSkip);
		}
		else
		{
			ClothCollection->CopyTo(&OutputClothCollection.Get());
		}

		if (ClothFacade.IsValid())  // Can only act on the collection if it is a valid cloth collection
		{
			if (bRemeshSim)
			{
				const float TotalArea2D = Algo::Accumulate(ClothFacade.GetSimIndices2D(), 0.0f, [&ClothFacade](float PreviousSum, const FIntVector3& Tri)
					{
						const TArrayView<const FVector2f>& Vertices2D = ClothFacade.GetSimPosition2D();
						const float TriArea = FVector2f::CrossProduct(Vertices2D[Tri[1]] - Vertices2D[Tri[0]], Vertices2D[Tri[2]] - Vertices2D[Tri[0]]);
						return PreviousSum + FMath::Abs(TriArea);
					});

				if (TotalArea2D < UE_SMALL_NUMBER)
				{
					Context.Warning(
						LOCTEXT("SimMeshAreaZeroDetails", "The input Cloth Collection has a 2D Sim mesh with zero area. Sim remeshing will not be performed")
						, this, Out
					);
				}
				else
				{
					Private::EmptySimSelections(OutputClothCollection);

					TSet<int32> SimFaceSelection;
					GetSimSelectionFromContext(Context, ClothCollection, SimDensityMapName, SimFaceSelection);

					Private::FRemeshSimMeshParams Params;
					Params.Iterations = IterationsSim;
					Params.Smoothing = SmoothingSim;
					Params.DensityMapName = SimDensityMapName;
					Params.DensityLowHigh = FVector2f(DensityMapSim.Low / 100.0f, DensityMapSim.High / 100.0f);
					Params.FaceSelection = &SimFaceSelection;

					TOptional<FText> WarningMsg;
					Private::RemeshSimMesh(ClothCollection, Params, OutputClothCollection, WarningMsg);
					if (WarningMsg)
					{
						Context.Warning(*WarningMsg, this, Out);
					}
					Private::RebuildTopologyDependentSimData(ClothCollection, OutputClothCollection);
				}
			}

			if (bRemeshRender)
			{
				Private::EmptyRenderSelections(OutputClothCollection);

				TSet<int32> RenderFaceSelection;
				GetRenderSelectionFromContext(Context, ClothCollection, RenderDensityMapName, RenderFaceSelection);

				Private::FRemeshRenderMeshParams Params;
				Params.RemeshMethod = RemeshMethodRender;
				Params.bRemeshSeams = bRemeshRenderSeams;
				Params.SeamRemeshIterations = RenderSeamRemeshIterations;
				Params.Iterations = IterationsRender;
				Params.Smoothing = SmoothingRender;
				Params.DensityMapName = RenderDensityMapName;
				Params.DensityLowHigh = FVector2f(DensityMapRender.Low / 100.0f, DensityMapRender.High / 100.0f);
				Params.TargetPercent = TargetPercentRender;
				Params.FaceSelection = &RenderFaceSelection;

				Private::RemeshRenderMesh(ClothCollection, Params, OutputClothCollection);
			}
		}

		SetValue(Context, MoveTemp(*OutputClothCollection), &Collection);
	}

}

#if WITH_EDITOR

bool FChaosClothAssetRemeshNode_v2::CanDebugDraw() const
{
	return true;
}
bool FChaosClothAssetRemeshNode_v2::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return true;//ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name || ViewModeName == UE::Chaos::ClothAsset::FCloth3DSimViewMode::Name;
}

void FChaosClothAssetRemeshNode_v2::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if (DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned)
	{
		using namespace UE::Chaos::ClothAsset;

		if (bRemeshSim || bRemeshRender)
		{
			FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
			const TSharedRef<FManagedArrayCollection> InClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
			FCollectionClothFacade InClothFacade(InClothCollection);

			TStringBuilder<1024> Builder;
			if (InClothFacade.IsValid())  // Can only act on the collection if it is a valid cloth collection
			{
				Builder.Appendf(TEXT("Original:\n"));
				if (bRemeshSim)
				{
					Builder.Appendf(TEXT("- Sim 3D mesh: %d vertices, %d triangles\n"), InClothFacade.GetNumSimVertices3D(), InClothFacade.GetNumSimFaces());
				}
				if (bRemeshRender)
				{
					Builder.Appendf(TEXT("- Render mesh: %d vertices, %d triangles\n"), InClothFacade.GetNumRenderVertices(), InClothFacade.GetNumRenderFaces());
				}
			}

			if (const FDataflowOutput* Output = FindOutput(&Collection))
			{
				FManagedArrayCollection OutCollection = Output->ReadValue(Context, Collection);
				const TSharedRef<FManagedArrayCollection> OutClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(OutCollection));
				FCollectionClothFacade OutClothFacade(OutClothCollection);
				if (OutClothFacade.IsValid())  // Can only act on the collection if it is a valid cloth collection
				{
					Builder.Appendf(TEXT("Remeshed:\n"));
					if (bRemeshSim)
					{
						Builder.Appendf(TEXT("- Sim 3D mesh: %d vertices, %d triangles\n"), OutClothFacade.GetNumSimVertices3D(), OutClothFacade.GetNumSimFaces());
					}
					if (bRemeshRender)
					{
						Builder.Appendf(TEXT("- Render mesh: %d vertices, %d triangles\n"), OutClothFacade.GetNumRenderVertices(), OutClothFacade.GetNumRenderFaces());
					}
				}
			}
			if (Builder.Len() > 0)
			{
				DataflowRenderingInterface.DrawOverlayText(Builder.ToString());
			}
		}
	}
}
#endif


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////



namespace UE::Chaos::ClothAsset::RemeshNodeV1::Private
{

	using UE::Chaos::ClothAsset::Private::FSeamCollapseParameters;
	using UE::Chaos::ClothAsset::Private::FSeamEdge;
	using UE::Chaos::ClothAsset::Private::IsSharpCorner;
	using UE::Chaos::ClothAsset::Private::GetSeamEdgeCollapseParameters;
	using UE::Chaos::ClothAsset::Private::CanSplitSeamEdge;
	using UE::Chaos::ClothAsset::Private::FindCoincidentBoundaryVertices;
	using UE::Chaos::ClothAsset::Private::FEdgeCollapseParameters;
	using UE::Chaos::ClothAsset::Private::GetBoundaryEdgeCollapseParameters;


	/** Given weight values for the edge vertices, compute a new edge length multiplier to be used in the remesher.
		A greater edge length means the edge is more likely to be split, resulting in a higher overall vertex density */
		// TODO: Experiment with other forms
		// TODO: Add parameters to rescale the 0-1 weight values
	static double EdgeScalingFunction(float WeightValueA, float WeightValueB)
	{
		const float Avg = 0.5f * (WeightValueA + WeightValueB);
		const float ClampedWeight = FMath::Clamp(Avg, 0.0f, 1.0f);
		const double LengthScale = 1.0f + ClampedWeight;
		return LengthScale;
	}

	//
	// Functions to support seam remeshing
	//

	static void RemeshSeams(UE::Geometry::FDynamicMesh3& Mesh, TArray<TArray<FIntVector2>>& Seams, double TargetEdgeLength, const UE::Geometry::FDynamicMeshWeightAttribute* const DensityMapLayer)
	{
		using namespace UE::Geometry;

		// constants pulled from FRemesher::SetTargetEdgeLength
		const double MinLength = 0.66 * TargetEdgeLength;
		const double MaxLength = 1.33 * TargetEdgeLength;

		for (int32 SeamID = 0; SeamID < Seams.Num(); ++SeamID)
		{
			TArray<FIntVector2>& Seam = Seams[SeamID];

			for (int32 StitchID = 0; StitchID < Seam.Num() - 1; ++StitchID)
			{
				const int32 SideAVertexA = Seam[StitchID][0];
				const int32 SideAVertexB = Seam[StitchID + 1][0];

				if (Mesh.FindEdge(SideAVertexA, SideAVertexB) == UE::Geometry::FDynamicMesh3::InvalidID)
				{
					continue;
				}

				double EdgeALength = (Mesh.GetVertex(SideAVertexA) - Mesh.GetVertex(SideAVertexB)).Length();

				if (DensityMapLayer)
				{
					float WeightValueA;
					DensityMapLayer->GetValue(SideAVertexA, &WeightValueA);
					float WeightValueB;
					DensityMapLayer->GetValue(SideAVertexB, &WeightValueB);

					EdgeALength = EdgeScalingFunction(WeightValueA, WeightValueB) * EdgeALength;
				}

				const int32 SideBVertexA = Seam[StitchID][1];
				const int32 SideBVertexB = Seam[StitchID + 1][1];

				if (Mesh.FindEdge(SideBVertexA, SideBVertexB) == UE::Geometry::FDynamicMesh3::InvalidID)
				{
					continue;
				}

				double EdgeBLength = (Mesh.GetVertex(SideBVertexA) - Mesh.GetVertex(SideBVertexB)).Length();

				if (DensityMapLayer)
				{
					float WeightValueA;
					DensityMapLayer->GetValue(SideBVertexA, &WeightValueA);
					float WeightValueB;
					DensityMapLayer->GetValue(SideBVertexB, &WeightValueB);

					EdgeBLength = EdgeScalingFunction(WeightValueA, WeightValueB) * EdgeBLength;
				}

				if ((EdgeALength < MinLength) && (EdgeBLength < MinLength))
				{
					//
					// Collapse
					//

					const FSeamCollapseParameters CollapseConstraints = GetSeamEdgeCollapseParameters(SeamID, StitchID, Mesh, Seams);

					if (!CollapseConstraints.bCanCollapse)
					{
						continue;
					}

					const int32 PatternAKeepVertex = Seam[CollapseConstraints.KeepStitchIndex][0];
					const int32 PatternADeleteVertex = Seam[CollapseConstraints.DeleteStitchIndex][0];
					const int32 PatternBKeepVertex = Seam[CollapseConstraints.KeepStitchIndex][1];
					const int32 PatternBDeleteVertex = Seam[CollapseConstraints.DeleteStitchIndex][1];

					FDynamicMesh3::FEdgeCollapseInfo CollapseInfoA;
					const EMeshResult ResultA = Mesh.CollapseEdge(PatternAKeepVertex, PatternADeleteVertex, CollapseConstraints.NewPositionEdgeParameter, CollapseInfoA);

					FDynamicMesh3::FEdgeCollapseInfo CollapseInfoB;
					const EMeshResult ResultB = Mesh.CollapseEdge(PatternBKeepVertex, PatternBDeleteVertex, CollapseConstraints.NewPositionEdgeParameter, CollapseInfoB);

					check(ResultA == EMeshResult::Ok && ResultB == EMeshResult::Ok)

						Seam.RemoveAt(CollapseConstraints.DeleteStitchIndex);
				}
				else if ((EdgeALength > MaxLength) && (EdgeBLength > MaxLength))
				{
					//
					// Split
					//


					const bool bCanSplit = CanSplitSeamEdge(SeamID, StitchID, Seams);

					if (!bCanSplit)
					{
						continue;
					}

					FDynamicMesh3::FEdgeSplitInfo SplitInfoA;
					const EMeshResult ResultA = Mesh.SplitEdge(SideAVertexA, SideAVertexB, SplitInfoA);

					if (ResultA == EMeshResult::Ok)
					{
						if (Mesh.FindEdge(SideBVertexA, SideBVertexB) == UE::Geometry::FDynamicMesh3::InvalidID)
						{
							// Don't split the same edge twice
							continue;
						}

						FDynamicMesh3::FEdgeSplitInfo SplitInfoB;
						const EMeshResult ResultB = Mesh.SplitEdge(SideBVertexA, SideBVertexB, SplitInfoB);
						check(ResultB == EMeshResult::Ok);

						const FIntVector2 NewStitch{ SplitInfoA.NewVertex, SplitInfoB.NewVertex };

						Seam.Insert(NewStitch, StitchID + 1);
					}
				}
			}
		}
	}


	//
	// Boundary remeshing
	//

	static void RemeshBoundaries(UE::Geometry::FDynamicMesh3& Mesh, const TArray<TArray<FIntVector2>>& Seams, double TargetEdgeLength, const UE::Geometry::FDynamicMeshWeightAttribute* const DensityMapLayer)
	{
		using namespace UE::Geometry;

		auto IsSeamEdge = [](const TArray<TArray<FIntVector2>>& Seams, int EdgeVertexA, int EdgeVertexB)
			{
				for (const TArray<FIntVector2>& Seam : Seams)
				{
					for (int32 StitchID = 0; StitchID < Seam.Num() - 1; ++StitchID)
					{
						const FIntVector2& Stitch = Seam[StitchID];
						const FIntVector2& NextStitch = Seam[StitchID + 1];

						for (int32 Side = 0; Side < 2; ++Side)
						{
							if ((int)Stitch[Side] == EdgeVertexA && (int)NextStitch[Side] == EdgeVertexB)
							{
								return true;
							}

							if ((int)Stitch[Side] == EdgeVertexB && (int)NextStitch[Side] == EdgeVertexA)
							{
								return true;
							}
						}
					}
				}

				return false;
			};

		// constants pulled from FRemesher::SetTargetEdgeLength
		const double MinLength = 0.66 * TargetEdgeLength;
		const double MaxLength = 1.33 * TargetEdgeLength;

		// Get the set of boundary edges up front and then check if they get invalidated later. 
		// If we process edges inside this loop it tends to collapse a bunch of sequential edges, which can lead to very high-valence vertices
		TArray<FIndex2i> BoundaryEdges;
		for (const int EdgeID : Mesh.BoundaryEdgeIndicesItr())
		{
			BoundaryEdges.Add(Mesh.GetEdge(EdgeID).Vert);
		}

		for (const FIndex2i& EdgeVerts : BoundaryEdges)
		{
			const int EdgeID = Mesh.FindEdge(EdgeVerts[0], EdgeVerts[1]);
			if (EdgeID == FDynamicMesh3::InvalidID || !Mesh.IsBoundaryEdge(EdgeID))
			{
				continue;
			}

			if (IsSeamEdge(Seams, EdgeVerts[0], EdgeVerts[1]))
			{
				continue;
			}

			double EdgeLength = (Mesh.GetVertex(EdgeVerts[0]) - Mesh.GetVertex(EdgeVerts[1])).Length();

			if (DensityMapLayer)
			{
				float WeightValueA;
				DensityMapLayer->GetValue(EdgeVerts[0], &WeightValueA);
				float WeightValueB;
				DensityMapLayer->GetValue(EdgeVerts[1], &WeightValueB);

				EdgeLength = EdgeScalingFunction(WeightValueA, WeightValueB) * EdgeLength;
			}


			if (EdgeLength < MinLength)
			{
				const FEdgeCollapseParameters CollapseParams = GetBoundaryEdgeCollapseParameters(Mesh, Seams, EdgeVerts);

				if (CollapseParams.bCanCollapse)
				{
					FDynamicMesh3::FEdgeCollapseInfo CollapseInfo;
					EMeshResult CollapseResult = Mesh.CollapseEdge(CollapseParams.KeepVertexIndex, CollapseParams.DeleteVertexIndex, CollapseParams.NewPositionEdgeParameter, CollapseInfo);
				}
			}
			else if (EdgeLength > MaxLength)
			{
				FDynamicMesh3::FEdgeSplitInfo SplitInfo;
				EMeshResult SplitResult = Mesh.SplitEdge(EdgeVerts[0], EdgeVerts[1], SplitInfo);
			}
		};
	}


	//
	// Remeshing away from Seams/Boundaries
	// 

	static bool Remesh(UE::Geometry::FDynamicMesh3& Mesh, double TargetEdgeLength, int32 Iterations, float SmoothingRate, bool bUniformSmoothing, const TArray<TArray<FIntVector2>>& Seams, const FString& DensityMapName, UE::Geometry::FCompactMaps* CompactMaps = nullptr)
	{
		using namespace UE::Geometry;

		//
		// These consts control overall remeshing behavior and are analogs of the properties exposed to the user in the actual Remesh tool
		// (i.e. things we might wish to add to the node properties in the future)
		//

		constexpr bool bReprojectToInputMesh = true;
		constexpr bool bDiscardAttributes = false;
		constexpr bool bUseFullRemeshPasses = true;
		constexpr bool bAllowFlips = true;
		constexpr bool bAllowSplits = true;
		constexpr bool bAllowCollapses = true;
		constexpr bool bPreventNormalFlips = true;
		constexpr bool bPreventTinyTriangles = true;
		constexpr bool bAutoCompact = true;
		constexpr bool bCoarsenBoundaries = false;

		const ERemeshSmoothingType SmoothingType = bUniformSmoothing ? ERemeshSmoothingType::Uniform : ERemeshSmoothingType::MeanValue;

		// Mesh seam behavior
		// Here we are talking UV, Normal, and Color seams, not Cloth Seams
		// This controls bAllowSeamCollapse and bAllowSeamSmoothing on those overlay seams
		// NOTE: These seams are not affected by bReprojectConstraints
		constexpr bool bPreserveSharpEdges = false;

		// Mesh boundaries
		const EEdgeRefineFlags MeshBoundaryConstraint = bCoarsenBoundaries ? EEdgeRefineFlags::NoFlip : EEdgeRefineFlags::FullyConstrained;

		// Group ID boundaries
		const EEdgeRefineFlags PolyGroupBoundaryConstraint = bCoarsenBoundaries ? EEdgeRefineFlags::NoFlip : EEdgeRefineFlags::FullyConstrained;

		// Material ID boundaries
		const EEdgeRefineFlags MaterialBoundaryConstraint = bCoarsenBoundaries ? EEdgeRefineFlags::NoFlip : EEdgeRefineFlags::FullyConstrained;

		// Whether to move boundary vertices back onto the poly-line defined by the original boundary in case of collapse
		const bool bReprojectConstraints = bCoarsenBoundaries;

		// Seam "corners" are held fixed. We use this angle threshold to determine what constitutes a seam corner
		constexpr float SeamCornerThresholdAngleDegrees = 45.0;

		FRemeshMeshOp RemeshOp;

		TSharedPtr<FDynamicMesh3> SourceMesh = MakeShared<FDynamicMesh3>(MoveTemp(Mesh));
		TSharedPtr<FDynamicMeshAABBTree3> SourceSpatial;
		if (bReprojectToInputMesh)
		{
			// acceleration structure is only used for reprojecting
			SourceSpatial = MakeShared<FDynamicMeshAABBTree3>(SourceMesh.Get(), true);
		}

		RemeshOp.OriginalMesh = SourceMesh;
		RemeshOp.OriginalMeshSpatial = SourceSpatial;

		RemeshOp.bDiscardAttributes = bDiscardAttributes;
		RemeshOp.RemeshType = (bUseFullRemeshPasses) ? ERemeshType::FullPass : ERemeshType::Standard;
		RemeshOp.RemeshIterations = Iterations;
		RemeshOp.MaxRemeshIterations = Iterations;
		RemeshOp.ExtraProjectionIterations = 0;		// unused for regular remeshing
		RemeshOp.TriangleCountHint = 0;				// unused for regular remeshing
		RemeshOp.SmoothingStrength = FMath::Clamp(SmoothingRate, 0.0f, 1.0f);
		RemeshOp.SmoothingType = SmoothingType;

		RemeshOp.TargetEdgeLength = TargetEdgeLength;
		RemeshOp.bPreserveSharpEdges = bPreserveSharpEdges;
		RemeshOp.bFlips = bAllowFlips;
		RemeshOp.bSplits = bAllowSplits;
		RemeshOp.bCollapses = bAllowCollapses;
		RemeshOp.bPreventNormalFlips = bPreventNormalFlips;
		RemeshOp.bPreventTinyTriangles = bPreventTinyTriangles;
		RemeshOp.MeshBoundaryConstraint = MeshBoundaryConstraint;
		RemeshOp.GroupBoundaryConstraint = PolyGroupBoundaryConstraint;
		RemeshOp.MaterialBoundaryConstraint = MaterialBoundaryConstraint;
		RemeshOp.bReproject = bReprojectToInputMesh;
		RemeshOp.ProjectionTarget = SourceMesh.Get();
		RemeshOp.ProjectionTargetSpatial = SourceSpatial.Get();
		RemeshOp.bReprojectConstraints = bReprojectConstraints;
		RemeshOp.BoundaryCornerAngleThreshold = SeamCornerThresholdAngleDegrees;
		RemeshOp.TargetMeshLocalToWorld = FTransformSRT3d::Identity();
		RemeshOp.ToolMeshLocalToWorld = FTransformSRT3d::Identity();
		RemeshOp.bUseWorldSpace = false;
		RemeshOp.bParallel = true;

		// RemeshOp makes a copy of the mesh to operate on, so we can't just pass the FDynamicMeshWeightAttribute pointer into CustomEdgeLengthScaleF
		int32 DensityMapLayerIndex = -1;
		bool bFoundDensityMapLayer = false;
		if (!DensityMapName.IsEmpty() && SourceMesh->HasAttributes())
		{
			for (int32 WeightLayerIndex = 0; WeightLayerIndex < SourceMesh->Attributes()->NumWeightLayers(); ++WeightLayerIndex)
			{
				if (SourceMesh->Attributes()->GetWeightLayer(WeightLayerIndex)->GetName() == DensityMapName)
				{
					DensityMapLayerIndex = WeightLayerIndex;
					bFoundDensityMapLayer = true;
					break;
				}
			}
		}

		RemeshOp.CustomEdgeLengthScaleF = [bFoundDensityMapLayer, DensityMapLayerIndex](const FDynamicMesh3& Mesh, int VertexA, int VertexB) -> double
			{
				if (bFoundDensityMapLayer)
				{
					check(Mesh.HasAttributes());
					const FDynamicMeshWeightAttribute* const DensityMapLayer = Mesh.Attributes()->GetWeightLayer(DensityMapLayerIndex);
					check(DensityMapLayer);

					float WeightValueA;
					DensityMapLayer->GetValue(VertexA, &WeightValueA);

					float WeightValueB;
					DensityMapLayer->GetValue(VertexB, &WeightValueB);

					return EdgeScalingFunction(WeightValueA, WeightValueB);
				}

				return 1.0;
			};

		// Set up constraints for Cloth Seam edges
		UE::Geometry::FMeshConstraints Constraints;
		for (const TArray<FIntVector2>& Seam : Seams)
		{
			if (Seam.Num() == 1)
			{
				for (int32 Side = 0; Side < 2; ++Side)
				{
					constexpr bool bCannotDelete = true;
					constexpr bool bCanMove = false;
					FVertexConstraint VertexConstraint(bCannotDelete, bCanMove);
					Constraints.SetOrCombineVertexConstraint(Seam[0][Side], VertexConstraint);
				}
			}
			else
			{
				for (int32 StitchIndex = 0; StitchIndex < Seam.Num() - 1; ++StitchIndex)
				{
					for (int32 Side = 0; Side < 2; ++Side)
					{
						const int EdgeID = SourceMesh->FindEdge(Seam[StitchIndex][Side], Seam[StitchIndex + 1][Side]);
						check(EdgeID != FDynamicMesh3::InvalidID);
						FEdgeConstraint EdgeConstraint(EEdgeRefineFlags::FullyConstrained);
						Constraints.SetOrUpdateEdgeConstraint(EdgeID, EdgeConstraint);

						constexpr bool bCannotDelete = true;
						constexpr bool bCanMove = false;
						FVertexConstraint VertexConstraint(bCannotDelete, bCanMove);
						Constraints.SetOrCombineVertexConstraint(Seam[StitchIndex][Side], VertexConstraint);
						Constraints.SetOrCombineVertexConstraint(Seam[StitchIndex + 1][Side], VertexConstraint);
					}
				}
			}
		}
		RemeshOp.SetUserSpecifiedConstraints(Constraints);

		constexpr FProgressCancel* Progress = nullptr;		// Don't allow cancel or report progress for now
		RemeshOp.CalculateResult(Progress);

		if (RemeshOp.GetResultInfo().Result == EGeometryResultType::Success)
		{
			TUniquePtr<FDynamicMesh3> ResultMesh = RemeshOp.ExtractResult();
			Mesh = MoveTemp(*ResultMesh);
		}
		else
		{
			return false;
		}

		// compact the input mesh if enabled
		if (bAutoCompact)
		{
			Mesh.CompactInPlace(CompactMaps);
		}

		return true;
	}

}


FChaosClothAssetRemeshNode::FChaosClothAssetRemeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterInputConnection(&DensityMapSim.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterInputConnection(&DensityMapRender.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
}


void FChaosClothAssetRemeshNode::RemeshSimMesh(const TSharedRef<const FManagedArrayCollection>& ClothCollection,
	const FString& DensityMapName,
	const TSharedRef<FManagedArrayCollection>& OutClothCollection) const
{
	using namespace UE::Geometry;
	using namespace UE::Chaos::ClothAsset;

	FCollectionClothConstFacade InClothFacade(ClothCollection);

	if (InClothFacade.GetNumSimPatterns() == 0)
	{
		FClothGeometryTools::DeleteSimMesh(OutClothCollection);
		return;
	}

	// Convert input patterns to a DynamicMesh

	FClothPatternToDynamicMesh Converter;

	FDynamicMesh3 Mesh2D;
	Converter.Convert(ClothCollection, INDEX_NONE, EClothPatternVertexType::Sim2D, Mesh2D);

	const double TotalArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(Mesh2D).Y;
	checkf(TotalArea > 0, TEXT("FChaosClothAssetRemeshNode::RemeshSimMesh: Expected 2D Sim mesh to have a positive area"));
	const int TriangleCount = Mesh2D.TriangleCount();

	// Copy pattern IDs into polygroup layer

	Mesh2D.EnableAttributes();
	const int32 PatternIndexLayerID = Mesh2D.Attributes()->NumPolygroupLayers();
	Mesh2D.Attributes()->SetNumPolygroupLayers(PatternIndexLayerID + 1);
	FDynamicMeshPolygroupAttribute* const PatternIndexLayer = Mesh2D.Attributes()->GetPolygroupLayer((int)PatternIndexLayerID);
	check(Mesh2D.TriangleCount() == InClothFacade.GetNumSimFaces());
	for (int32 FaceIndex = 0; FaceIndex < InClothFacade.GetNumSimFaces(); ++FaceIndex)
	{
		const int32 PatternID = InClothFacade.FindSimPatternByFaceIndex(FaceIndex);
		PatternIndexLayer->SetValue(FaceIndex, PatternID);
	}

	// Look for the DensityMap weight layer
	const FDynamicMeshWeightAttribute* DensityMapLayer = nullptr;
	if (Mesh2D.HasAttributes())
	{
		for (int32 WeightLayerIndex = 0; WeightLayerIndex < Mesh2D.Attributes()->NumWeightLayers(); ++WeightLayerIndex)
		{
			if (Mesh2D.Attributes()->GetWeightLayer(WeightLayerIndex)->GetName() == DensityMapName)
			{
				DensityMapLayer = Mesh2D.Attributes()->GetWeightLayer(WeightLayerIndex);
				break;
			}
		}
	}

	// Grab the seam information from the input collection

	// Seams are comprised of a set of Stitches. Each Stitch is simply a pair of vertex indices indicating vertices that should be welded to form the 3D mesh.

	// Stitches are given in random order within the Seam. To make remeshing them easier, we will find connected strips of stitches and store them in sequential order.
	// So the vertices in Stitch N are connected to the vertices in Stitch N+1.

	TArray<TArray<FIntVector2>> Seams;
	for (int32 SeamIndex = 0; SeamIndex < InClothFacade.GetNumSeams(); ++SeamIndex)
	{
		FCollectionClothSeamConstFacade SeamFacade = InClothFacade.GetSeam(SeamIndex);
		FClothGeometryTools::BuildConnectedSeams2D(ClothCollection, SeamIndex, Mesh2D, Seams);

		// Check seams are valid
		for (int32 SeamID = 0; SeamID < Seams.Num(); ++SeamID)
		{
			const TArray<FIntVector2>& SubSeam = Seams[SeamID];
			for (int32 StitchID = 0; StitchID < SubSeam.Num() - 1; ++StitchID)
			{
				const int32 NextStitchID = StitchID + 1;
				for (int32 Side = 0; Side < 2; ++Side)
				{
					const int32 StitchVert = SubSeam[StitchID][Side];
					const int32 NextStitchVert = SubSeam[NextStitchID][Side];

					if (StitchVert == NextStitchVert)
					{
						continue;
					}

					const int32 FoundEdge = Mesh2D.FindEdge(StitchVert, NextStitchVert);

					// This would indicate a problem in BuildConnectedSeams
					checkf(FoundEdge != UE::Geometry::FDynamicMesh3::InvalidID, TEXT("Could not find a mesh edge between sequential seam vertices %d, %d"), StitchVert, NextStitchVert);
				}
			}
		}
	}


	// For remeshing purposes, we also want to find and constrain any vertices that connect the two seam sides together
	// (This is only relevant for "internal" seams, which connect vertices within the same pattern)
	for (TArray<FIntVector2>& Seam : Seams)
	{
		if (Seam.Num() == 0)
		{
			continue;
		}

		for (const FIntVector2& EndStitch : { Seam[0], Seam.Last() })
		{
			for (const int NeighborA : Mesh2D.VtxVerticesItr(EndStitch[0]))
			{
				bool bMatchFound = false;

				for (const int NeighborB : Mesh2D.VtxVerticesItr(EndStitch[1]))
				{
					if (NeighborA == NeighborB)
					{
						if (Seam.Contains(FIntVector2(NeighborA, NeighborB)))
						{
							continue;
						}

						if (EndStitch == Seam[0])
						{
							Seam.Insert(FIntVector2(NeighborA, NeighborB), 0);
						}
						else
						{
							Seam.Add(FIntVector2(NeighborA, NeighborB));
						}

						bMatchFound = true;
						break;
					}
				}

				if (bMatchFound)
				{
					break;
				}
			}
		}
	}

	// Remesh seams

	const int TargetTriangleCount = FMath::RoundToInt(static_cast<float>(TargetPercentSim) / 100.0f * static_cast<float>(TriangleCount));
	const double TargetEdgeLength = FRemeshMeshOp::CalculateTargetEdgeLength(nullptr, TargetTriangleCount, TotalArea);

	for (int32 ResampleIter = 0; ResampleIter < IterationsSim; ++ResampleIter)
	{
		UE::Chaos::ClothAsset::RemeshNodeV1::Private::RemeshSeams(Mesh2D, Seams, TargetEdgeLength, DensityMapLayer);
	}

	// Remesh boundaries

	for (int32 ResampleIter = 0; ResampleIter < IterationsSim; ++ResampleIter)
	{
		UE::Chaos::ClothAsset::RemeshNodeV1::Private::RemeshBoundaries(Mesh2D, Seams, TargetEdgeLength, DensityMapLayer);
	}

	// Do the remeshing of the rest of the mesh

	UE::Geometry::FCompactMaps CompactMaps;
	constexpr bool bUniformSmoothing = true;
	UE::Chaos::ClothAsset::RemeshNodeV1::Private::Remesh(Mesh2D, TargetEdgeLength, IterationsSim, SmoothingSim, bUniformSmoothing, Seams, DensityMapName, &CompactMaps);

	// Update stitches
	for (TArray<FIntVector2>& Seam : Seams)
	{
		for (FIntVector2& Stitch : Seam)
		{
			Stitch[0] = CompactMaps.GetVertexMapping(Stitch[0]);
			Stitch[1] = CompactMaps.GetVertexMapping(Stitch[1]);
			checkf(Stitch[0] != FDynamicMesh3::InvalidID, TEXT("Stitch vertex %d was deleted by remeshing"), Stitch[0]);
			checkf(Stitch[1] != FDynamicMesh3::InvalidID, TEXT("Stitch vertex %d was deleted by remeshing"), Stitch[1]);
		}
	}


	// Project the 3D vertices onto the input 3D mesh

	// For each 2D vertex, we will find the closest triangle on the input 2D mesh, then look up that triangle on the input 3D mesh to get the final 3D location.
	// We will do this pattern-by-pattern to handle issues where the patterns overlap in 2D space.

	const FDynamicMeshPolygroupAttribute* const NewPatternIndexLayer = Mesh2D.Attributes()->GetPolygroupLayer((int)PatternIndexLayerID);
	check(NewPatternIndexLayer);

	FDynamicMesh3 Mesh3D;
	Mesh3D.Copy(Mesh2D);

	TMap<int32, TSet<int32>> PatternVertexIDs;
	for (int32 TID : Mesh2D.TriangleIndicesItr())
	{
		const int32 PatternID = NewPatternIndexLayer->GetValue(TID);
		if (!PatternVertexIDs.Contains(PatternID))
		{
			PatternVertexIDs.Add(PatternID, TSet<int32>());
		}

		const FIndex3i Tri = Mesh2D.GetTriangle(TID);
		PatternVertexIDs[PatternID].Add(Tri[0]);
		PatternVertexIDs[PatternID].Add(Tri[1]);
		PatternVertexIDs[PatternID].Add(Tri[2]);
	}

	for (int32 PatternID = 0; PatternID < InClothFacade.GetNumSimPatterns(); ++PatternID)
	{
		if (!PatternVertexIDs.Contains(PatternID))
		{
			continue;
		}

		FDynamicMesh3 ProjectionTarget2D;
		Converter.Convert(ClothCollection, PatternID, EClothPatternVertexType::Sim2D, ProjectionTarget2D);
		FDynamicMesh3 ProjectionTarget3D;
		Converter.Convert(ClothCollection, PatternID, EClothPatternVertexType::Sim3D, ProjectionTarget3D);
		TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3> ProjectionTargetSpatial = MakeShared<UE::Geometry::FDynamicMeshAABBTree3>(&ProjectionTarget2D, true);

		for (const int32 VID : PatternVertexIDs[PatternID])
		{
			const FVector3d SrcVert = Mesh2D.GetVertex(VID);

			double Distance;
			const int NearestTriangle = ProjectionTargetSpatial->FindNearestTriangle(SrcVert, Distance);

			const FDistPoint3Triangle3d Dist = TMeshQueries<FDynamicMesh3>::TriangleDistance(ProjectionTarget2D, NearestTriangle, SrcVert);
			const FVector3d Bary = Dist.TriangleBaryCoords;
			const FVector3d InterpolatedPoint = ProjectionTarget3D.GetTriBaryPoint(NearestTriangle, Bary[0], Bary[1], Bary[2]);

			Mesh3D.SetVertex(VID, InterpolatedPoint);
		}
	}

	// Build the output cloth sim mesh

	constexpr bool bAppendToExistingMesh = false;
	constexpr bool bTransferWeightMaps = true;
	constexpr bool bTransferSimSkinningData = true;
	TMap<int, int32> DynamicMeshToClothVertexMap;
	FClothGeometryTools::BuildSimMeshFromDynamicMeshes(OutClothCollection, Mesh2D, Mesh3D, PatternIndexLayerID, bTransferWeightMaps, bTransferSimSkinningData, bAppendToExistingMesh, DynamicMeshToClothVertexMap);


	// Re-apply the seam info from the input sim mesh. This will create a new Seam for each set of connected stitches

	FCollectionClothFacade OutClothFacade(OutClothCollection);
	for (int32 SeamIndex = 0; SeamIndex < Seams.Num(); ++SeamIndex)
	{
		const TArray<FIntVector2>& Seam = Seams[SeamIndex];

		TArray<FIntVector2> NewSeam;
		for (const FIntVector2& Stitch : Seam)
		{
			if (Stitch[0] == Stitch[1])
			{
				continue;
			}

			FIntVector2 NewStitch;
			NewStitch[0] = DynamicMeshToClothVertexMap[Stitch[0]];
			NewStitch[1] = DynamicMeshToClothVertexMap[Stitch[1]];
			NewSeam.Add(NewStitch);
		}

		FCollectionClothSeamFacade NewSeamFacade = OutClothFacade.AddGetSeam();
		NewSeamFacade.Initialize(NewSeam);
	}
}


void FChaosClothAssetRemeshNode::RemeshRenderMesh(const TSharedRef<const FManagedArrayCollection>& ClothCollection,
	const FString& DensityMapName,
	const TSharedRef<FManagedArrayCollection>& OutClothCollection) const
{
	using namespace UE::Geometry;
	using namespace UE::Chaos::ClothAsset;

	// Get the source mesh
	FClothPatternToDynamicMesh Converter;
	FDynamicMesh3 DynamicMesh;

	// NOTE: When applied to the Render mesh, this Convert function will assign PatternIDs to the MaterialID attribute of the DynamicMesh. 
	// After remeshing we will use the MatrialID attribute to determine which triangles should go into which output pattern.
	Converter.Convert(ClothCollection, INDEX_NONE, EClothPatternVertexType::Render, DynamicMesh);

	check(DynamicMesh.HasAttributes());

	const int InputMeshVertexCount = DynamicMesh.VertexCount();
	const int InputMeshTriangleCount = DynamicMesh.TriangleCount();

	const bool bHasUVs = (DynamicMesh.Attributes()->PrimaryUV() != nullptr);

	const int TargetTriangleCount = FMath::RoundToInt(static_cast<float>(TargetPercentRender) / 100.0f * static_cast<float>(InputMeshTriangleCount));
	const double TargetEdgeLength = FRemeshMeshOp::CalculateTargetEdgeLength(&DynamicMesh, TargetTriangleCount);

	TArray<TArray<FIntVector2>> Seams;

	if (bRemeshRenderSeams)
	{
		// Create pseudo-stitches based on boundary vertex proximity. These stitches aren't going to actually weld vertices together, but they will guide boundary remeshing.
		// The goal is to maintain a vertex pairing along boundaries in order to avoid holes opening up when the mesh deforms due to skinning.
		TArray<FIntVector2> Stitches;
		UE::Chaos::ClothAsset::RemeshNodeV1::Private::FindCoincidentBoundaryVertices(DynamicMesh, Stitches);

		FClothGeometryTools::BuildConnectedSeams(Stitches, DynamicMesh, Seams);

		// Add density map for for Render mesh
		const UE::Geometry::FDynamicMeshWeightAttribute* DensityMapLayer = nullptr;
		if (DynamicMesh.HasAttributes())
		{
			for (int32 WeightLayerIndex = 0; WeightLayerIndex < DynamicMesh.Attributes()->NumWeightLayers(); ++WeightLayerIndex)
			{
				if (DynamicMesh.Attributes()->GetWeightLayer(WeightLayerIndex)->GetName() == DensityMapName)
				{
					DensityMapLayer = DynamicMesh.Attributes()->GetWeightLayer(WeightLayerIndex);
					break;
				}
			}
		}

		for (int RemeshPass = 0; RemeshPass < RenderSeamRemeshIterations; ++RemeshPass)
		{
			UE::Chaos::ClothAsset::RemeshNodeV1::Private::RemeshSeams(DynamicMesh, Seams, TargetEdgeLength, DensityMapLayer);
		}

		// Also remesh the open boundaries that are not constrained by seams
		for (int RemeshPass = 0; RemeshPass < RenderSeamRemeshIterations; ++RemeshPass)
		{
			UE::Chaos::ClothAsset::RemeshNodeV1::Private::RemeshBoundaries(DynamicMesh, Seams, TargetEdgeLength, DensityMapLayer);
		}
	}


	UE::Geometry::FCompactMaps CompactMaps;
	if (RemeshMethodRender == EChaosClothAssetRemeshMethod::Remesh)
	{
		constexpr bool bUniformSmoothing = false;	// uniform smoothing can distort the UV layer pretty badly
		const bool bSuccess = UE::Chaos::ClothAsset::RemeshNodeV1::Private::Remesh(DynamicMesh, TargetEdgeLength, IterationsRender, SmoothingRender, bUniformSmoothing, Seams, DensityMapName, &CompactMaps);
		check(bSuccess);
	}
	else
	{
		const bool bCoarsenBoundariesDuringSimplify = !bRemeshRenderSeams;
		const int TargetVertexCount = FMath::RoundToInt(static_cast<float>(TargetPercentRender) / 100.0f * static_cast<float>(InputMeshVertexCount));
		UE::Chaos::ClothAsset::Private::Simplify(DynamicMesh, TargetVertexCount, bCoarsenBoundariesDuringSimplify, &CompactMaps);
	}

	// Collect outputs

	//
	// Normals
	//

	const bool bHasNormals = (DynamicMesh.Attributes()->PrimaryNormals() != nullptr);
	check(bHasNormals);

	TArray<FVector3f> Normals;
	Normals.SetNum(DynamicMesh.VertexCount());
	const FDynamicMeshNormalOverlay* const NormalOverlay = DynamicMesh.Attributes()->PrimaryNormals();
	for (const int TriangleIndex : DynamicMesh.TriangleIndicesItr())
	{
		const FIndex3i Tri = DynamicMesh.GetTriangle(TriangleIndex);

		for (int TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
		{
			const int VertexIndex = Tri[TriangleVertexIndex];

			// NOTE: This assumes one normal per vertex in the overlay (i.e. no "hard edges")
			Normals[VertexIndex] = NormalOverlay->GetElementAtVertex(TriangleIndex, VertexIndex);
		}
	}

	//
	// Tangents
	//

	TArray<FVector3f> TangentUs;
	TArray<FVector3f> TangentVs;
	{
		const bool bHasTangentUs = (DynamicMesh.Attributes()->PrimaryTangents() != nullptr);
		const bool bHasTangentVs = (DynamicMesh.Attributes()->PrimaryBiTangents() != nullptr);
		if (!bHasTangentUs || !bHasTangentVs)
		{
			FMeshTangentsf::ComputeDefaultOverlayTangents(DynamicMesh);
		}
		TangentUs.SetNumZeroed(DynamicMesh.VertexCount());
		TangentVs.SetNumZeroed(DynamicMesh.VertexCount());

		const FDynamicMeshNormalOverlay* const TangentUOverlay = DynamicMesh.Attributes()->PrimaryTangents();
		const FDynamicMeshNormalOverlay* const TangentVOverlay = DynamicMesh.Attributes()->PrimaryBiTangents();

		for (const int TriangleIndex : DynamicMesh.TriangleIndicesItr())
		{
			const FIndex3i Tri = DynamicMesh.GetTriangle(TriangleIndex);

			for (int TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
			{
				const int VertexIndex = Tri[TriangleVertexIndex];

				TangentUs[VertexIndex] += TangentUOverlay->GetElementAtVertex(TriangleIndex, VertexIndex);
				TangentVs[VertexIndex] += TangentVOverlay->GetElementAtVertex(TriangleIndex, VertexIndex);
			}
		}

		for (int32 VertexIndex = 0; VertexIndex < DynamicMesh.VertexCount(); ++VertexIndex)
		{
			TangentUs[VertexIndex].Normalize();
			TangentVs[VertexIndex].Normalize();
		}
	}

	//
	// UVs
	//

	// TODO: Account for multiple UV layers
	TArray<FVector2f> UVs;
	if (bHasUVs)
	{
		UVs.SetNum(DynamicMesh.VertexCount());

		const FDynamicMeshUVOverlay* const UVOverlay = DynamicMesh.Attributes()->PrimaryUV();

		// Assume no seams in the dynamic mesh UV overlay
		checkSlow(!UVOverlay->HasInteriorSeamEdges());

		for (const int TriangleIndex : DynamicMesh.TriangleIndicesItr())
		{
			const FIndex3i Tri = DynamicMesh.GetTriangle(TriangleIndex);

			for (int TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
			{
				const int VertexIndex = Tri[TriangleVertexIndex];
				UVs[VertexIndex] = UVOverlay->GetElementAtVertex(TriangleIndex, VertexIndex);
			}
		}
	}

	//
	// Skin Weights
	//

	FDynamicMeshAttributeSet* Attributes = DynamicMesh.Attributes();
	check(Attributes);

	TArray<TArray<int32>> BoneIndices;
	TArray<TArray<float>> BoneWeights;
	BoneIndices.SetNum(DynamicMesh.VertexCount());
	BoneWeights.SetNum(DynamicMesh.VertexCount());

	for (const TPair<FName, TUniquePtr<FDynamicMeshVertexSkinWeightsAttribute>>& SkinWeightLayer : Attributes->GetSkinWeightsAttributes())
	{
		const FDynamicMeshVertexSkinWeightsAttribute* const SkinWeightAttribute = SkinWeightLayer.Value.Get();

		for (const int TriangleIndex : DynamicMesh.TriangleIndicesItr())
		{
			const FIndex3i Tri = DynamicMesh.GetTriangle(TriangleIndex);

			for (int TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
			{
				const int VertexIndex = Tri[TriangleVertexIndex];
				SkinWeightAttribute->GetValue(VertexIndex, BoneIndices[VertexIndex], BoneWeights[VertexIndex]);
			}
		}
	}


	// Find the set of triangles per MaterialID

	TMap<int32, TArray<int32>> MaterialTriangles;

	const FDynamicMeshMaterialAttribute* const MaterialAttribute = Attributes->GetMaterialID();
	check(MaterialAttribute);

	for (const int32 TriangleID : DynamicMesh.TriangleIndicesItr())
	{
		const int32 MaterialID = MaterialAttribute->GetValue(TriangleID);
		if (!MaterialTriangles.Contains(MaterialID))
		{
			MaterialTriangles.Add(MaterialID);
		}
		MaterialTriangles[MaterialID].Add(TriangleID);
	}

	TArray<int32> MaterialIDs;
	MaterialTriangles.GetKeys(MaterialIDs);

	const int32 NumMaterials = MaterialIDs.Num();

	//
	// Populate output cloth collection
	// 

	FClothGeometryTools::DeleteRenderMesh(OutClothCollection);
	FCollectionClothFacade OutClothFacade(OutClothCollection);
	OutClothFacade.SetNumRenderPatterns(NumMaterials);

	for (int32 DestPatternID = 0; DestPatternID < MaterialIDs.Num(); ++DestPatternID)
	{
		FCollectionClothRenderPatternFacade OutClothPatternFacade = OutClothFacade.GetRenderPattern(DestPatternID);
		check(OutClothPatternFacade.GetNumRenderFaces() == 0);
		check(OutClothPatternFacade.GetNumRenderVertices() == 0);

		const int32 SourceMaterialID = MaterialIDs[DestPatternID];
		check(MaterialTriangles.Contains(SourceMaterialID));
		const TArray<int32>& TriangleIDs = MaterialTriangles[SourceMaterialID];

		TSet<int32> VertexIndices;
		for (int32 PatternTriangleIndex = 0; PatternTriangleIndex < TriangleIDs.Num(); ++PatternTriangleIndex)
		{
			const int32 TInd = TriangleIDs[PatternTriangleIndex];
			const FIndex3i Tri = DynamicMesh.GetTriangle(TInd);
			VertexIndices.Add(Tri[0]);
			VertexIndices.Add(Tri[1]);
			VertexIndices.Add(Tri[2]);
		}
		const TArray<int32> SourceVertexIndicesArray = VertexIndices.Array();
		const int32 NumVerticesThisPattern = SourceVertexIndicesArray.Num();

		OutClothPatternFacade.SetNumRenderVertices(NumVerticesThisPattern);
		TArrayView<FVector3f> RenderPosition = OutClothPatternFacade.GetRenderPosition();
		TArrayView<FVector3f> RenderNormal = OutClothPatternFacade.GetRenderNormal();
		TArrayView<FVector3f> RenderTangentU = OutClothPatternFacade.GetRenderTangentU();
		TArrayView<FVector3f> RenderTangentV = OutClothPatternFacade.GetRenderTangentV();
		TArrayView<TArray<FVector2f>> RenderUVs = OutClothPatternFacade.GetRenderUVs();
		TArrayView<FLinearColor> RenderColor = OutClothPatternFacade.GetRenderColor();
		TArrayView<TArray<int32>> RenderBoneIndices = OutClothPatternFacade.GetRenderBoneIndices();
		TArrayView<TArray<float>> RenderBoneWeights = OutClothPatternFacade.GetRenderBoneWeights();

		TMap<int32, int32> SourceToDestVertexMap;
		for (int32 PatternVertexIndex = 0; PatternVertexIndex < NumVerticesThisPattern; ++PatternVertexIndex)
		{
			const int32 SourceVertexIndex = SourceVertexIndicesArray[PatternVertexIndex];

			SourceToDestVertexMap.Add(SourceVertexIndex, PatternVertexIndex);

			RenderPosition[PatternVertexIndex] = FVector3f(DynamicMesh.GetVertex(SourceVertexIndex));
			if (bHasUVs)
			{
				RenderUVs[PatternVertexIndex].SetNum(MAX_TEXCOORDS);
				RenderUVs[PatternVertexIndex][0] = UVs[SourceVertexIndex];
			}
			if (bHasNormals)
			{
				RenderNormal[PatternVertexIndex] = Normals[SourceVertexIndex];
			}
			RenderTangentU[PatternVertexIndex] = TangentUs[SourceVertexIndex];
			RenderTangentV[PatternVertexIndex] = TangentVs[SourceVertexIndex];
			RenderColor[PatternVertexIndex] = FLinearColor::White;
			RenderBoneIndices[PatternVertexIndex] = BoneIndices[SourceVertexIndex];
			RenderBoneWeights[PatternVertexIndex] = BoneWeights[SourceVertexIndex];
		}

		OutClothPatternFacade.SetNumRenderFaces(TriangleIDs.Num());
		for (int32 PatternTriangleIndex = 0; PatternTriangleIndex < TriangleIDs.Num(); ++PatternTriangleIndex)
		{
			const int32 VertexOffset = OutClothPatternFacade.GetRenderVerticesOffset();
			TArrayView<FIntVector3> RenderIndices = OutClothPatternFacade.GetRenderIndices();

			const int32 TInd = TriangleIDs[PatternTriangleIndex];
			const FIndex3i SourceTri = DynamicMesh.GetTriangle(TInd);

			RenderIndices[PatternTriangleIndex][0] = VertexOffset + SourceToDestVertexMap[SourceTri[0]];
			RenderIndices[PatternTriangleIndex][1] = VertexOffset + SourceToDestVertexMap[SourceTri[1]];
			RenderIndices[PatternTriangleIndex][2] = VertexOffset + SourceToDestVertexMap[SourceTri[2]];
		}

		FCollectionClothConstFacade InClothFacade(ClothCollection);
		FCollectionClothRenderPatternConstFacade InPatternFacade = InClothFacade.GetRenderPattern(SourceMaterialID);
		OutClothPatternFacade.SetRenderMaterialSoftObjectPathName(InPatternFacade.GetRenderMaterialSoftObjectPathName());
	}
}


void FChaosClothAssetRemeshNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothFacade ClothFacade(ClothCollection);

		const FString SimDensityMapName = GetValue<FString>(Context, &DensityMapSim.StringValue, FString());
		const FString RenderDensityMapName = GetValue<FString>(Context, &DensityMapRender.StringValue, FString());

		// Copy collection to output
		TSharedPtr<FManagedArrayCollection> OutputClothCollection = MakeShared<FManagedArrayCollection>();

		if ((bRemeshSim || bRemeshRender) && ClothFacade.IsValid(EClothCollectionExtendedSchemas::RenderDeformer))
		{
			Context.Warning(
				LOCTEXT("InputHasDeformerDataDetails", "The input Cloth Collection has Proxy Deformer data that will be removed by the Remesh node. Default deformer bindings will be computed in the final asset. Consider placing ProxyDeformer Node after the Remesh Node.")
				, this, Out
			);

			// Don't copy proxy deformer data
			const TArray<FName> GroupsToSkip = TArray<FName>();
			TArray<TTuple<FName, FName>> AttributesToSkip;
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerNumInfluences, ClothCollectionGroup::RenderPatterns });
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerPositionBaryCoordsAndDist, ClothCollectionGroup::RenderVertices});
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerNormalBaryCoordsAndDist, ClothCollectionGroup::RenderVertices });
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerTangentBaryCoordsAndDist, ClothCollectionGroup::RenderVertices });
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerSimIndices3D, ClothCollectionGroup::RenderVertices });
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerWeight, ClothCollectionGroup::RenderVertices });
			AttributesToSkip.Add({ ClothCollectionAttribute::RenderDeformerSkinningBlend, ClothCollectionGroup::RenderVertices });
			
			ClothCollection->CopyTo(OutputClothCollection.Get(), GroupsToSkip, AttributesToSkip);
		}
		else
		{
			ClothCollection->CopyTo(OutputClothCollection.Get());
		}

		TSharedRef<FManagedArrayCollection> OutputClothCollectionRef = OutputClothCollection.ToSharedRef();

		if (ClothFacade.IsValid())  // Can only act on the collection if it is a valid cloth collection
		{
			if (bRemeshSim)
			{			
				const float TotalArea2D = Algo::Accumulate(ClothFacade.GetSimIndices2D(), 0.0f, [&ClothFacade](float PreviousSum, const FIntVector3& Tri)
				{
					const TArrayView<const FVector2f>& Vertices2D = ClothFacade.GetSimPosition2D();
					const float TriArea = FVector2f::CrossProduct(Vertices2D[Tri[1]] - Vertices2D[Tri[0]], Vertices2D[Tri[2]] - Vertices2D[Tri[0]]);
					return PreviousSum + FMath::Abs(TriArea);
				});

				if (TotalArea2D < UE_SMALL_NUMBER)
				{
					Context.Warning(
						LOCTEXT("SimMeshAreaZeroDetails", "The input Cloth Collection has a 2D Sim mesh with zero area. Sim remeshing will not be performed")
						, this, Out
					);
				}
				else
				{
					Private::EmptySimSelections(OutputClothCollectionRef);
					Private::EmptySprings(OutputClothCollectionRef);
					RemeshSimMesh(ClothCollection, SimDensityMapName, OutputClothCollectionRef);
					Private::RebuildTopologyDependentSimData(ClothCollection, OutputClothCollectionRef);
				}
			}

			if (bRemeshRender)
			{
				Private::EmptyRenderSelections(OutputClothCollectionRef);
				RemeshRenderMesh(ClothCollection, RenderDensityMapName, OutputClothCollectionRef);
			}
		}

		SetValue(Context, MoveTemp(*OutputClothCollectionRef), &Collection);
	}
}


#undef LOCTEXT_NAMESPACE

