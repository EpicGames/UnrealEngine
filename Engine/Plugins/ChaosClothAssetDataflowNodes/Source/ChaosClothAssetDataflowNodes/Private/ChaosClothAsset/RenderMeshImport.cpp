// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/RenderMeshImport.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothRenderPatternFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"

#include "Engine/EngineTypes.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "MeshDescription.h"
#include "MeshUVChannelInfo.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		struct FBuildVertex
		{
			FVertexID OrigVertID;
			FVertexInstanceID OrigVertInstanceID;

			FVector3f Position;
			TArray<FVector2f> UVs;
			FVector3f Normal;
			FVector3f Tangent;
			float BiNormalSign;
			FVector4f Color;

			bool operator==(const FBuildVertex& Other) const
			{
				if (OrigVertID != Other.OrigVertID)
				{
					return false;
				}

				// No need to check position since it's from OrigVertID

				if (!ensure(UVs.Num() == Other.UVs.Num()))
				{
					return false;
				}
				for (int32 UVChannelIndex = 0; UVChannelIndex < UVs.Num(); ++UVChannelIndex)
				{
					if (!UVs[UVChannelIndex].Equals(Other.UVs[UVChannelIndex], UE_THRESH_UVS_ARE_SAME))
					{
						return false;
					}
				}
				if (!Normal.Equals(Other.Normal, UE_THRESH_NORMALS_ARE_SAME))
				{
					return false;
				}
				if (!Tangent.Equals(Other.Tangent, UE_THRESH_NORMALS_ARE_SAME))
				{
					return false;
				}
				if (BiNormalSign != Other.BiNormalSign) // I think these are just -1 or 1, so just comparing them
				{
					return false;
				}
				// It looks like we just quantize the 0-1 color values to uint8s. (Call FLinearColor::ToFColor(bSRGB = false)) when we consume these.
				// Going to be a little more strict in case we ever decide to switch to the gamma conversion.
				if (!Color.Equals(Other.Color, UE_THRESH_NORMALS_ARE_SAME))
				{
					return false;
				}
				return true;
			}
		};

		static void MergeVertexInstances(const FMeshDescription* const MeshDescription, TArray<FBuildVertex>& MergedVertices, TArray<int32>& VertexInstanceToMerged)
		{
			FStaticMeshConstAttributes Attributes(*MeshDescription);

			TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();

			TVertexInstanceAttributesConstRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();
			TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
			TVertexInstanceAttributesConstRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
			TVertexInstanceAttributesConstRef<float> VertexInstanceBiNormalSigns = Attributes.GetVertexInstanceBinormalSigns();
			TVertexInstanceAttributesConstRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();

			// Merge identical vertex instances that correspond with the same vertex.
			auto MakeBuildVertex = [&VertexPositions, &VertexInstanceUVs, &VertexInstanceNormals, &VertexInstanceTangents, &VertexInstanceBiNormalSigns, &VertexInstanceColors](FVertexID VertID, FVertexInstanceID VertexInstanceID)
				{
					FBuildVertex BuildVertex;
					BuildVertex.OrigVertID = VertID;
					BuildVertex.OrigVertInstanceID = VertexInstanceID;
					BuildVertex.Position = VertexPositions[VertID];
					BuildVertex.UVs.SetNumUninitialized(VertexInstanceUVs.GetNumChannels());
					for (int32 UVChannelIndex = 0; UVChannelIndex < VertexInstanceUVs.GetNumChannels(); ++UVChannelIndex)
					{
						BuildVertex.UVs[UVChannelIndex] = VertexInstanceUVs.Get(VertexInstanceID, UVChannelIndex);
					}
					BuildVertex.Normal = VertexInstanceNormals[VertexInstanceID];
					BuildVertex.Tangent = VertexInstanceTangents[VertexInstanceID];
					BuildVertex.BiNormalSign = VertexInstanceBiNormalSigns[VertexInstanceID];
					BuildVertex.Color = VertexInstanceColors[VertexInstanceID];
					return BuildVertex;
				};
			MergedVertices.Reset(MeshDescription->VertexInstances().GetArraySize());
			VertexInstanceToMerged.Init(INDEX_NONE, MeshDescription->VertexInstances().GetArraySize());

			for (const FVertexID& VertID : MeshDescription->Vertices().GetElementIDs())
			{
				TConstArrayView<FVertexInstanceID> VertexInstances = MeshDescription->GetVertexVertexInstanceIDs(VertID);
				if (VertexInstances.IsEmpty())
				{
					continue;
				}
				const int32 FirstMergedVert = MergedVertices.Add(MakeBuildVertex(VertID, VertexInstances[0]));
				VertexInstanceToMerged[VertexInstances[0].GetValue()] = FirstMergedVert;
				for (int32 LocalInstance = 1; LocalInstance < VertexInstances.Num(); ++LocalInstance)
				{
					FBuildVertex BuildVert = MakeBuildVertex(VertID, VertexInstances[LocalInstance]);
					bool bFoundDuplicate = false;
					for (int32 CompareIndex = FirstMergedVert; CompareIndex < MergedVertices.Num(); ++CompareIndex)
					{
						if (BuildVert == MergedVertices[CompareIndex])
						{
							bFoundDuplicate = true;
							VertexInstanceToMerged[VertexInstances[LocalInstance].GetValue()] = CompareIndex;
							break;
						}
					}

					if (!bFoundDuplicate)
					{
						const int32 AddedMergedVert = MergedVertices.Add(MoveTemp(BuildVert));
						VertexInstanceToMerged[VertexInstances[LocalInstance].GetValue()] = AddedMergedVert;
					}
				}
			}
		}
	}  // End namespace Private

	FRenderMeshImport::FRenderMeshImport(const FMeshDescription& InMeshDescription, const FMeshBuildSettings& BuildSettings)
	{
		using namespace UE::Chaos::ClothAsset::Private;

		const FMeshDescription* MeshDescription = &InMeshDescription;
		FMeshDescription WritableMeshDescription;
		if (BuildSettings.bRecomputeNormals || BuildSettings.bRecomputeTangents)
		{
			// check if any are invalid
			bool bHasInvalidNormals, bHasInvalidTangents;
			FStaticMeshOperations::HasInvalidVertexInstanceNormalsOrTangents(InMeshDescription, bHasInvalidNormals, bHasInvalidTangents);

			// if neither are invalid we are not going to recompute
			if (bHasInvalidNormals || bHasInvalidTangents)
			{
				WritableMeshDescription = InMeshDescription;
				MeshDescription = &WritableMeshDescription;
				FStaticMeshAttributes Attributes(WritableMeshDescription);
				if (!Attributes.GetTriangleNormals().IsValid() || !Attributes.GetTriangleTangents().IsValid())
				{
					// If these attributes don't exist, create them and compute their values for each triangle
					FStaticMeshOperations::ComputeTriangleTangentsAndNormals(WritableMeshDescription);
				}

				EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::BlendOverlappingNormals;
				ComputeNTBsOptions |= BuildSettings.bRecomputeNormals ? EComputeNTBsFlags::Normals : EComputeNTBsFlags::None;
				ComputeNTBsOptions |= BuildSettings.bRecomputeTangents ? EComputeNTBsFlags::Tangents : EComputeNTBsFlags::None;
				ComputeNTBsOptions |= BuildSettings.bUseMikkTSpace ? EComputeNTBsFlags::UseMikkTSpace : EComputeNTBsFlags::None;
				ComputeNTBsOptions |= BuildSettings.bComputeWeightedNormals ? EComputeNTBsFlags::WeightedNTBs : EComputeNTBsFlags::None;
				ComputeNTBsOptions |= BuildSettings.bRemoveDegenerates ? EComputeNTBsFlags::IgnoreDegenerateTriangles : EComputeNTBsFlags::None;

				FStaticMeshOperations::ComputeTangentsAndNormals(WritableMeshDescription, ComputeNTBsOptions);
			}
		}

		// Merge vertex instances that share the same vertex. These will become the pattern vertices.
		TArray<FBuildVertex> MergedVertices;
		TArray<int32> VertexInstanceToMerged;
		MergeVertexInstances(MeshDescription, MergedVertices, VertexInstanceToMerged);

		// Vertex data (this will MoveTemp stuff out of MergedVertices!!)
		TArray<FVertex> Vertices;
		if (const int32 NumVertices = MergedVertices.Num())
		{
			Vertices.SetNumUninitialized(NumVertices);

			for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				FBuildVertex& BuildVertex = MergedVertices[VertexIndex];
				FVertex& Vertex = Vertices[VertexIndex];
				Vertex.RenderPosition = BuildVertex.Position;
				Vertex.RenderNormal = BuildVertex.Normal;
				Vertex.RenderTangentU = BuildVertex.Tangent;
				Vertex.RenderTangentV = FVector3f::CrossProduct(BuildVertex.Normal, BuildVertex.Tangent).GetSafeNormal() * BuildVertex.BiNormalSign;
				new (&Vertex.RenderUVs) TArray<FVector2f>(MoveTemp(BuildVertex.UVs));
				Vertex.RenderColor = FLinearColor(BuildVertex.Color);
				Vertex.OriginalIndex = BuildVertex.OrigVertID.GetValue();
			}
		}

		// Face data
		TArray<FTriangle> Triangles;
		if (const int32 NumTriangles = MeshDescription->Triangles().Num())
		{
			Triangles.SetNumUninitialized(NumTriangles);

			int32 FaceIndex = 0;
			for (const FTriangleID TriangleID : MeshDescription->Triangles().GetElementIDs())
			{
				const FPolygonGroupID PolygonGroupID = MeshDescription->GetTrianglePolygonGroup(TriangleID);
				const TConstArrayView<FVertexInstanceID> VertexInstances = MeshDescription->GetTriangleVertexInstances(TriangleID);
				check(VertexInstances.Num() == 3);
				check(MergedVertices.IsValidIndex(VertexInstanceToMerged[VertexInstances[0].GetValue()]));
				check(MergedVertices.IsValidIndex(VertexInstanceToMerged[VertexInstances[1].GetValue()]));
				check(MergedVertices.IsValidIndex(VertexInstanceToMerged[VertexInstances[2].GetValue()]));
				FTriangle& Triangle = Triangles[FaceIndex];
				Triangle.VertexIndices = FIntVector3(
					VertexInstanceToMerged[VertexInstances[0].GetValue()],
					VertexInstanceToMerged[VertexInstances[1].GetValue()],
					VertexInstanceToMerged[VertexInstances[2].GetValue()]);
				Triangle.OriginalIndex = TriangleID.GetValue();
				Triangle.MaterialIndex = PolygonGroupID.GetValue();
				++FaceIndex;
			}

			// Sort the triangles by material index
			Triangles.Sort([](const FTriangle& Triangle0, const FTriangle& Triangle1) -> bool
				{
					return Triangle0.MaterialIndex < Triangle1.MaterialIndex;
				});

			// Split into material sections
			TArray<int32> RemappedVertexIndices;
			int32 SectionTrianglesStart = 0;
			int32 SectionMaterialIndex = Triangles[SectionTrianglesStart].MaterialIndex;
			int32 SectionMinVertex = Triangles[SectionTrianglesStart].VertexIndices.GetMin();
			int32 SectionMaxVertex = Triangles[SectionTrianglesStart].VertexIndices.GetMax();

			for (int32 TriangleIndex = SectionTrianglesStart + 1; ; ++TriangleIndex)
			{
				const bool bEndIteration = (TriangleIndex == Triangles.Num());
				if (bEndIteration || SectionMaterialIndex != Triangles[TriangleIndex].MaterialIndex)
				{
					// Add a new section with this triangle range
					FSection& Section = Sections.Emplace(SectionMaterialIndex);
					const int32 SectionTrianglesCount = TriangleIndex - SectionTrianglesStart;
					Section.Triangles = TConstArrayView<FTriangle>(Triangles.GetData() + SectionTrianglesStart, SectionTrianglesCount);
					Section.Vertices.Reserve(SectionMaxVertex - SectionMinVertex + 1);

					// Copy the section's vertices and remap the section's triangle vertex indices
					RemappedVertexIndices.Init(INDEX_NONE, Vertices.Num());
					int32 SectionNumTexCoords = MAX_TEXCOORDS;
					for (FTriangle& Triangle : Section.Triangles)
					{
						for (int32 Index = 0; Index < 3; ++Index)
						{
							int32& TriangleVertexIndex = Triangle.VertexIndices[Index];
							if (RemappedVertexIndices[TriangleVertexIndex] == INDEX_NONE)
							{
								RemappedVertexIndices[TriangleVertexIndex] = Section.Vertices.Emplace(Vertices[TriangleVertexIndex]);
								SectionNumTexCoords = FMath::Min(SectionNumTexCoords, Vertices[TriangleVertexIndex].RenderUVs.Num());
							}
							TriangleVertexIndex = RemappedVertexIndices[TriangleVertexIndex];
						}
					}
					Section.NumTexCoords = SectionNumTexCoords;

					if (bEndIteration)
					{
						break;
					}

					// Prepare next section
					SectionTrianglesStart = TriangleIndex;
					SectionMaterialIndex = Triangles[SectionTrianglesStart].MaterialIndex;
					SectionMinVertex = Triangles[SectionTrianglesStart].VertexIndices.GetMin();
					SectionMaxVertex = Triangles[SectionTrianglesStart].VertexIndices.GetMax();
					continue;
				}
				// Update Min/Max
				SectionMinVertex = FMath::Min(SectionMinVertex, Triangles[TriangleIndex].VertexIndices.GetMin());
				SectionMaxVertex = FMath::Max(SectionMaxVertex, Triangles[TriangleIndex].VertexIndices.GetMax());
			}
		}
	}

	void FRenderMeshImport::AddRenderSections(
		const TSharedRef<FManagedArrayCollection> ClothCollection,
		const TArray<FStaticMaterial>& Materials,
		const FName OriginalTrianglesName,
		const FName OriginalVerticesName
	)
	{
		TArray<FSoftObjectPath> MaterialPaths;
		MaterialPaths.Reserve(Materials.Num());
		for (const FStaticMaterial& StaticMaterial : Materials)
		{
			MaterialPaths.Add(StaticMaterial.MaterialInterface ? StaticMaterial.MaterialInterface.GetPathName() : FString());
		}

		AddRenderSections(ClothCollection, MaterialPaths, OriginalTrianglesName, OriginalVerticesName);
	}

	void FRenderMeshImport::AddRenderSections(
		const TSharedRef<FManagedArrayCollection> ClothCollection,
		const TArray<FSoftObjectPath>& MaterialPaths,
		const FName OriginalTrianglesName,
		const FName OriginalVerticesName
	)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (!ClothFacade.IsValid())
		{
			ClothFacade.DefineSchema();
		}

		// Add support for original indices
		ClothFacade.AddUserDefinedAttribute<TArray<int32>>(OriginalTrianglesName, ClothCollectionGroup::RenderFaces);
		ClothFacade.AddUserDefinedAttribute<TArray<int32>>(OriginalVerticesName, ClothCollectionGroup::RenderVertices);

		for (const TPair<int32, FSection>& Section : Sections)
		{
			const TArray<FVertex>& Vertices = Section.Value.Vertices;
			const TArray<FTriangle>& Triangles = Section.Value.Triangles;
			const int32 NumVertices = Vertices.Num();
			const int32 NumTriangles = Triangles.Num();
			if (NumVertices && NumTriangles)
			{
				FCollectionClothRenderPatternFacade ClothPatternFacade = ClothFacade.AddGetRenderPattern();
				ClothPatternFacade.SetNumRenderVertices(NumVertices);
				ClothPatternFacade.SetNumRenderFaces(NumTriangles);

				TArrayView<FVector3f> RenderPosition = ClothPatternFacade.GetRenderPosition();
				TArrayView<FVector3f> RenderNormal = ClothPatternFacade.GetRenderNormal();
				TArrayView<FVector3f> RenderTangentU = ClothPatternFacade.GetRenderTangentU();
				TArrayView<FVector3f> RenderTangentV = ClothPatternFacade.GetRenderTangentV();
				TArrayView<TArray<FVector2f>> RenderUVs = ClothPatternFacade.GetRenderUVs();
				TArrayView<FLinearColor> RenderColor = ClothPatternFacade.GetRenderColor();
				TArrayView<TArray<int32>> RenderBoneIndices = ClothPatternFacade.GetRenderBoneIndices();
				TArrayView<TArray<float>> RenderBoneWeights = ClothPatternFacade.GetRenderBoneWeights();
				const int32 VertexOffset = ClothPatternFacade.GetRenderVerticesOffset();
				const int32 FaceOffset = ClothPatternFacade.GetRenderFacesOffset();
				TArrayView<TArray<int32>> OriginalTriangles = ClothFacade.GetUserDefinedAttribute<TArray<int32>>(OriginalTrianglesName, ClothCollectionGroup::RenderFaces);   // Requery after adding a pattern
				TArrayView<TArray<int32>> OriginalVertices = ClothFacade.GetUserDefinedAttribute<TArray<int32>>(OriginalVerticesName, ClothCollectionGroup::RenderVertices);  // in case the array gets reallocated

				const int32 NumTexCoords = FMath::Min((int32)MAX_TEXCOORDS, Section.Value.NumTexCoords);
				for (int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
				{
					const FVertex& Vertex = Vertices[VertexIndex];
					RenderPosition[VertexIndex] = Vertex.RenderPosition;
					RenderNormal[VertexIndex] = Vertex.RenderNormal;
					RenderTangentU[VertexIndex] = Vertex.RenderTangentU;
					RenderTangentV[VertexIndex] = Vertex.RenderTangentV;
					RenderUVs[VertexIndex].SetNum(NumTexCoords);
					for (int32 TexCoordIndex = 0; TexCoordIndex < NumTexCoords; ++TexCoordIndex)
					{
						RenderUVs[VertexIndex][TexCoordIndex] = Vertex.RenderUVs[TexCoordIndex];
					}

					RenderColor[VertexIndex] = FLinearColor(Vertex.RenderColor);

					constexpr int32 NumBones = 0;
					RenderBoneIndices[VertexIndex].SetNum(NumBones);
					RenderBoneWeights[VertexIndex].SetNum(NumBones);

					OriginalVertices[VertexIndex + VertexOffset] = { Vertex.OriginalIndex };
				}

				TArrayView<FIntVector3> RenderIndices = ClothPatternFacade.GetRenderIndices();
				for (int32 FaceIndex = 0; FaceIndex < NumTriangles; ++FaceIndex)
				{
					RenderIndices[FaceIndex] = Triangles[FaceIndex].VertexIndices + FIntVector3(VertexOffset);

					OriginalTriangles[FaceIndex + FaceOffset] = { Triangles[FaceIndex].OriginalIndex };
				}

				const int32 MaterialIndex = Section.Key;
				if (MaterialPaths.IsValidIndex(MaterialIndex))
				{
					ClothPatternFacade.SetRenderMaterialSoftObjectPathName(MaterialPaths[MaterialIndex]);
				}
			}
		}
	}

	void FRenderMeshImport::AddRenderPatternSelectionSets(
		const TSharedRef<FManagedArrayCollection> ClothCollection,
		const TMap<FName, TSet<int32>>& PatternNamesToIndices,
		const FName OriginalTrianglesName
	)
	{
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (!ClothFacade.IsValid())
		{
			// The caller should ensure this is a cloth collection
			return;
		}

		FCollectionClothSelectionFacade ClothSelectionFacade(ClothCollection);
		if (!ClothSelectionFacade.IsValid())
		{
			ClothSelectionFacade.DefineSchema();
		}

		// Create pattern selection sets
		const TConstArrayView<TArray<int32>> OriginalTriangles = ClothFacade.GetUserDefinedAttribute<TArray<int32>>(
			OriginalTrianglesName,
			ClothCollectionGroup::RenderFaces
		);

		if (OriginalTriangles.Num())
		{
			const TConstArrayView<FIntVector3> TriangleToVertexIndex = ClothFacade.GetRenderIndices();
			const TArray<int32> OriginalToNewTriangles = FClothDataflowTools::GetOriginalToNewIndices<TArray<int32>>(
				OriginalTriangles,
				TriangleToVertexIndex.Num()
			);

			for (const TPair<FName, TSet<int32>>& PatternNameFaces : PatternNamesToIndices)
			{
				// Add the pattern triangle list as a selection set
				TSet<int32>& SelectionSet = ClothSelectionFacade.FindOrAddSelectionSet(PatternNameFaces.Key, ClothCollectionGroup::RenderFaces);
				SelectionSet.Empty(PatternNameFaces.Value.Num());
				for (const int32 Index : PatternNameFaces.Value)
				{
					if (OriginalToNewTriangles.IsValidIndex(Index) && OriginalToNewTriangles[Index] != INDEX_NONE)
					{
						SelectionSet.Emplace(OriginalToNewTriangles[Index]);
					}
				}
			}
		}
	}
}  // End namespace UE::Chaos::ClothAsset
