// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ReverseNormalsNode.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReverseNormalsNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetReverseNormalsNode"

FChaosClothAssetReverseNormalsNode::FChaosClothAssetReverseNormalsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetReverseNormalsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		if (FCollectionClothFacade(ClothCollection).IsValid())  // Can only act on the collection if it is a valid cloth collection
		{
			FClothGeometryTools::ReverseMesh(
				ClothCollection,
				bReverseSimMeshNormals,
				bReverseSimMeshWindingOrder,
				bReverseRenderMeshNormals,
				bReverseRenderMeshWindingOrder,
				SimPatterns,
				RenderPatterns);
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
