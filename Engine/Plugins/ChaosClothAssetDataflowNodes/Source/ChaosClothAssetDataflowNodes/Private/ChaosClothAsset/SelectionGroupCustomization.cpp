// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SelectionGroupCustomization.h"
#include "ChaosClothAsset/CollectionClothFacade.h"

namespace UE::Chaos::ClothAsset
{
	TSharedRef<IPropertyTypeCustomization> FSelectionGroupCustomization::MakeInstance()
	{
		return MakeShareable(new FSelectionGroupCustomization);
	}

	TArray<FName> FSelectionGroupCustomization::GetTargetGroupNames(const FManagedArrayCollection& /*Collection*/) const
	{
		return FCollectionClothFacade::GetValidClothCollectionGroupName();
	}
}  // End namespace UE::Chaos::ClothAsset
