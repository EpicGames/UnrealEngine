// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/PropertyGroupCustomization.h"

namespace UE::Chaos::ClothAsset
{
	/**
	 * Customization for selection node group names input.
	 */
	class FSelectionGroupCustomization : public UE::Dataflow::FPropertyGroupCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	private:
		//~ Begin FPropertyGroupCustomization interface
		virtual TArray<FName> GetTargetGroupNames(const FManagedArrayCollection& Collection) const override;
		//~ End FPropertyGroupCustomization interface
	};
}
