// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SelectionNode.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowObject.h"
#include "InteractiveToolChange.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SelectionNode)

#define LOCTEXT_NAMESPACE "FChaosClothAssetSelectionNode"

namespace UE::Chaos::ClothAsset::Private
{
	void ConvertWeightMapToVertexSelection(const TArray<float>& WeightMap, const float TransferSelectionThreshold, TSet<int32>& OutSelection)
	{
		OutSelection.Reset();
		for (int32 Index = 0; Index < WeightMap.Num(); ++Index)
		{
			if (WeightMap[Index] >= TransferSelectionThreshold)
			{
				OutSelection.Add(Index);
			}
		}
	}

	void ConvertWeightMapToFaceSelection(const TArray<float>& WeightMap, const float TransferSelectionThreshold, const TConstArrayView<FIntVector3>& Indices, TSet<int32>& OutSelection)
	{
		OutSelection.Reset();
		for (int32 FaceIndex = 0; FaceIndex < Indices.Num(); ++FaceIndex)
		{
			const FIntVector& Element = Indices[FaceIndex];
			if (WeightMap[Element[0]] >= TransferSelectionThreshold &&
				WeightMap[Element[1]] >= TransferSelectionThreshold &&
				WeightMap[Element[2]] >= TransferSelectionThreshold)
			{
				OutSelection.Add(FaceIndex);
			}
		}
	}

	template<bool bIsSecondarySelection>
	bool TransferSelectionSet(const TSharedRef<const FManagedArrayCollection>& TransferClothCollection, const TSharedRef<FManagedArrayCollection>& ClothCollection,
		const FName& InInputName, const FName& SelectionGroupName, const EChaosClothAssetWeightMapTransferType SimTransferType, const float TransferSelectionThreshold, TSet<int32>& OutSelection)
	{
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		FCollectionClothConstFacade TransferClothFacade(TransferClothCollection);
		FCollectionClothSelectionConstFacade TransferSelectionFacade(TransferClothCollection);

		const bool bIsValidRenderSelection = SelectionGroupName == ClothCollectionGroup::RenderFaces || SelectionGroupName == ClothCollectionGroup::RenderVertices; 
		const bool bIsValidSimSelection = SelectionGroupName == ClothCollectionGroup::SimFaces || SelectionGroupName == ClothCollectionGroup::SimVertices2D || SelectionGroupName == ClothCollectionGroup::SimVertices3D;

		if (!bIsValidRenderSelection && !bIsValidSimSelection)
		{
			return false;
		}

		// Get the selection as a vertex set.
		TSet<int32> TransferSet;
		const FName DesiredTransferGroup = bIsValidRenderSelection ? ClothCollectionGroup::RenderVertices :
			(SimTransferType == EChaosClothAssetWeightMapTransferType::Use2DSimMesh ? ClothCollectionGroup::SimVertices2D : ClothCollectionGroup::SimVertices3D);
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!FClothGeometryTools::ConvertSelectionToNewGroupType(TransferClothCollection, InInputName, DesiredTransferGroup, bIsSecondarySelection, TransferSet))
		{
			return false;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Convert to weights that are 0 on unselected vertices and 1 on selected vertices
		TArray<float> TransferWeights;
		TransferWeights.SetNumZeroed(TransferClothCollection->NumElements(DesiredTransferGroup));
		for (const int32 SetIndex : TransferSet)
		{
			if (TransferWeights.IsValidIndex(SetIndex))
			{
				TransferWeights[SetIndex] = 1.f;
			}
		}

		// Transfer weights
		TArray<float> RemappedWeights;
		RemappedWeights.SetNumZeroed(ClothCollection->NumElements(DesiredTransferGroup));
		if (bIsValidRenderSelection)
		{
			FClothGeometryTools::TransferWeightMap(
				TransferClothFacade.GetRenderPosition(),
				TransferClothFacade.GetRenderIndices(),
				TransferWeights,
				ClothFacade.GetRenderPosition(),
				ClothFacade.GetRenderNormal(),
				ClothFacade.GetRenderIndices(),
				TArrayView<float>(RemappedWeights));

			OutSelection.Reset();

			if (SelectionGroupName == ClothCollectionGroup::RenderFaces)
			{
				ConvertWeightMapToFaceSelection(RemappedWeights, TransferSelectionThreshold, ClothFacade.GetRenderIndices(), OutSelection);
			}
			else
			{
				check(SelectionGroupName == ClothCollectionGroup::RenderVertices);
				ConvertWeightMapToVertexSelection(RemappedWeights, TransferSelectionThreshold, OutSelection);
			}
		}
		else if (SimTransferType == EChaosClothAssetWeightMapTransferType::Use2DSimMesh)
		{
			TArray<FVector3f> TransferSimPositions2DAs3D;
			TConstArrayView<FVector2f> TransferPositions2D = TransferClothFacade.GetSimPosition2D();
			TransferSimPositions2DAs3D.SetNumUninitialized(TransferPositions2D.Num());
			for (int32 Index = 0; Index < TransferPositions2D.Num(); ++Index)
			{
				TransferSimPositions2DAs3D[Index] = FVector3f(TransferPositions2D[Index], 0.f);
			}

			TConstArrayView<FVector2f> Positions2D = ClothFacade.GetSimPosition2D();
			TArray<FVector3f> Positions2DAs3D;
			TArray<FVector3f> NormalsZAxis;
			Positions2DAs3D.SetNumUninitialized(Positions2D.Num());
			NormalsZAxis.Init(FVector3f::ZAxisVector, Positions2D.Num());
			for (int32 Index = 0; Index < Positions2D.Num(); ++Index)
			{
				Positions2DAs3D[Index] = FVector3f(Positions2D[Index], 0.f);
			}

			FClothGeometryTools::TransferWeightMap(
				TConstArrayView<FVector3f>(TransferSimPositions2DAs3D),
				TransferClothFacade.GetSimIndices2D(),
				TransferWeights,
				TConstArrayView<FVector3f>(Positions2DAs3D),
				TConstArrayView<FVector3f>(NormalsZAxis),
				ClothFacade.GetSimIndices2D(),
				TArrayView<float>(RemappedWeights));

			if (SelectionGroupName == ClothCollectionGroup::SimFaces)
			{
				ConvertWeightMapToFaceSelection(RemappedWeights, TransferSelectionThreshold, ClothFacade.GetSimIndices2D(), OutSelection);
			}
			else if (SelectionGroupName == ClothCollectionGroup::SimVertices2D)
			{
				ConvertWeightMapToVertexSelection(RemappedWeights, TransferSelectionThreshold, OutSelection);
			}
			else
			{
				check(SelectionGroupName == ClothCollectionGroup::SimVertices3D);
				TSet<int32> Selection2D;
				ConvertWeightMapToVertexSelection(RemappedWeights, TransferSelectionThreshold, Selection2D);
				const TConstArrayView<int32> SimVertex3DLookup = ClothFacade.GetSimVertex3DLookup();
				for (const int32 Vertex2D : Selection2D)
				{
					OutSelection.Add(SimVertex3DLookup[Vertex2D]);
				}
			}
		}
		else
		{
			check(SimTransferType == EChaosClothAssetWeightMapTransferType::Use3DSimMesh);
			FClothGeometryTools::TransferWeightMap(
				TransferClothFacade.GetSimPosition3D(),
				TransferClothFacade.GetSimIndices3D(),
				TransferWeights,
				ClothFacade.GetSimPosition3D(),
				ClothFacade.GetSimNormal(),
				ClothFacade.GetSimIndices3D(),
				TArrayView<float>(RemappedWeights));

			if (SelectionGroupName == ClothCollectionGroup::SimFaces)
			{
				ConvertWeightMapToFaceSelection(RemappedWeights, TransferSelectionThreshold, ClothFacade.GetSimIndices3D(), OutSelection);
			}
			else if (SelectionGroupName == ClothCollectionGroup::SimVertices3D)
			{
				ConvertWeightMapToVertexSelection(RemappedWeights, TransferSelectionThreshold, OutSelection);
			}
			else
			{
				check(SelectionGroupName == ClothCollectionGroup::SimVertices2D);
				TSet<int32> Selection3D;
				ConvertWeightMapToVertexSelection(RemappedWeights, TransferSelectionThreshold, Selection3D);
				const TConstArrayView<TArray<int32>> SimVertex2DLookup = ClothFacade.GetSimVertex2DLookup();
				for (const int32 Vertex3D : Selection3D)
				{
					for (const int32 Vertex2D : SimVertex2DLookup[Vertex3D])
					{
						OutSelection.Add(Vertex2D);
					}
				}
			}
		}
		return true;
	}

	void SetIndices(const TSet<int32>& InputSet, const TSet<int32>& FinalSet, EChaosClothAssetSelectionOverrideType OverrideType, TSet<int32>& Indices, TSet<int32>& RemoveIndices)
	{
		if (InputSet.IsEmpty() || OverrideType == EChaosClothAssetSelectionOverrideType::ReplaceAll)
		{
			Indices = FinalSet;
			RemoveIndices.Reset();
			return;
		}

		Indices = FinalSet.Difference(InputSet);
		RemoveIndices = InputSet.Difference(FinalSet);
	}

	void CalculateFinalSet(const TSet<int32>& InputSet, TSet<int32>& FinalSet, EChaosClothAssetSelectionOverrideType OverrideType, const TSet<int32>& Indices, const TSet<int32>& RemoveIndices)
	{
		if (InputSet.IsEmpty() || OverrideType == EChaosClothAssetSelectionOverrideType::ReplaceAll)
		{
			FinalSet = Indices;
			return;
		}

		FinalSet = InputSet;
		FinalSet.Append(Indices);
		if (!RemoveIndices.IsEmpty())
		{
			FinalSet = FinalSet.Difference(RemoveIndices);
		}
	}
}  // namespace UE::Chaos::ClothAsset::Private

// Object encapsulating a change to the Selection Node's values. Used for Undo/Redo.
class FChaosClothAssetSelectionNode_v2::FSelectionNodeChange final : public FToolCommandChange
{
public:
	FSelectionNodeChange(const FChaosClothAssetSelectionNode_v2& Node)
		: NodeGuid(Node.GetGuid())
		, SavedName(Node.OutputName.StringValue)
		, SavedSelectionOverrideType(Node.SelectionOverrideType)
		, SavedGroup(Node.Group)
		, SavedIndices(Node.Indices)
		, SavedRemoveIndices(Node.RemoveIndices)
	{}

private:
	FGuid NodeGuid;
	FString SavedName;
	EChaosClothAssetSelectionOverrideType SavedSelectionOverrideType;
	FChaosClothAssetNodeSelectionGroup SavedGroup;
	TSet<int32> SavedIndices;
	TSet<int32> SavedRemoveIndices;

	virtual FString ToString() const final { return TEXT("ChaosClothAssetSelectionNodeChange"); }

	virtual void Apply(UObject* Object) final { SwapApplyRevert(Object); }
	virtual void Revert(UObject* Object) final { SwapApplyRevert(Object); }

	void SwapApplyRevert(UObject* Object)
	{
		if (UDataflow* const Dataflow = Cast<UDataflow>(Object))
		{
			if (const TSharedPtr<FDataflowNode> BaseNode = Dataflow->GetDataflow()->FindBaseNode(NodeGuid))
			{
				if (FChaosClothAssetSelectionNode_v2* const Node = BaseNode->AsType<FChaosClothAssetSelectionNode_v2>())
				{
					Swap(Node->OutputName.StringValue, SavedName);
					Swap(Node->SelectionOverrideType, SavedSelectionOverrideType);
					Swap(Node->Group, SavedGroup);
					Swap(Node->Indices, SavedIndices);
					Swap(Node->RemoveIndices, SavedRemoveIndices);
					Node->Invalidate();
				}
			}
		}
	}
};

FChaosClothAssetSelectionNode_v2::FChaosClothAssetSelectionNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
	, Import(FDataflowFunctionProperty::FDelegate::CreateRaw(this, &FChaosClothAssetSelectionNode_v2::OnImport))
	, ImportSecondary(FDataflowFunctionProperty::FDelegate::CreateRaw(this, &FChaosClothAssetSelectionNode_v2::OnImportSecondary))
	, Transfer(FDataflowFunctionProperty::FDelegate::CreateRaw(this, &FChaosClothAssetSelectionNode_v2::OnTransfer))
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&InputName.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&TransferCollection)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&OutputName.StringValue, (FString*)nullptr, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableOStringValue, StringValue));
}

void FChaosClothAssetSelectionNode_v2::OnImport(UE::Dataflow::FContext& Context)
{
	using namespace UE::Chaos::ClothAsset;

	FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
	const FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);

	const FName InInputName = GetInputName(Context);
	if (const TSet<int32>* const SelectionSet = SelectionFacade.IsValid() ? SelectionFacade.FindSelectionSet(InInputName) : nullptr)
	{
		Indices = *SelectionSet;
		RemoveIndices.Empty();
		Group.Name = SelectionFacade.GetSelectionGroup(InInputName).ToString();
		SelectionOverrideType = EChaosClothAssetSelectionOverrideType::ReplaceAll;
	}
	else
	{
		Context.Warning(
			FText::Format(LOCTEXT("NoMatchingSelectionFoundDetails", "No matching selection with the name \"{0}\" has been found to import."),
				FText::FromName(InInputName))
			, this
		);
	}
}

void FChaosClothAssetSelectionNode_v2::OnImportSecondary(UE::Dataflow::FContext& Context)
{
	using namespace UE::Chaos::ClothAsset;

	FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
	const FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);

	const FName InInputName = GetInputName(Context);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (const TSet<int32>* const SelectionSet = SelectionFacade.IsValid() ? SelectionFacade.FindSelectionSecondarySet(InInputName) : nullptr)
	{
		Indices = *SelectionSet;
		RemoveIndices.Empty();
		Group.Name = SelectionFacade.GetSelectionSecondaryGroup(InInputName).ToString();
		SelectionOverrideType = EChaosClothAssetSelectionOverrideType::ReplaceAll;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	else
	{
		Context.Warning(
			FText::Format(LOCTEXT("NoMatchingSecondarySelectionFoundDetails", "No matching secondary selection with the name \"{0}\" has been found to import."),
				FText::FromName(InInputName))
			, this
		);
	}
}

void FChaosClothAssetSelectionNode_v2::OnTransfer(UE::Dataflow::FContext& Context)
{
	using namespace UE::Chaos::ClothAsset;

	// Transfer selection if the transfer collection input has changed and is valid
	FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
	FCollectionClothConstFacade ClothFacade(ClothCollection);
	if (ClothFacade.HasValidSimulationData())  // Can only act on the collection if it is a valid cloth collection
	{
		FManagedArrayCollection InTransferCollection = GetValue<FManagedArrayCollection>(Context, &TransferCollection);
		const TSharedRef<const FManagedArrayCollection> TransferClothCollection = MakeShared<const FManagedArrayCollection>(MoveTemp(InTransferCollection));
		FCollectionClothConstFacade TransferClothFacade(TransferClothCollection);
		FCollectionClothSelectionConstFacade TransferSelectionFacade(TransferClothCollection);

		const FName InInputName = GetInputName(Context);
		const FName SelectionGroupName(*Group.Name);

		TSet<int32> PrimaryFinalSelection;
		if (Private::TransferSelectionSet<false>(TransferClothCollection, ClothCollection, InInputName, SelectionGroupName, SimTransferType, TransferSelectionThreshold, PrimaryFinalSelection))
		{
			TSet<int32> InputSelection;
			FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, InInputName, SelectionGroupName, InputSelection);

			SetIndices(InputSelection, PrimaryFinalSelection);
		}
	}
}

void FChaosClothAssetSelectionNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	auto CopyIntoSelection = [this, &Context, Out](const TSharedRef<FManagedArrayCollection> SelectionCollection,
		const FName& SelectionGroupName,
		const TSet<int32>& SourceIndices,
		TSet<int32>& DestSelectionSet)
		{
			const int32 NumElementsInGroup = SelectionCollection->NumElements(SelectionGroupName);
			bool bFoundAnyInvalidIndex = false;

			DestSelectionSet.Reset();

			for (const int32 Index : SourceIndices)
			{
				if (Index < 0 || Index >= NumElementsInGroup)
				{
					const FText LogErrorMessage = FText::Format(LOCTEXT("SelectionIndexOutOfBoundsDetails", "Selection index {0} not valid for group \"{1}\" with {2} elements"),
						Index,
						FText::FromName(SelectionGroupName),
						NumElementsInGroup);

					// Log all indices, but toast once
					UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "%ls", *LogErrorMessage.ToString());
					bFoundAnyInvalidIndex = true;
				}
				else
				{
					DestSelectionSet.Add(Index);
				}
			}

			if (bFoundAnyInvalidIndex)
			{
				// Toast once
				const FText WarningMessage = FText::Format(LOCTEXT("AnySelectionIndexOutOfBoundsDetails", "Found invalid selection indices for group \"{0}.\" See log for details"),
					FText::FromName(SelectionGroupName));
				Context.Warning(WarningMessage, this, Out);
			}
		};

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Evaluate InputName
		const FName InInputName = GetInputName(Context);
		const FName SelectionName(OutputName.StringValue.IsEmpty() ? InInputName : FName(OutputName.StringValue));

		if (SelectionName == NAME_None || Group.Name.IsEmpty())
		{
			const FManagedArrayCollection& SelectionCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
			SetValue(Context, SelectionCollection, &Collection);
			return;
		}

		const FName SelectionGroupName(*Group.Name);

		FManagedArrayCollection InSelectionCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> SelectionCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InSelectionCollection));

		FCollectionClothSelectionFacade SelectionFacade(SelectionCollection);
		SelectionFacade.DefineSchema();
		check(SelectionFacade.IsValid());

		TSet<int32> InputSelectionSet;
		FClothGeometryTools::ConvertSelectionToNewGroupType(SelectionCollection, InInputName, SelectionGroupName, InputSelectionSet);
		TSet<int32> FinalSet;
		CalculateFinalSet(InputSelectionSet, FinalSet);

		TSet<int32>& SelectionSet = SelectionFacade.FindOrAddSelectionSet(SelectionName, SelectionGroupName);
		CopyIntoSelection(SelectionCollection, SelectionGroupName, FinalSet, SelectionSet);

		SetValue(Context, MoveTemp(*SelectionCollection), &Collection);
	}
	else if (Out->IsA<FString>(&OutputName.StringValue))
	{
		FString InputNameString = GetValue<FString>(Context, &InputName.StringValue);
		FClothDataflowTools::MakeCollectionName(InputNameString);
		SetValue(Context, OutputName.StringValue.IsEmpty() ? InputNameString : OutputName.StringValue, &OutputName.StringValue);
	}
}

FName FChaosClothAssetSelectionNode_v2::GetInputName(UE::Dataflow::FContext& Context) const
{
	FString InputNameString = GetValue<FString>(Context, &InputName.StringValue);
	UE::Chaos::ClothAsset::FClothDataflowTools::MakeCollectionName(InputNameString);
	const FName InInputName(*InputNameString);
	return InInputName != NAME_None ? InInputName : FName(OutputName.StringValue);
}

void FChaosClothAssetSelectionNode_v2::SetIndices(const TSet<int32>& InputSet, const TSet<int32>& FinalSet)
{
	UE::Chaos::ClothAsset::Private::SetIndices(InputSet, FinalSet, SelectionOverrideType, Indices, RemoveIndices);
}

void FChaosClothAssetSelectionNode_v2::CalculateFinalSet(const TSet<int32>& InputSet, TSet<int32>& FinalSet) const
{
	UE::Chaos::ClothAsset::Private::CalculateFinalSet(InputSet, FinalSet, SelectionOverrideType, Indices, RemoveIndices);
}

TUniquePtr<FToolCommandChange> FChaosClothAssetSelectionNode_v2::MakeSelectedNodeChange(const FChaosClothAssetSelectionNode_v2& Node)
{
	return MakeUnique<FChaosClothAssetSelectionNode_v2::FSelectionNodeChange>(Node);
}

FChaosClothAssetSelectionNode::FChaosClothAssetSelectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&InputName.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&TransferCollection)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&Name);
}

void FChaosClothAssetSelectionNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	using namespace UE::Chaos::ClothAsset;

	if (UChaosClothAsset* const ClothAsset = Cast<UChaosClothAsset>(Asset.Get()))
	{
		if (UDataflow* const DataflowAsset = ClothAsset->GetDataflow())
		{
			const TSharedPtr<UE::Dataflow::FGraph, ESPMode::ThreadSafe> Dataflow = DataflowAsset->GetDataflow();
			if (const TSharedPtr<FDataflowNode> BaseNode = Dataflow->FindBaseNode(this->GetGuid()))  // This is basically a safe const_cast
			{
				FChaosClothAssetSelectionNode* const MutableThis = static_cast<FChaosClothAssetSelectionNode*>(BaseNode.Get());
				check(MutableThis == this);

				// Make the name a valid attribute name, and replace the value in the UI
				FClothDataflowTools::MakeCollectionName(MutableThis->Name);

				const FName SelectionGroupName(*Group.Name);
				const FName SelectionSecondaryGroupName(*SecondaryGroup.Name);

				// Transfer selection if the transfer collection input has changed and is valid
				FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
				const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
				FCollectionClothConstFacade ClothFacade(ClothCollection);
				if (ClothFacade.HasValidSimulationData())  // Can only act on the collection if it is a valid cloth collection
				{
					FManagedArrayCollection InTransferCollection = GetValue<FManagedArrayCollection>(Context, &TransferCollection);
					const TSharedRef<const FManagedArrayCollection> TransferClothCollection = MakeShared<const FManagedArrayCollection>(MoveTemp(InTransferCollection));
					FCollectionClothConstFacade TransferClothFacade(TransferClothCollection);
					FCollectionClothSelectionConstFacade TransferSelectionFacade(TransferClothCollection);

					const FName InInputName = GetInputName(Context);
					uint32 InTransferCollectionHash = HashCombineFast(GetTypeHash(InInputName), GetTypeHash(SelectionGroupName));
					InTransferCollectionHash = HashCombineFast(InTransferCollectionHash, GetTypeHash(SelectionSecondaryGroupName));
					InTransferCollectionHash = HashCombineFast(InTransferCollectionHash, (uint32)SimTransferType);
					if (TransferClothFacade.HasValidSimulationData() && TransferSelectionFacade.IsValid() && InInputName != NAME_None && TransferSelectionFacade.HasSelection(InInputName))
					{
						InTransferCollectionHash = HashCombineFast(InTransferCollectionHash, GetTypeHash(TransferSelectionFacade.GetSelectionGroup(InInputName)));
						const TArray<int32> SelectionAsArray = TransferSelectionFacade.GetSelectionSet(InInputName).Array();
						InTransferCollectionHash = GetArrayHash(SelectionAsArray.GetData(), SelectionAsArray.Num(), InTransferCollectionHash);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
						if (TransferSelectionFacade.HasSelectionSecondarySet(InInputName))
						{
							InTransferCollectionHash = HashCombineFast(InTransferCollectionHash, GetTypeHash(TransferSelectionFacade.GetSelectionSecondaryGroup(InInputName)));
							const TArray<int32> SecondarySelectionAsArray = TransferSelectionFacade.GetSelectionSecondarySet(InInputName).Array();
							InTransferCollectionHash = GetArrayHash(SecondarySelectionAsArray.GetData(), SecondarySelectionAsArray.Num(), InTransferCollectionHash);
						}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
					else
					{
						InTransferCollectionHash = 0;
					}

					if (TransferCollectionHash != InTransferCollectionHash)
					{
						MutableThis->TransferCollectionHash = InTransferCollectionHash;

						if(TransferCollectionHash) //-V1051
						{
							TSet<int32> PrimaryFinalSelection;
							if (Private::TransferSelectionSet<false>(TransferClothCollection, ClothCollection, InInputName, SelectionGroupName, SimTransferType, TransferSelectionThreshold, PrimaryFinalSelection))
							{
								TSet<int32> InputSelection;
								FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, InInputName, SelectionGroupName, InputSelection);

								MutableThis->SetIndices(InputSelection, PrimaryFinalSelection);
							}

							TSet<int32> SecondaryFinalSelection;
							if (Private::TransferSelectionSet<true>(TransferClothCollection, ClothCollection, InInputName, SelectionSecondaryGroupName, SimTransferType, TransferSelectionThreshold, SecondaryFinalSelection))
							{
								TSet<int32> InputSelection;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
								FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, InInputName, SelectionGroupName, true, InputSelection);

								MutableThis->SetSecondaryIndices(InputSelection, SecondaryFinalSelection);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
							}
						}
					}
				}
			}
		}
	}
}

void FChaosClothAssetSelectionNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	auto CopyIntoSelection = [this, &Context, Out](const TSharedRef<FManagedArrayCollection> SelectionCollection, 
		const FName& SelectionGroupName, 
		const TSet<int32>& SourceIndices,
		TSet<int32>& DestSelectionSet)
	{
		const int32 NumElementsInGroup = SelectionCollection->NumElements(SelectionGroupName);
		bool bFoundAnyInvalidIndex = false;

		DestSelectionSet.Reset();

		for (const int32 Index : SourceIndices)
		{
			if (Index < 0 || Index >= NumElementsInGroup)
			{
				const FText LogErrorMessage = FText::Format(LOCTEXT("SelectionIndexOutOfBoundsDetails", "Selection index {0} not valid for group \"{1}\" with {2} elements"),
					Index,
					FText::FromName(SelectionGroupName),
					NumElementsInGroup);

				// Log all indices, but toast once
				UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "%ls", *LogErrorMessage.ToString());
				bFoundAnyInvalidIndex = true;
			}
			else
			{
				DestSelectionSet.Add(Index);
			}
		}

		if (bFoundAnyInvalidIndex)
		{
			// Toast once
			const FText WarningMessage = FText::Format(LOCTEXT("AnySelectionIndexOutOfBoundsDetails", "Found invalid selection indices for group \"{0}.\" See log for details"),
				FText::FromName(SelectionGroupName));
			Context.Warning(WarningMessage, this, Out);
		}
	};

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Evaluate InputName
		const FName InInputName = GetInputName(Context);
		const FName SelectionName(Name.IsEmpty() ? InInputName : FName(Name));

		if (SelectionName == NAME_None || Group.Name.IsEmpty())
		{
			const FManagedArrayCollection& SelectionCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
			SetValue(Context, SelectionCollection, &Collection);
			return;
		}

		const FName SelectionGroupName(*Group.Name);

		FManagedArrayCollection InSelectionCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> SelectionCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InSelectionCollection));

		FCollectionClothSelectionFacade SelectionFacade(SelectionCollection);
		SelectionFacade.DefineSchema();
		check(SelectionFacade.IsValid());

		TSet<int32> InputSelectionSet;
		FClothGeometryTools::ConvertSelectionToNewGroupType(SelectionCollection, InInputName, SelectionGroupName, InputSelectionSet);
		TSet<int32> FinalSet;
		CalculateFinalSet(InputSelectionSet, FinalSet);

		TSet<int32>& SelectionSet = SelectionFacade.FindOrAddSelectionSet(SelectionName, SelectionGroupName);
		CopyIntoSelection(SelectionCollection, SelectionGroupName, FinalSet, SelectionSet);

		if (!SecondaryGroup.Name.IsEmpty() && !SecondaryIndices.IsEmpty())
		{
			const FName SecondarySelectionName(Name.IsEmpty() ? InInputName : FName(Name));
			const FName SecondarySelectionGroupName = (*SecondaryGroup.Name);

			InputSelectionSet.Reset();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FClothGeometryTools::ConvertSelectionToNewGroupType(SelectionCollection, InInputName, SelectionGroupName, true, InputSelectionSet);
			FinalSet.Reset();
			CalculateFinalSecondarySet(InputSelectionSet, FinalSet);
			TSet<int32>& SecondarySelectionSet = SelectionFacade.FindOrAddSelectionSecondarySet(SecondarySelectionName, SecondarySelectionGroupName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			CopyIntoSelection(SelectionCollection, SecondarySelectionGroupName, FinalSet, SecondarySelectionSet);
		}

		SetValue(Context, MoveTemp(*SelectionCollection), &Collection);
	}
	else if (Out->IsA<FString>(&Name))
	{
		FString InputNameString = GetValue<FString>(Context, &InputName.StringValue);
		FClothDataflowTools::MakeCollectionName(InputNameString);
		SetValue(Context, Name.IsEmpty() ? InputNameString : Name, &Name);
	}
}

void FChaosClothAssetSelectionNode::Serialize(FArchive& Ar)
{
	using namespace UE::Chaos::ClothAsset;

	// This is just for convenience and can be removed post 5.4 once the plugin loses its experimental status
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Ar.IsLoading() && Type_DEPRECATED != EChaosClothAssetSelectionType::Deprecated)
	{
		switch (Type_DEPRECATED)
		{
		case EChaosClothAssetSelectionType::SimVertex2D: Group.Name = ClothCollectionGroup::SimVertices2D.ToString(); break;
		case EChaosClothAssetSelectionType::SimVertex3D: Group.Name = ClothCollectionGroup::SimVertices3D.ToString(); break;
		case EChaosClothAssetSelectionType::RenderVertex: Group.Name = ClothCollectionGroup::RenderVertices.ToString(); break;
		case EChaosClothAssetSelectionType::SimFace: Group.Name = ClothCollectionGroup::SimFaces.ToString(); break;
		case EChaosClothAssetSelectionType::RenderFace: Group.Name = ClothCollectionGroup::RenderFaces.ToString(); break;
		default: checkNoEntry();
		}
		Type_DEPRECATED = EChaosClothAssetSelectionType::Deprecated;  // This is only for clarity since the Type property won't be saved from now on

		FClothDataflowTools::LogAndToastWarning(*this,
			LOCTEXT("DeprecatedSelectionType", "Outdated Dataflow asset."),
			LOCTEXT("DeprecatedSelectionDetails", "This node is out of date and contains deprecated data. The asset needs to be re-saved before it stops working at the next version update."));
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

FName FChaosClothAssetSelectionNode::GetInputName(UE::Dataflow::FContext& Context) const
{
	FString InputNameString = GetValue<FString>(Context, &InputName.StringValue);
	UE::Chaos::ClothAsset::FClothDataflowTools::MakeCollectionName(InputNameString);
	const FName InInputName(*InputNameString);
	return InInputName != NAME_None ? InInputName : FName(Name);
}

void FChaosClothAssetSelectionNode::SetIndices(const TSet<int32>& InputSet, const TSet<int32>& FinalSet)
{
	UE::Chaos::ClothAsset::Private::SetIndices(InputSet, FinalSet, SelectionOverrideType, Indices, RemoveIndices);
}

void FChaosClothAssetSelectionNode::SetSecondaryIndices(const TSet<int32>& InputSet, const TSet<int32>& FinalSet)
{
	UE::Chaos::ClothAsset::Private::SetIndices(InputSet, FinalSet, SelectionOverrideType, SecondaryIndices, RemoveSecondaryIndices);
}

void FChaosClothAssetSelectionNode::CalculateFinalSet(const TSet<int32>& InputSet, TSet<int32>& FinalSet) const
{
	UE::Chaos::ClothAsset::Private::CalculateFinalSet(InputSet, FinalSet, SelectionOverrideType, Indices, RemoveIndices);
}

void FChaosClothAssetSelectionNode::CalculateFinalSecondarySet(const TSet<int32>& InputSet, TSet<int32>& FinalSet) const
{
	UE::Chaos::ClothAsset::Private::CalculateFinalSet(InputSet, FinalSet, SelectionOverrideType, SecondaryIndices, RemoveSecondaryIndices);
}

#undef LOCTEXT_NAMESPACE
