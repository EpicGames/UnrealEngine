// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SelectionToIntMapNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SelectionToIntMapNode)
#define LOCTEXT_NAMESPACE "FChaosClothAssetSelectionToIntMapNode"

FChaosClothAssetSelectionToIntMapNode::FChaosClothAssetSelectionToIntMapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterInputConnection(&SelectionName.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterInputConnection(&IntMapName.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIOStringValue, StringValue));
	RegisterOutputConnection(&IntMapName.StringValue, &IntMapName.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIOStringValue, StringValue));
}

void FChaosClothAssetSelectionToIntMapNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothFacade ClothFacade(ClothCollection);

		const FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);
		const FName InSelectionName(*GetValue<FString>(Context, &SelectionName.StringValue));
		const FString& InIntMapNameString = GetValue<FString>(Context, &IntMapName.StringValue);
		const FName InIntMapName = InIntMapNameString.IsEmpty() ? InSelectionName : FName(*InIntMapNameString);
		if (SelectionFacade.IsValid() && ClothFacade.IsValid() && InSelectionName != NAME_None && InIntMapName != NAME_None)
		{
			if (const TSet<int32>* const SelectionSet = SelectionFacade.FindSelectionSet(InSelectionName))
			{
				const FName& SelectionGroup = SelectionFacade.GetSelectionGroup(InSelectionName);

				bool bValidMap = true;
				bool bIsNewMap = false;
				if (!ClothFacade.HasUserDefinedAttribute<int32>(InIntMapName, SelectionGroup))
				{
					if (ClothFacade.AddUserDefinedAttribute<int32>(InIntMapName, SelectionGroup))
					{
						bIsNewMap = true;
					}
					else
					{
						bValidMap = false;
						Context.Warning(
							FText::Format(LOCTEXT("IntMapNameInvalidDetails", "Could not create (or find) int map with name \"{0}\" in group \"{1}\""),
								FText::FromName(InIntMapName),
								FText::FromName(SelectionGroup))
							, this, Out
						);
					}
				}
				if (bValidMap)
				{
					TArrayView<int32> IntMap = ClothFacade.GetUserDefinedAttribute<int32>(InIntMapName, SelectionGroup);

					if (bIsNewMap || !bKeepExistingUnselectedValues)
					{
						for (int32& MapValue : IntMap)
						{
							MapValue = UnselectedValue;
						}
					}
					for (const int32 Index : *SelectionSet)
					{
						if (IntMap.IsValidIndex(Index))
						{
							IntMap[Index] = SelectedValue;
						}
					}
				}
			}
			else
			{
				Context.Warning(
					FText::Format(LOCTEXT("SelectionNameNotFoundDetails", "Selection \"{0}\" was not found in the collection."),
						FText::FromName(InSelectionName))
					, this, Out
				);
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
	else if (Out->IsA<FString>(&IntMapName.StringValue))
	{
		const FString& InIntMapNameString = GetValue<FString>(Context, &IntMapName.StringValue);
		SetValue(Context, InIntMapNameString.IsEmpty() ? GetValue<FString>(Context, &SelectionName.StringValue) : InIntMapNameString, &IntMapName.StringValue);
	}
}


#undef LOCTEXT_NAMESPACE
