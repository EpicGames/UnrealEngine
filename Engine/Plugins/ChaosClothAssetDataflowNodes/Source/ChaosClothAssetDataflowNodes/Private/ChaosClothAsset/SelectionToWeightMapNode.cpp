// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SelectionToWeightMapNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SelectionToWeightMapNode)
#define LOCTEXT_NAMESPACE "FChaosClothAssetSelectionToWeightMapNode"

FChaosClothAssetSelectionToWeightMapNode::FChaosClothAssetSelectionToWeightMapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterInputConnection(&SelectionName);
	RegisterOutputConnection(&WeightMapName);
}

void FChaosClothAssetSelectionToWeightMapNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothFacade ClothFacade(ClothCollection);

		const FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);
		const FName InSelectionName(*GetValue<FString>(Context, &SelectionName));

		if (SelectionFacade.IsValid() && ClothFacade.IsValid() && InSelectionName != NAME_None)
		{
			if (const TSet<int32>* const SelectionSet = SelectionFacade.FindSelectionSet(InSelectionName))
			{
				const FName& SelectionGroup = SelectionFacade.GetSelectionGroup(InSelectionName);

				if (SelectionGroup == ClothCollectionGroup::SimVertices2D ||
					SelectionGroup == ClothCollectionGroup::SimVertices3D || 
					SelectionGroup == ClothCollectionGroup::SimFaces)
				{
					const FName InMapName(WeightMapName.IsEmpty() ? InSelectionName : FName(WeightMapName));
					ClothFacade.AddWeightMap(InMapName);
					TArrayView<float> OutClothWeights = ClothFacade.GetWeightMap(InMapName);
					if (OutClothWeights.Num() == ClothFacade.GetNumSimVertices3D())
					{
						if (SelectionGroup == ClothCollectionGroup::SimVertices3D)
						{
							for (int32 VertexIndex = 0; VertexIndex < OutClothWeights.Num(); ++VertexIndex)
							{
								OutClothWeights[VertexIndex] = SelectionSet->Contains(VertexIndex) ? SelectedValue : UnselectedValue;
							}
						}
						else if (SelectionGroup == ClothCollectionGroup::SimVertices2D)
						{
							// We are given a selection over the set of 2D vertices, but weight maps only exist for 3D vertices, so
							// we need a bit of translation
							const TConstArrayView<TArray<int32>> Vertex3DTo2D = ClothFacade.GetSimVertex2DLookup();

							for (int32 Vertex3DIndex = 0; Vertex3DIndex < OutClothWeights.Num(); ++Vertex3DIndex)
							{
								// If any corresponding 2D vertex is selected, set the 3D weight map value to one, otherwise it gets zero
								OutClothWeights[Vertex3DIndex] = UnselectedValue;
								for (const int32 Vertex2DIndex : Vertex3DTo2D[Vertex3DIndex])
								{
									if (SelectionSet->Contains(Vertex2DIndex))
									{
										OutClothWeights[Vertex3DIndex] = SelectedValue;
										break;
									}
								}
							}
						}
						else
						{
							check(SelectionGroup == ClothCollectionGroup::SimFaces);

							// Fill with unselected value and then set all vertices in the selected faces to the selected value.
							for (int32 Vertex3DIndex = 0; Vertex3DIndex < OutClothWeights.Num(); ++Vertex3DIndex)
							{
								OutClothWeights[Vertex3DIndex] = UnselectedValue;
							}
							const TConstArrayView<FIntVector3> SimIndices3D = ClothFacade.GetSimIndices3D();
							for (const int32 FaceIndex : *SelectionSet)
							{
								if (SimIndices3D.IsValidIndex(FaceIndex))
								{
									// Indices in SimIndices3D are assumed to be valid to index OutClothWeights
									OutClothWeights[SimIndices3D[FaceIndex][0]] = SelectedValue;
									OutClothWeights[SimIndices3D[FaceIndex][1]] = SelectedValue;
									OutClothWeights[SimIndices3D[FaceIndex][2]] = SelectedValue;
								}
							}
						}
					}
					else
					{
						check(OutClothWeights.Num() == 0);
						Context.Warning(
							FText::Format(LOCTEXT("InvalidWeightMapNameDetails", "Could not create a weight map with name \"{0}\" (reserved name? wrong type?)."),
								FText::FromName(InMapName))
							, this, Out
						);
					}
				}
				else
				{
					Context.Warning(
						FText::Format(LOCTEXT("SelectionTypeNotCorrectDetails", "Selection \"{0}\" does not have its index dependency group set to \"{1}\", \"{2}\", or \"{3}\"."),
							FText::FromName(InSelectionName),
							FText::FromName(ClothCollectionGroup::SimVertices2D),
							FText::FromName(ClothCollectionGroup::SimVertices3D),
							FText::FromName(ClothCollectionGroup::SimFaces))
						, this, Out
					);
				}
			}
			else
			{
				Context.Warning(
					FText::Format(LOCTEXT("SelectionNameNotFoundDetails", "Selection \"{0}\" was not found in the collection."),
						FText::FromName(InSelectionName))
					, this, Out
				);
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
	else if (Out->IsA<FString>(&WeightMapName))
	{
		SetValue(Context, WeightMapName.IsEmpty() ? GetValue<FString>(Context, &SelectionName) : WeightMapName, &WeightMapName);
	}
}


#undef LOCTEXT_NAMESPACE
