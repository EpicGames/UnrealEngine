// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SetPhysicsAssetNode.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "PhysicsEngine/PhysicsAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SetPhysicsAssetNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetSetPhysicsAssetNode"

FChaosClothAssetSetPhysicsAssetNode::FChaosClothAssetSetPhysicsAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&PhysicsAsset);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&PhysicsAsset, &PhysicsAsset)
		.SetCanHidePin(true).SetPinIsHidden(true); // we only need the output for the construction view rendering
}

void FChaosClothAssetSetPhysicsAssetNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		if (FCollectionClothFacade(ClothCollection).IsValid())  // Can only act on the collection if it is a valid cloth collection
		{
			FCollectionClothFacade CollectionClothFacade(ClothCollection);
			TObjectPtr<const UPhysicsAsset> InPhysicsAsset = GetValue(Context, &PhysicsAsset);
			CollectionClothFacade.SetPhysicsAssetSoftObjectPathName(InPhysicsAsset ? InPhysicsAsset.GetPathName() : FString());
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
	else if (Out->IsA(&PhysicsAsset))
	{
		SafeForwardInput(Context, &PhysicsAsset, &PhysicsAsset);
	}
}

#undef LOCTEXT_NAMESPACE
