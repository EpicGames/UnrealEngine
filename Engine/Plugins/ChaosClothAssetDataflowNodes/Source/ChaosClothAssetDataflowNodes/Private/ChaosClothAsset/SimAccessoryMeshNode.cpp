// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimAccessoryMeshNode.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#if WITH_EDITOR
#include "ChaosClothAsset/ClothDataflowViewModes.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimAccessoryMeshNode)


namespace UE::Chaos::ClothAsset::Private
{
	struct FDebugDrawAccessoryMesh : public IDataflowDebugDrawInterface::IDebugDrawMesh
	{
		virtual int32 GetMaxVertexIndex() const override
		{
			return Points.Num();
		}

		virtual bool IsValidVertex(int32 VertexIndex) const override
		{
			return Points.IsValidIndex(VertexIndex);
		}

		virtual FVector GetVertexPosition(int32 VertexIndex) const override
		{
			return FVector(Points[VertexIndex]);
		}

		virtual FVector GetVertexNormal(int32 VertexIndex) const override
		{
			return FVector(Normals[VertexIndex]);
		}

		virtual int32 GetMaxTriangleIndex() const override
		{
			return Elements.Num();
		}

		virtual bool IsValidTriangle(int32 TriangleIndex) const override
		{
			return Elements.IsValidIndex(TriangleIndex);
		}

		virtual FIntVector3 GetTriangle(int32 TriangleIndex) const override
		{
			return Elements[TriangleIndex];
		}

		TConstArrayView<FIntVector3> Elements;
		TConstArrayView<FVector3f> Points;
		TConstArrayView<FVector3f> Normals;
	};
}

FChaosClothAssetSimAccessoryMeshNode::FChaosClothAssetSimAccessoryMeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&SimAccessoryMeshCollection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&AccessoryMeshName);
}

void FChaosClothAssetSimAccessoryMeshNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection) || Out->IsA<FString>(&AccessoryMeshName))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothFacade ClothFacade(ClothCollection);
		FString OutAccessoryMeshName = AccessoryMeshName;
		if (ClothFacade.IsValid())
		{
			FManagedArrayCollection InAccessoryCollection = GetValue<FManagedArrayCollection>(Context, &SimAccessoryMeshCollection);  // Can't use a const reference here sadly since the facade needs a SharedRef to be created
			const TSharedRef<const FManagedArrayCollection> AccessoryClothCollection = MakeShared<const FManagedArrayCollection>(MoveTemp(InAccessoryCollection));
			FCollectionClothConstFacade AccessoryClothFacade(AccessoryClothCollection);

			if (AccessoryClothFacade.IsValid())
			{
				FText WarningText;
				const int32 AccessoryMeshIndex = FClothEngineTools::CopySimMeshToSimAccessoryMesh(FName(OutAccessoryMeshName), ClothFacade, AccessoryClothFacade, bUseSimImportVertexID, &WarningText);
				if (ClothFacade.GetSimAccessoryMeshName().IsValidIndex(AccessoryMeshIndex))
				{
					OutAccessoryMeshName = ClothFacade.GetSimAccessoryMeshName()[AccessoryMeshIndex].ToString();
				}
				else
				{
					Context.Warning(WarningText.ToString());
				}
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
		SetValue(Context, MoveTemp(OutAccessoryMeshName), &AccessoryMeshName);
	}
}

#if WITH_EDITOR
bool FChaosClothAssetSimAccessoryMeshNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name || ViewModeName == UE::Chaos::ClothAsset::FCloth3DSimViewMode::Name;
}

void FChaosClothAssetSimAccessoryMeshNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	if (DebugDrawParameters.bNodeIsSelected || DebugDrawParameters.bNodeIsPinned)
	{
		using namespace UE::Chaos::ClothAsset;
		const FName OutAccessoryMeshName = FName(GetOutputValue(Context, &AccessoryMeshName, AccessoryMeshName));
		FManagedArrayCollection OutCollection = GetOutputValue<FManagedArrayCollection>(Context, &Collection, Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(OutCollection));
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		const int32 AccessoryMeshIndex = ClothFacade.FindSimAccessoryMeshIndexByName(OutAccessoryMeshName);
		if (AccessoryMeshIndex >= 0 && AccessoryMeshIndex < ClothFacade.GetNumSimAccessoryMeshes())
		{			
			FCollectionClothSimAccessoryMeshConstFacade SimAccessoryMeshFacade = ClothFacade.GetSimAccessoryMesh(AccessoryMeshIndex);

			UE::Chaos::ClothAsset::Private::FDebugDrawAccessoryMesh DrawAccessoryMesh;
			DrawAccessoryMesh.Elements = ClothFacade.GetSimIndices3D();
			DrawAccessoryMesh.Points = SimAccessoryMeshFacade.GetSimAccessoryMeshPosition3D();
			DrawAccessoryMesh.Normals = SimAccessoryMeshFacade.GetSimAccessoryMeshNormal();

			DataflowRenderingInterface.DrawMesh(DrawAccessoryMesh);
		}
	}
}
#endif
