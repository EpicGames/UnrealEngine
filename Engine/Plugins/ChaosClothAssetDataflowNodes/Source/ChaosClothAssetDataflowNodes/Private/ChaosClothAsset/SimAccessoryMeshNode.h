// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "SimAccessoryMeshNode.generated.h"


/** Add a SimAccessoryMesh by converting a cloth collection into an accessory mesh and attaching it to an existing cloth collection. Any unmatched vertices will use the existing cloth collection's sim mesh data to populate the accessory mesh data */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimAccessoryMeshNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimAccessoryMeshNode, "SimAccessoryMeshNode", "Cloth", "Cloth Sim Accessory Mesh")

public:

	FChaosClothAssetSimAccessoryMeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
#if WITH_EDITOR
	virtual bool CanDebugDraw() const override
	{
		return true;
	}
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
	//~ End FDataflowNode interface

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(Meta = (DataflowInput))
	FManagedArrayCollection SimAccessoryMeshCollection;

	/** Name of the new accessory mesh */
	UPROPERTY(EditAnywhere, Category = "Sim Accessory Mesh", Meta = (DataflowOutput))
	FString AccessoryMeshName = TEXT("AccessoryMesh");

	/** Use SimImportVertexID (e.g., imported vertex ID from an FBX->SKM->ClothCollection conversion) to match vertices*/
	UPROPERTY(EditAnywhere, Category = "Sim Accessory Mesh")
	bool bUseSimImportVertexID = true;
};
