// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationAerodynamicsConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/SimulationBaseConfigNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationAerodynamicsConfigNode)

FChaosClothAssetSimulationAerodynamicsConfigNode::FChaosClothAssetSimulationAerodynamicsConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&Drag.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&OuterDrag.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&Lift.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&OuterLift.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationAerodynamicsConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyBool(this, &bUsePointBasedWindModel);
	PropertyHelper.SetProperty(this, &FluidDensity);
	PropertyHelper.SetPropertyEnum(this, &WindVelocitySpace);
	PropertyHelper.SetProperty(this, &WindVelocity);
	PropertyHelper.SetProperty(this, &TurbulenceRatio);

	// Drag/Lift weighted values and water-body interaction are only meaningful for the accurate
	// aerodynamic model. The legacy point-based wind model uses scalar drag/lift internally and
	// does not interact with water bodies, so skip emitting these properties when it is enabled.
	if (!bUsePointBasedWindModel)
	{
		PropertyHelper.SetSolverPropertyWeighted(FName(TEXT("Drag")), Drag, [](
					const UE::Chaos::ClothAsset::FCollectionClothFacade& ClothFacade)-> float
		{
			return ClothFacade.GetSolverAirDamping();
		},{});

		if (bEnableOuterDrag)
		{
			PropertyHelper.SetSolverPropertyWeighted(FName(TEXT("OuterDrag")), OuterDrag, [](
				const UE::Chaos::ClothAsset::FCollectionClothFacade& ClothFacade)-> float
				{
					return ClothFacade.GetSolverAirDamping();
				}, {});
		}

		PropertyHelper.SetSolverPropertyWeighted(FName(TEXT("Lift")), Lift, [](
					const UE::Chaos::ClothAsset::FCollectionClothFacade& ClothFacade)-> float
		{
			return ClothFacade.GetSolverAirDamping();
		},{});

		if (bEnableOuterLift)
		{
			PropertyHelper.SetSolverPropertyWeighted(FName(TEXT("OuterLift")), OuterLift, [](
				const UE::Chaos::ClothAsset::FCollectionClothFacade& ClothFacade)-> float
				{
					return ClothFacade.GetSolverAirDamping();
				}, {});
		}

		if (bUseWithWaterBodies)
		{
			PropertyHelper.SetProperty(this, &ClothDensityInWater);
			PropertyHelper.SetProperty(this, &WaterDensity);
			PropertyHelper.SetProperty(this, &WaterTurbulenceRatio);
		}
	}
}
