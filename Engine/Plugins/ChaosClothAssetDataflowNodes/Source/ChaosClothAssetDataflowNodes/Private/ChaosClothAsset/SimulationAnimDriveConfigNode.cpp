// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationAnimDriveConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationAnimDriveConfigNode)

FChaosClothAssetSimulationAnimDriveConfigNode::FChaosClothAssetSimulationAnimDriveConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&AnimDriveStiffness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&AnimDriveDamping.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationAnimDriveConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyWeighted(this, &AnimDriveStiffness);
	PropertyHelper.SetPropertyWeighted(this, &AnimDriveDamping);
}
