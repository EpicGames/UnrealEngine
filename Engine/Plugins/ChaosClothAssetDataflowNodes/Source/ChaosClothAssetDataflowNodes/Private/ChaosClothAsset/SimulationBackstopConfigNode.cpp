// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationBackstopConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationBackstopConfigNode)

FChaosClothAssetSimulationBackstopConfigNode::FChaosClothAssetSimulationBackstopConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&BackstopDistance.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BackstopRadius.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BackstopMeshName.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationBackstopConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyWeighted(this, &BackstopDistance);
	PropertyHelper.SetPropertyWeighted(this, &BackstopRadius);
	PropertyHelper.SetPropertyString(this, &BackstopMeshName, {}, ECollectionPropertyFlags::None);
}
