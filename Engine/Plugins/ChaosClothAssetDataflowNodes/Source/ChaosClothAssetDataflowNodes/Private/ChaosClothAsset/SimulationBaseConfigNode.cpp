// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationBaseConfigNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetSimulationBaseConfigNode"

namespace UE::Chaos::ClothAsset::Private
{
	static void LogAndToastDuplicateProperty(const FDataflowNode& DataflowNode, const FName& PropertyName)
	{
		using namespace UE::Chaos::ClothAsset;

		static const FText Headline = LOCTEXT("DuplicatePropertyHeadline", "Duplicate property.");

		const FText Details = FText::Format(
			LOCTEXT(
				"DuplicatePropertyDetails",
				"Cloth collection property '{0}' was already set in an upstream node.\n"
				"Its values have now been overridden."),
			FText::FromName(PropertyName));

		FClothDataflowTools::LogAndToastWarning(DataflowNode, Headline, Details);
	}

	static void LogAndToastSimilarProperty(const FDataflowNode& DataflowNode, const FName& PropertyName, const FName& SimilarPropertyName)
	{
		using namespace UE::Chaos::ClothAsset;

		static const FText Headline = LOCTEXT("SimilarPropertyHeadline", "Similar property.");

		const FText Details = FText::Format(
			LOCTEXT(
				"SimilarPropertyDetails",
				"Cloth collection property '{0}' is similar to the property '{1}' already set in an upstream node.\n"
				"This might result in an undefined simulation behavior."),
			FText::FromName(PropertyName),
			FText::FromName(SimilarPropertyName));

		FClothDataflowTools::LogAndToastWarning(DataflowNode, Headline, Details);
	}

	static FWeightedValueBounds ComputeFabricWeightedValueBounds(FCollectionClothFacade& ClothFacade,
		 TArray<float>& PatternValues, const TFunction<float(const FCollectionClothFabricFacade&)>& FabricValueFunction)
	{
		const int32 NumPatterns = ClothFacade.GetNumSimPatterns();
		
		PatternValues.Init(0.0f, NumPatterns);
		float MinValue = FLT_MAX, MaxValue = 0.0f;
		
		for(int32 PatternIndex = 0; PatternIndex < NumPatterns; ++PatternIndex)
		{
			FCollectionClothSimPatternFacade PatternFacade = ClothFacade.GetSimPattern(PatternIndex);
			const int32 FabricIndex = PatternFacade.GetFabricIndex();

			if(FabricIndex >= 0 && FabricIndex < ClothFacade.GetNumFabrics())
			{
				FCollectionClothFabricFacade FabricFacade = ClothFacade.GetFabric(FabricIndex);
				const float FabricValue = FabricValueFunction(FabricFacade);

				MinValue = FMath::Min(MinValue, FabricValue);
				MaxValue = FMath::Max(MaxValue, FabricValue);

				PatternValues[PatternIndex] = FabricValue;
			}
		}
		return {MinValue, MaxValue};
	}
	
	static FWeightedValueBounds ComputePatternWeightedValueBounds(FCollectionClothFacade& ClothFacade,
		 TArray<float>& PatternValues, const TFunction<float(const FCollectionClothSimPatternFacade&)>& PatternValueFunction)
	{
		const int32 NumPatterns = ClothFacade.GetNumSimPatterns();
		
		PatternValues.Init(0.0f, NumPatterns);
		float MinValue = FLT_MAX, MaxValue = 0.0f;
		
		for(int32 PatternIndex = 0; PatternIndex < NumPatterns; ++PatternIndex)
		{
			FCollectionClothSimPatternFacade PatternFacade = ClothFacade.GetSimPattern(PatternIndex);
			const float PatternValue = PatternValueFunction(PatternFacade);
			
			MinValue = FMath::Min(MinValue, PatternValue);
			MaxValue = FMath::Max(MaxValue, PatternValue);

			PatternValues[PatternIndex] = PatternValue;
		}
		return {MinValue, MaxValue};
	}

	static FWeightedValueBounds ComputeWeightedValueMap(FCollectionClothFacade& ClothFacade,
					  const FString& WeightMapName, const FWeightedValueBounds& WeightValueBounds, const TArray<float>& PatternValues)
	{
		const int32 NumPatterns = ClothFacade.GetNumSimPatterns();
		if(WeightValueBounds.Low != WeightValueBounds.High)
		{
			const int32 NumVertices = ClothFacade.GetNumSimVertices3D();
		
			TArray<int32> NumValues;
			NumValues.Init(0, NumVertices);

			TArray<float> VertexValues;
			VertexValues.Init(0.0f, NumVertices);
			
			ClothFacade.AddWeightMap(*WeightMapName);
			const TArrayView<float> ValueWeightMap = ClothFacade.GetWeightMap(*WeightMapName);
			
			for(int32 PatternIndex = 0; PatternIndex < NumPatterns; ++PatternIndex)
			{
				FCollectionClothSimPatternFacade PatternFacade = ClothFacade.GetSimPattern(PatternIndex);
				
				const TConstArrayView<int32> SimVertex3DLookup =
					static_cast<FCollectionClothSimPatternConstFacade&>(PatternFacade).GetSimVertex3DLookup();

				// Average of the pattern values at the seam
				for(const int32& SimVertex3DIndex : SimVertex3DLookup)
				{
					VertexValues[SimVertex3DIndex] += PatternValues[PatternIndex];
					NumValues[SimVertex3DIndex]++;
				}
			}
			for(int32 VertexIndex = 0; VertexIndex < NumVertices; ++VertexIndex)
			{
				if(NumValues[VertexIndex] > 0)
				{
					ValueWeightMap[VertexIndex] = (VertexValues[VertexIndex] / NumValues[VertexIndex]  - WeightValueBounds.Low) /
						(WeightValueBounds.High - WeightValueBounds.Low);
				}
			}
		}
		
		return WeightValueBounds;
	}

	static FWeightedValueBounds BuildFabricWeightedValue(FCollectionClothFacade& ClothFacade,
					  const FString& WeightMapName, const TFunction<float(const FCollectionClothFabricFacade&)>& FabricValueFunction)
	{
		TArray<float> PatternValues;
		const FWeightedValueBounds WeightValueBounds = ComputeFabricWeightedValueBounds(ClothFacade, PatternValues, FabricValueFunction);

		return ComputeWeightedValueMap(ClothFacade, WeightMapName, WeightValueBounds, PatternValues);
	}

	static FWeightedValueBounds BuildPatternWeightedValue(FCollectionClothFacade& ClothFacade,
					  const FString& WeightMapName, const TFunction<float(const FCollectionClothSimPatternFacade&)>& PatternValueFunction)
	{
		TArray<float> PatternValues;
		const FWeightedValueBounds WeightValueBounds = ComputePatternWeightedValueBounds(ClothFacade, PatternValues, PatternValueFunction);

		return ComputeWeightedValueMap(ClothFacade, WeightMapName, WeightValueBounds, PatternValues);
	}
}

FChaosClothAssetSimulationBaseConfigNode::FChaosClothAssetSimulationBaseConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{}

void FChaosClothAssetSimulationBaseConfigNode::RegisterCollectionConnections()
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetSimulationBaseConfigNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace Chaos::Softs;
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		FCollectionPropertyMutableFacade Properties(ClothCollection);
		Properties.DefineSchema();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AddProperties(Context, Properties);  // Deprecated 5.4
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FPropertyHelper PropertyHelper(*this, Context, Properties, ClothCollection);
		AddProperties(PropertyHelper);

		if (FCollectionClothFacade(ClothCollection).IsValid())  // Can only act on the collection if it is a valid cloth collection
		{
			EvaluateClothCollection(Context, ClothCollection);
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

int32 FChaosClothAssetSimulationBaseConfigNode::AddPropertyHelper(
	::Chaos::Softs::FCollectionPropertyMutableFacade& Properties,
	const FName& PropertyName,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags) const
{
	using namespace UE::Chaos::ClothAsset;

	int32 KeyIndex = Properties.GetKeyNameIndex(PropertyName);
	if (KeyIndex == INDEX_NONE)
	{
		KeyIndex = Properties.AddProperty(PropertyName);
	}
	else if (bWarnDuplicateProperty && !Properties.IsLegacy(KeyIndex))  // Only warns of duplicates when the property hasn't been set using a legacy Chaos config
	{
		Private::LogAndToastDuplicateProperty(*this, PropertyName);
	}
	// else don't warn and hope people know what they are doing

	// Set/clear flags
	Properties.SetFlags(KeyIndex, ECollectionPropertyFlags::Enabled | PropertyFlags); // Always enabled when then node is active, no longer a legacy property after being set by this node

	// Check for similar properties
	for (const FName& SimilarPropertyName : SimilarPropertyNames)
	{
		const int32 SimilarPropertyKeyIndex = Properties.GetKeyNameIndex(SimilarPropertyName);
		if (SimilarPropertyKeyIndex != INDEX_NONE && !Properties.IsLegacy(SimilarPropertyKeyIndex))  // Only warns of overrides when the property hasn't been set using a legacy Chaos config
		{
			Private::LogAndToastSimilarProperty(*this, PropertyName, SimilarPropertyName);
		}
	}

	return KeyIndex;
}

FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::FPropertyHelper(const FChaosClothAssetSimulationBaseConfigNode& InConfigNode, UE::Dataflow::FContext& InContext, ::Chaos::Softs::FCollectionPropertyMutableFacade& InProperties, const TSharedRef<FManagedArrayCollection>& InClothCollection)
	: ConfigNode(InConfigNode)
	, Context(InContext)
	, Properties(InProperties)
	, ClothCollection(InClothCollection) 
{}

int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyBool(
	const FName& PropertyName,
	bool PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	checkf(!PropertyName.ToString().StartsWith(TEXT("b")), TEXT("Unlike its C++ counterpart the Boolean property name should not start with a 'b'."));
	const int32 PropertyKeyIndex = ConfigNode.AddPropertyHelper(Properties, PropertyName, SimilarPropertyNames, PropertyFlags);
	Properties.SetValue(PropertyKeyIndex, PropertyValue);
	return PropertyKeyIndex;
}

int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyString(
	const FName& PropertyName,
	const FString& PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	const int32 PropertyKeyIndex = ConfigNode.AddPropertyHelper(Properties, PropertyName, SimilarPropertyNames, PropertyFlags);
	Properties.SetStringValue(PropertyKeyIndex, PropertyValue);
	return PropertyKeyIndex;
}

FString FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::GetPropertyString(const FString* PropertyReference) const
{
	return ConfigNode.GetValue<FString>(Context, PropertyReference);
}

int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyWeighted(
	const FName& PropertyName,
	const bool bIsAnimatable,
	const float& PropertyLow,
	const float& PropertyHigh,
	const FString& WeightMap,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags) const
{
	ensureMsgf(!EnumHasAnyFlags(PropertyFlags, ECollectionPropertyFlags::Animatable), TEXT("Animatable flag ignored. Weighted properties are set animatable through FChaosClothAssetWeightedValue::bIsAnimatable."));
	if (bIsAnimatable)
	{
		EnumAddFlags(PropertyFlags, ECollectionPropertyFlags::Animatable);  // Animatable
	}
	else
	{
		EnumRemoveFlags(PropertyFlags, ECollectionPropertyFlags::Animatable);  // Non-animatable
	}
	EnumAddFlags(PropertyFlags, ECollectionPropertyFlags::Interpolable);  // Interpolable
	
	const int32 PropertyKeyIndex = ConfigNode.AddPropertyHelper(Properties, PropertyName, SimilarPropertyNames, PropertyFlags);
	Properties.SetWeightedValue(PropertyKeyIndex, PropertyLow, PropertyHigh);
	Properties.SetStringValue(PropertyKeyIndex, ConfigNode.GetValue<FString>(Context, &WeightMap));
	return PropertyKeyIndex;
}

int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyWeighted(
	const FName& PropertyName,
	const FVector2f& PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	EnumAddFlags(PropertyFlags, ECollectionPropertyFlags::Interpolable);  // Interpolable
	const int32 PropertyKeyIndex = ConfigNode.AddPropertyHelper(Properties, PropertyName, SimilarPropertyNames, PropertyFlags);
	Properties.SetWeightedValue(PropertyKeyIndex, PropertyValue[0], PropertyValue[1]);
	return PropertyKeyIndex;
}

int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyWeighted(
	const FName& PropertyName,
	const FChaosClothAssetWeightedValue& PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	return SetPropertyWeighted(PropertyName, PropertyValue.bIsAnimatable, PropertyValue.Low,
		PropertyValue.High, PropertyValue.WeightMap, SimilarPropertyNames, PropertyFlags);
}

int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyWeighted(
	const FName& PropertyName,
	const FChaosClothAssetWeightedValueNonAnimatable& PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	return SetPropertyWeighted(PropertyName, false, PropertyValue.Low,
		PropertyValue.High, PropertyValue.WeightMap, SimilarPropertyNames, PropertyFlags);
}

int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyWeighted(
	const FName& PropertyName,
	const FChaosClothAssetWeightedValueNonAnimatableNoLowHighRange& PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	return SetPropertyWeighted(PropertyName, false, 0.0f,
		1.0f, PropertyValue.WeightMap, SimilarPropertyNames, PropertyFlags);
}


void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::OverridePropertiesBool(const TArray<FName>& PropertyNames, bool bPropertyValue)
{
	for (const FName& PropertyName : PropertyNames)
	{
		const int32 PropertyKeyIndex = Properties.GetKeyNameIndex(PropertyName);
		if (PropertyKeyIndex != INDEX_NONE)
		{
			Properties.SetValue(PropertyKeyIndex, bPropertyValue);
		}
	}
}

void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::OverridePropertiesFloat(const TArray<FName>& PropertyNames, const EChaosClothAssetConstraintOverrideType OverrideType, const float OverrideValue)
{
	if (OverrideType == EChaosClothAssetConstraintOverrideType::None)
	{
		return;
	}
	for (const FName& PropertyName : PropertyNames)
	{
		const int32 PropertyKeyIndex = Properties.GetKeyNameIndex(PropertyName);
		if (PropertyKeyIndex != INDEX_NONE)
		{
			if (OverrideType == EChaosClothAssetConstraintOverrideType::Override)
			{
				Properties.SetValue(PropertyKeyIndex, OverrideValue);
			}
			else if (OverrideType == EChaosClothAssetConstraintOverrideType::Multiply)
			{
				Properties.SetValue(PropertyKeyIndex, Properties.GetValue<float>(PropertyKeyIndex) * OverrideValue);
			}
		}
	}
}

void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::OverridePropertiesWeighted(const TArray<FName>& PropertyNames, const EChaosClothAssetConstraintOverrideType OverrideType, const FChaosClothAssetWeightedValueOverride& OverrideValue)
{
	if (OverrideType == EChaosClothAssetConstraintOverrideType::None)
	{
		return;
	}
	for (const FName& PropertyName : PropertyNames)
	{
		const int32 PropertyKeyIndex = Properties.GetKeyNameIndex(PropertyName);
		if (PropertyKeyIndex != INDEX_NONE)
		{
			if (OverrideType == EChaosClothAssetConstraintOverrideType::Override)
			{
				Properties.SetWeightedValue(PropertyKeyIndex, OverrideValue.Low, OverrideValue.High);
				
			}
			else if (OverrideType == EChaosClothAssetConstraintOverrideType::Multiply)
			{
				Properties.SetWeightedValue(PropertyKeyIndex, Properties.GetLowValue<float>(PropertyKeyIndex) * OverrideValue.Low, Properties.GetHighValue<float>(PropertyKeyIndex) * OverrideValue.High);
			}
		}
	}
}

int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyString(
	const FName& PropertyName,
	const FChaosClothAssetConnectableIStringValue& PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	const int32 PropertyKeyIndex = ConfigNode.AddPropertyHelper(Properties, PropertyName, SimilarPropertyNames, PropertyFlags);
	Properties.SetStringValue(PropertyKeyIndex, ConfigNode.GetValue<FString>(Context, &PropertyValue.StringValue));
	return PropertyKeyIndex;
}

template<typename PropertyType>
void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetSolverProperty(const FName& PropertyName, const PropertyType& PropertyValue,
	const TFunction<typename PropertyType::ImportedType(UE::Chaos::ClothAsset::FCollectionClothFacade&)>& SolverValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags)
{
	UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(GetClothCollection());
	if(PropertyValue.bUseImportedValue && ClothFacade.IsValid(UE::Chaos::ClothAsset::EClothCollectionExtendedSchemas::Solvers) && ClothFacade.HasSolverElement())
	{
		PropertyValue.ImportedValue = SolverValueFunction(ClothFacade);
	}
	SetProperty(PropertyName, PropertyValue.ImportedValue, SimilarPropertyNames, PropertyFlags);
}

template<typename PropertyType>
void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetFabricProperty(const FName& PropertyName, const PropertyType& PropertyValue,
	const TFunction<typename PropertyType::ImportedType(UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags)
{
	UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(GetClothCollection());
	if(PropertyValue.bUseImportedValue && ClothFacade.IsValid(UE::Chaos::ClothAsset::EClothCollectionExtendedSchemas::Fabrics) && ClothFacade.GetNumFabrics() > 0)
	{
		const int32 NumFabrics = ClothFacade.GetNumFabrics();
		
		typename PropertyType::ImportedType AveragedFabricValue(0.0f);
		for(int32 FabricIndex = 0; FabricIndex < NumFabrics; ++FabricIndex)
		{
			UE::Chaos::ClothAsset::FCollectionClothFabricFacade FabricFacade = ClothFacade.GetFabric(FabricIndex);
			AveragedFabricValue += FabricValueFunction(FabricFacade);
		}
		AveragedFabricValue /= NumFabrics;
		PropertyValue.ImportedValue = AveragedFabricValue;
	}
	SetProperty(PropertyName, PropertyValue.ImportedValue, SimilarPropertyNames, PropertyFlags);
}

template<typename PropertyType>
void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetFabricPropertyWeighted(
	const FName& PropertyName, const PropertyType& PropertyValue,
	const TFunction<float(const UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags)
{
	UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(GetClothCollection());
	if(PropertyValue.bCouldUseFabrics && ClothFacade.IsValid(UE::Chaos::ClothAsset::EClothCollectionExtendedSchemas::Fabrics) && (ClothFacade.GetNumFabrics() > 0) &&
			(PropertyValue.bImportFabricBounds || PropertyValue.bBuildFabricMaps))
	{
		UE::Chaos::ClothAsset::FWeightedValueBounds WeightedValueBounds;
		if(PropertyValue.bBuildFabricMaps)
		{
			WeightedValueBounds = UE::Chaos::ClothAsset::Private::BuildFabricWeightedValue(ClothFacade, 
			 GetPropertyString(&PropertyValue.WeightMap), FabricValueFunction);
		}
		else 
		{
			TArray<float> PatternValues;
			WeightedValueBounds = UE::Chaos::ClothAsset::Private::ComputeFabricWeightedValueBounds(
				ClothFacade, PatternValues, FabricValueFunction);
		}
		if(PropertyValue.bImportFabricBounds)
		{
			PropertyValue.Low = WeightedValueBounds.Low;
			PropertyValue.High = WeightedValueBounds.High;
		}
	}
	SetPropertyWeighted(PropertyName, PropertyValue, SimilarPropertyNames, PropertyFlags);
}


template<typename MapType, typename PropertyType>
void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetFabricPropertyString(
	const FName& PropertyName, const PropertyType& PropertyValue,
	const TFunction<MapType(const UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags, const FName& GroupName)
{
	UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(GetClothCollection());
	if(PropertyValue.bCouldUseFabrics && ClothFacade.IsValid(UE::Chaos::ClothAsset::EClothCollectionExtendedSchemas::Fabrics) && (ClothFacade.GetNumFabrics() > 0) && PropertyValue.bBuildFabricMaps)
	{
		const int32 NumPatterns = ClothFacade.GetNumSimPatterns();

		const FName StringValue(*GetPropertyString(&PropertyValue.StringValue));
		if (!ClothFacade.HasUserDefinedAttribute<MapType>(StringValue, GroupName))
		{
			ClothFacade.AddUserDefinedAttribute<MapType>(StringValue, GroupName);
		}
		TArrayView<MapType> UserMap = ClothFacade.GetUserDefinedAttribute<MapType>(StringValue, GroupName);
		
		for(int32 PatternIndex = 0; PatternIndex < NumPatterns; ++PatternIndex) 
		{
			UE::Chaos::ClothAsset::FCollectionClothSimPatternFacade PatternFacade = ClothFacade.GetSimPattern(PatternIndex);
			const int32 FabricIndex = PatternFacade.GetFabricIndex();

			if(FabricIndex >= 0 && FabricIndex < ClothFacade.GetNumFabrics())
			{
				UE::Chaos::ClothAsset::FCollectionClothFabricFacade FabricFacade = ClothFacade.GetFabric(FabricIndex);
				const float FabricValue = FabricValueFunction(FabricFacade);

				if(GroupName == UE::Chaos::ClothAsset::ClothCollectionGroup::SimFaces)
				{
					const int32 PatternFacesStart = PatternFacade.GetSimFacesOffset();
					const int32 PatternFacesEnd = PatternFacade.GetNumSimFaces() + PatternFacesStart;
				
					for(int32 SimFaceIndex = PatternFacesStart; SimFaceIndex < PatternFacesEnd; ++SimFaceIndex)
					{
						UserMap[SimFaceIndex] = FabricValue;
					}
				}
				else if(GroupName == UE::Chaos::ClothAsset::ClothCollectionGroup::SimVertices3D)
				{
					const TConstArrayView<int32> SimVertex3DLookup =
					static_cast<UE::Chaos::ClothAsset::FCollectionClothSimPatternConstFacade&>(PatternFacade).GetSimVertex3DLookup();

					for(const int32& SimVertex3DIndex : SimVertex3DLookup)
					{
						UserMap[SimVertex3DIndex] = FabricValue;
					}
				}
			}
		}
	}
	SetPropertyString(PropertyName, PropertyValue, SimilarPropertyNames, PropertyFlags);
}

template<typename PropertyType>
void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetSolverPropertyWeighted(
	const FName& PropertyName, const PropertyType& PropertyValue,
	const TFunction<float(const UE::Chaos::ClothAsset::FCollectionClothFacade&)>& SolverValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags)
{
	UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(GetClothCollection());
	if(PropertyValue.bCouldUseFabrics && (ClothFacade.IsValid(UE::Chaos::ClothAsset::EClothCollectionExtendedSchemas::Solvers) && ClothFacade.HasSolverElement()) && PropertyValue.bImportFabricBounds)
	{
		const float SolverValue = SolverValueFunction(ClothFacade);
	
		PropertyValue.Low = SolverValue;
		PropertyValue.High = SolverValue;
	}
	SetPropertyWeighted(PropertyName, PropertyValue, SimilarPropertyNames, PropertyFlags);
}

template CHAOSCLOTHASSETDATAFLOWNODES_API void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetSolverProperty<FChaosClothAssetImportedVectorValue>(const FName& PropertyName,
	const FChaosClothAssetImportedVectorValue& PropertyValue,
	const TFunction<FChaosClothAssetImportedVectorValue::ImportedType(UE::Chaos::ClothAsset::FCollectionClothFacade&)>& SolverValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags);

template CHAOSCLOTHASSETDATAFLOWNODES_API void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetSolverProperty<FChaosClothAssetImportedFloatValue>(const FName& PropertyName,
	const FChaosClothAssetImportedFloatValue& PropertyValue,
	const TFunction<FChaosClothAssetImportedFloatValue::ImportedType(UE::Chaos::ClothAsset::FCollectionClothFacade&)>& SolverValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags);

template CHAOSCLOTHASSETDATAFLOWNODES_API void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetSolverProperty<FChaosClothAssetImportedIntValue>(const FName& PropertyName,
	const FChaosClothAssetImportedIntValue& PropertyValue,
	const TFunction<FChaosClothAssetImportedIntValue::ImportedType(UE::Chaos::ClothAsset::FCollectionClothFacade&)>& SolverValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags);
	
template void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetFabricProperty<FChaosClothAssetImportedVectorValue>(const FName& PropertyName,
	const FChaosClothAssetImportedVectorValue& PropertyValue, 
	const TFunction<FChaosClothAssetImportedVectorValue::ImportedType(UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags);

template void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetFabricProperty<FChaosClothAssetImportedFloatValue>(const FName& PropertyName,
	const FChaosClothAssetImportedFloatValue& PropertyValue, 
	const TFunction<FChaosClothAssetImportedFloatValue::ImportedType(UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags);

template void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetFabricProperty<FChaosClothAssetImportedIntValue>(const FName& PropertyName,
	const FChaosClothAssetImportedIntValue& PropertyValue, 
	const TFunction<FChaosClothAssetImportedIntValue::ImportedType(UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags);

template void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetSolverPropertyWeighted(
	const FName& PropertyName, const FChaosClothAssetWeightedValueNonAnimatable& PropertyValue,
	const TFunction<float(const UE::Chaos::ClothAsset::FCollectionClothFacade&)>& SolverValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags);

template void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetSolverPropertyWeighted(
	const FName& PropertyName, const FChaosClothAssetWeightedValue& PropertyValue,
	const TFunction<float(const UE::Chaos::ClothAsset::FCollectionClothFacade&)>& SolverValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags);

template void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetFabricPropertyWeighted<FChaosClothAssetWeightedValue>(const FName& PropertyName,
	const FChaosClothAssetWeightedValue& PropertyValue, 
	const TFunction<float(const UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags);

template void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetFabricPropertyWeighted<FChaosClothAssetWeightedValueNonAnimatable>(const FName& PropertyName,
	const FChaosClothAssetWeightedValueNonAnimatable& PropertyValue, 
	const TFunction<float(const UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags);

template void FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetFabricPropertyString<int32,FChaosClothAssetConnectableIStringValue>(
	const FName& PropertyName, const FChaosClothAssetConnectableIStringValue& PropertyValue,
	const TFunction<int32(const UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
	const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags, const FName& GroupName);


#undef LOCTEXT_NAMESPACE
