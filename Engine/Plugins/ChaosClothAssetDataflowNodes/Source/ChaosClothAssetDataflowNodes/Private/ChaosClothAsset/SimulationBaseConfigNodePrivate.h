// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowInputOutput.h"
#include "ChaosClothAsset/WeightedValue.h"

// These macros are now deprecated, use the FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper struct instead.

/** Macro for adding and setting value properties from inside AddProperties() */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTY(Property) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property))); \
	Properties.SetValue(Property##KeyIndex, Property);

/** Macro for adding and setting boolean value properties from inside AddProperties() */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTYBOOL(Property) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property))); \
	Properties.SetValue(Property##KeyIndex, b##Property);

/** Macro for adding and setting enum value as int32 properties from inside AddProperties() */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTYENUM(Property) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property))); \
	Properties.SetValue(Property##KeyIndex, (int32)Property);

/** Macro for adding and setting string value from inside AddProperties() */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTYSTRING(Property) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property))); \
	Properties.SetStringValue(Property##KeyIndex, Property);

/** Macro for adding and setting weighted value properties from inside AddProperties() and check for a similar property being overriden. */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTYCHECKED1(Property, SimilarProperty1) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property)), true, \
		{ FName(TEXT(#SimilarProperty1)) }); \
	Properties.SetValue(Property##KeyIndex, Property);

/** Macro for adding and setting weighted value properties from inside AddProperties() and check for two similar properties being overriden. */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTYCHECKED2(Property, SimilarProperty1, SimilarProperty2) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property)), true, \
		{ FName(TEXT(#SimilarProperty1)), FName(TEXT(#SimilarProperty2)) }); \
	Properties.SetValue(Property##KeyIndex, Property);

/** Macro for adding and setting enum value as int32 properties from inside AddProperties() and check for two similar properties being overriden. */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTYENUMCHECKED2(Property, SimilarProperty1, SimilarProperty2) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property)), true, \
		{ FName(TEXT(#SimilarProperty1)), FName(TEXT(#SimilarProperty2) )}); \
	Properties.SetValue(Property##KeyIndex, (int32)Property);

/** Macro for adding and setting weighted value properties from inside AddProperties() */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTYWEIGHTED(Property) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property)), Property.bIsAnimatable); \
	Properties.SetWeightedValue(Property##KeyIndex, Property.Low, Property.High); \
	Properties.SetStringValue(Property##KeyIndex, GetValue<FString>(Context, &Property.WeightMap)); \
	Property.WeightMap_Override = GetValue<FString>(Context, &Property.WeightMap, UE::Chaos::ClothAsset::FWeightMapTools::NotOverridden); \
	UE::Chaos::ClothAsset::FWeightMapTools::MakeWeightMapName(Property.WeightMap);

/** Macro for adding and setting weighted value properties from inside AddProperties() and check for a similar property being overriden. */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTYWEIGHTEDCHECKED1(Property, SimilarProperty1) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property)), Property.bIsAnimatable, \
		{ FName(TEXT(#SimilarProperty1)) }); \
	Properties.SetWeightedValue(Property##KeyIndex, Property.Low, Property.High); \
	Properties.SetStringValue(Property##KeyIndex, GetValue<FString>(Context, &Property.WeightMap)); \
	Property.WeightMap_Override = GetValue<FString>(Context, &Property.WeightMap, UE::Chaos::ClothAsset::FWeightMapTools::NotOverridden); \
	UE::Chaos::ClothAsset::FWeightMapTools::MakeWeightMapName(Property.WeightMap);

/** Macro for adding and setting weighted value properties from inside AddProperties() and check for two similar properties being overriden. */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTYWEIGHTEDCHECKED2(Property, SimilarProperty1, SimilarProperty2) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property)), Property.bIsAnimatable, \
		{ FName(TEXT(#SimilarProperty1)), FName(TEXT(#SimilarProperty2)) }); \
	Properties.SetWeightedValue(Property##KeyIndex, Property.Low, Property.High); \
	Properties.SetStringValue(Property##KeyIndex, GetValue<FString>(Context, &Property.WeightMap)); \
	Property.WeightMap_Override = GetValue<FString>(Context, &Property.WeightMap, UE::Chaos::ClothAsset::FWeightMapTools::NotOverridden); \
	UE::Chaos::ClothAsset::FWeightMapTools::MakeWeightMapName(Property.WeightMap);

/** Macro for adding and setting weighted value properties from inside AddProperties() and check for three similar properties being overriden. */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTYWEIGHTEDCHECKED3(Property, SimilarProperty1, SimilarProperty2, SimilarProperty3) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property)), Property.bIsAnimatable, \
		{ FName(TEXT(#SimilarProperty1)), FName(TEXT(#SimilarProperty2)), FName(TEXT(#SimilarProperty3)) }); \
	Properties.SetWeightedValue(Property##KeyIndex, Property.Low, Property.High); \
	Properties.SetStringValue(Property##KeyIndex, GetValue<FString>(Context, &Property.WeightMap)); \
	Property.WeightMap_Override = GetValue<FString>(Context, &Property.WeightMap, UE::Chaos::ClothAsset::FWeightMapTools::NotOverridden); \
	UE::Chaos::ClothAsset::FWeightMapTools::MakeWeightMapName(Property.WeightMap);

/** Macro for adding and setting weighted value properties from inside AddProperties() and check for four similar properties being overriden. */
#define UE_CHAOS_CLOTHASSET_SIMULATIONCONFIG_SETPROPERTYWEIGHTEDCHECKED4(Property, SimilarProperty1, SimilarProperty2, SimilarProperty3, SimilarProperty4) \
	const int32 Property##KeyIndex = AddPropertyHelper(Properties, FName(TEXT(#Property)), Property.bIsAnimatable, \
		{ FName(TEXT(#SimilarProperty1)), FName(TEXT(#SimilarProperty2)), FName(TEXT(#SimilarProperty3)), FName(TEXT(#SimilarProperty4)) }); \
	Properties.SetWeightedValue(Property##KeyIndex, Property.Low, Property.High); \
	Properties.SetStringValue(Property##KeyIndex, GetValue<FString>(Context, &Property.WeightMap)); \
	Property.WeightMap_Override = GetValue<FString>(Context, &Property.WeightMap, UE::Chaos::ClothAsset::FWeightMapTools::NotOverridden); \
	UE::Chaos::ClothAsset::FWeightMapTools::MakeWeightMapName(Property.WeightMap);

