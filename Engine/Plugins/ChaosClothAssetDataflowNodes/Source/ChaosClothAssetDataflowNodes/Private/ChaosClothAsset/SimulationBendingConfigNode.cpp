// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationBendingConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationBendingConfigNode)

FChaosClothAssetSimulationBendingConfigNode::FChaosClothAssetSimulationBendingConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&FlatnessRatio.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&RestAngle.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BendingStiffness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BendingStiffnessWarp.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BendingStiffnessWeft.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BendingStiffnessBias.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BendingDamping.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BendingAnisoDamping.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&AnisoBucklingRatio.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BucklingStiffness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BucklingStiffnessWarp.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BucklingStiffnessWeft.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BucklingStiffnessBias.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&BucklingRatioWeighted.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationBendingConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	if(SolverType == EChaosClothAssetConstraintSolverType::XPBD)
	{
		if(DistributionType == EChaosClothAssetConstraintDistributionType::Isotropic)
		{
			if(ConstraintType == EChaosClothAssetBendingConstraintType::FacesSpring)
			{
				PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDBendingSpringStiffness")), BendingStiffness,{
					FName(TEXT("XPBDBendingElementStiffness")),
					FName(TEXT("XPBDAnisoBendingStiffnessWarp")),
					FName(TEXT("BendingSpringStiffness")),
					FName(TEXT("BendingElementStiffness"))});

				PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDBendingSpringDamping")), BendingDamping,{
					FName(TEXT("XPBDBendingElementDamping")),
					FName(TEXT("XPBDAnisoBendingDamping"))});
			}
			else
			{
				PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDBendingElementStiffness")), BendingStiffness,{
					FName(TEXT("XPBDBendingSpringStiffness")),
					FName(TEXT("XPBDAnisoBendingStiffnessWarp")),
					FName(TEXT("BendingSpringStiffness")),
					FName(TEXT("BendingElementStiffness"))});

				PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDBendingElementDamping")), BendingDamping,{
					FName(TEXT("XPBDBendingSpringDamping")),
					FName(TEXT("XPBDAnisoBendingDamping"))});
				
				PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDBucklingStiffness")), BucklingStiffness,{
					FName(TEXT("XPBDAnisoBucklingStiffnessWarp")),
					FName(TEXT("BucklingStiffness"))});
				
				PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDBucklingRatio")), BucklingRatioWeighted, {
					FName(TEXT("XPBDAnisoBucklingRatio")),
					FName(TEXT("BucklingRatio"))});
			}
			PropertyHelper.SetPropertyEnum(FName(TEXT("XPBDRestAngleType")), RestAngleType, {
				FName(TEXT("XPBDAnisoRestAngleType")),
				FName(TEXT("RestAngleType")) 
			}, ECollectionPropertyFlags::None);
			
			PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDFlatnessRatio")), FlatnessRatio, {
				FName(TEXT("XPBDAnisoFlatnessRatio")),
				FName(TEXT("FlatnessRatio"))       
			});
			
			PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDRestAngle")), RestAngle, {
				FName(TEXT("XPBDAnisoRestAngle")),
				FName(TEXT("RestAngle"))       
			});
		}
		else
		{
			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoBendingStiffnessWarp")), BendingStiffnessWarp, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			 {
				 return FabricFacade.GetBendingStiffness().Warp;
			 }, {
				FName(TEXT("BendingSpringStiffness")),     
				FName(TEXT("BendingElementStiffness")),     
				FName(TEXT("XPBDBendingSpringStiffness")),  
				FName(TEXT("XPBDBendingElementStiffness"))});

			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoBendingStiffnessWeft")), BendingStiffnessWeft, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			 {
				 return FabricFacade.GetBendingStiffness().Weft;
			 }, {});
			
			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoBendingStiffnessBias")), BendingStiffnessBias, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			 {
				 return FabricFacade.GetBendingStiffness().Bias;
			 }, {});
			
			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoBucklingStiffnessWarp")), BucklingStiffnessWarp, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			 {
				 return FabricFacade.GetBucklingRatio() > UE_SMALL_NUMBER ? FabricFacade.GetBucklingStiffness().Warp : FabricFacade.GetBendingStiffness().Warp;
			 }, {
				FName(TEXT("BucklingStiffness")),    
				FName(TEXT("XPBDBucklingStiffness"))});
			
			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoBucklingStiffnessWeft")), BucklingStiffnessWeft, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			{
				return FabricFacade.GetBucklingRatio() > UE_SMALL_NUMBER ? FabricFacade.GetBucklingStiffness().Weft : FabricFacade.GetBendingStiffness().Weft;
			},{});
			
			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoBucklingStiffnessBias")), BucklingStiffnessBias, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			{
				return FabricFacade.GetBucklingRatio() > UE_SMALL_NUMBER ? FabricFacade.GetBucklingStiffness().Bias : FabricFacade.GetBendingStiffness().Bias;
			},{});

			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoBendingDamping")), BendingAnisoDamping, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			{
				return FabricFacade.GetDamping();
			},{
				FName(TEXT("XPBDBendingSpringDamping")),  
				FName(TEXT("XPBDBendingElementDamping"))});

			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoBucklingRatio")), AnisoBucklingRatio, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			{
				return FabricFacade.GetBucklingRatio();
			}, {
				FName(TEXT("BucklingRatio")),
				FName(TEXT("XPBDBucklingRatio"))});

			PropertyHelper.SetPropertyEnum(FName(TEXT("XPBDAnisoRestAngleType")), RestAngleType,{
					FName(TEXT("XPBDRestAngleType")),
					FName(TEXT("RestAngleType"))  
				}, ECollectionPropertyFlags::None);
			
			PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDAnisoFlatnessRatio")), FlatnessRatio, {
				FName(TEXT("XPBDFlatnessRatio")),
				FName(TEXT("FlatnessRatio"))       
			});
			
			PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDAnisoRestAngle")), RestAngle, {
				FName(TEXT("XPBDRestAngle")),
				FName(TEXT("RestAngle"))       
			});
		}
	}
	else
	{
		if(ConstraintType == EChaosClothAssetBendingConstraintType::FacesSpring)
		{
			PropertyHelper.SetPropertyWeighted(FName(TEXT("BendingSpringStiffness")), BendingStiffness, {
				FName(TEXT("BendingElementStiffness")),
				FName(TEXT("XPBDBendingSpringStiffness")),    
				FName(TEXT("XPBDBendingElementStiffness")),   
				FName(TEXT("XPBDAnisoBendingStiffnessWarp"))});
		}
		else
		{
			PropertyHelper.SetPropertyWeighted(FName(TEXT("BendingElementStiffness")), BendingStiffness, {
				FName(TEXT("BendingSpringStiffness")),        // Existing properties to warn against
				FName(TEXT("XPBDBendingSpringStiffness")),
				FName(TEXT("XPBDBendingElementStiffness")),   
				FName(TEXT("XPBDAnisoBendingStiffnessWarp"))});
			
			PropertyHelper.SetPropertyWeighted(FName(TEXT("BucklingStiffness")), BucklingStiffness, {
				FName(TEXT("XPBDBucklingStiffness")),
				FName(TEXT("XPBDAnisoBucklingStiffnessWarp"))});
			
			PropertyHelper.SetPropertyWeighted(FName(TEXT("BucklingRatio")), BucklingRatioWeighted, {
				FName(TEXT("XPBDBucklingRatio")),
				FName(TEXT("XPBDAnisoBucklingRatio"))});
		}
		
		PropertyHelper.SetPropertyEnum(FName(TEXT("RestAngleType")), RestAngleType, {
			FName(TEXT("XPBDAnisoRestAngleType")),
			FName(TEXT("XPBDRestAngleType")) 
		}, ECollectionPropertyFlags::None);
		
		PropertyHelper.SetPropertyWeighted(FName(TEXT("FlatnessRatio")), FlatnessRatio, {
				FName(TEXT("XPBDAnisoFlatnessRatio")),
				FName(TEXT("XPBDFlatnessRatio"))       
			});
		PropertyHelper.SetPropertyWeighted(FName(TEXT("RestAngle")), RestAngle, {
			FName(TEXT("XPBDAnisoRestAngle")), 
			FName(TEXT("XPBDRestAngle"))       
		});
	}
}

void FChaosClothAssetSimulationBendingConfigNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsLoading())
	{
#if WITH_EDITORONLY_DATA
		if (BucklingRatio_DEPRECATED != BucklingRatioDeprecatedDefault)
		{
			BucklingRatioWeighted.Low = BucklingRatioWeighted.High = BucklingRatio_DEPRECATED;
			AnisoBucklingRatio.Low = AnisoBucklingRatio.High = BucklingRatio_DEPRECATED;
			BucklingRatio_DEPRECATED = BucklingRatioDeprecatedDefault;
		}
#endif
	}
}
