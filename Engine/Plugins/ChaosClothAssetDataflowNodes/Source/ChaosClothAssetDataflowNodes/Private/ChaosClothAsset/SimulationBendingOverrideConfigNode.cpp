// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationBendingOverrideConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationBendingOverrideConfigNode)

FChaosClothAssetSimulationBendingOverrideConfigNode::FChaosClothAssetSimulationBendingOverrideConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
}

void FChaosClothAssetSimulationBendingOverrideConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	if (OverrideFlatnessRatio != EChaosClothAssetConstraintOverrideType::None)
	{
		PropertyHelper.OverridePropertiesWeighted({
				FName(TEXT("FlatnessRatio")),
				FName(TEXT("XPBDFlatnessRatio")),
				FName(TEXT("XPBDAnisoFlatnessRatio")) }, OverrideFlatnessRatio, FlatnessRatio);
	}
	if (OverrideBendingStiffness != EChaosClothAssetConstraintOverrideType::None)
	{
		TArray<FName> BendingPropertyNames =
		{
			FName(TEXT("BendingSpringStiffness")),
			FName(TEXT("BendingElementStiffness")),
			FName(TEXT("XPBDBendingSpringStiffness")),
			FName(TEXT("XPBDBendingElementStiffness"))
		};
		if (bApplyUniformBendingStiffnessOverride)
		{
			BendingPropertyNames.Append(
				{
					FName(TEXT("XPBDAnisoBendingStiffnessWarp")),
					FName(TEXT("XPBDAnisoBendingStiffnessWeft")),
					FName(TEXT("XPBDAnisoBendingStiffnessBias")),
				});
		}
		if (bApplyBendingStiffnessOverrideToBuckling)
		{
			BendingPropertyNames.Append(
				{
					FName(TEXT("BucklingStiffness")),
					FName(TEXT("XPBDBucklingStiffness"))
				});

			if (bApplyUniformBendingStiffnessOverride)
			{
				BendingPropertyNames.Append(
					{
						FName(TEXT("XPBDAnisoBucklingStiffnessWarp")),
						FName(TEXT("XPBDAnisoBucklingStiffnessWeft")),
						FName(TEXT("XPBDAnisoBucklingStiffnessBias")),
					});
			}
		}
		PropertyHelper.OverridePropertiesWeighted(BendingPropertyNames, OverrideBendingStiffness, BendingStiffness);
	}

	if (OverrideBucklingRatio != EChaosClothAssetConstraintOverrideType::None)
	{
		PropertyHelper.OverridePropertiesFloat({
				FName(TEXT("BucklingRatio")),
				FName(TEXT("XPBDBucklingRatio")),
				FName(TEXT("XPBDAnisoBucklingRatio")) }, OverrideBucklingRatio, BucklingRatio);
	}

	if ((OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyBendingStiffnessOverrideToBuckling) && OverrideBucklingStiffness != EChaosClothAssetConstraintOverrideType::None)
	{
		TArray<FName> BucklingPropertyNames =
		{
			FName(TEXT("BucklingStiffness")),
			FName(TEXT("XPBDBucklingStiffness"))
		};
		if (bApplyUniformBucklingStiffnessOverride)
		{
			BucklingPropertyNames.Append(
				{
						FName(TEXT("XPBDAnisoBucklingStiffnessWarp")),
						FName(TEXT("XPBDAnisoBucklingStiffnessWeft")),
						FName(TEXT("XPBDAnisoBucklingStiffnessBias")),
				});
		}
		PropertyHelper.OverridePropertiesWeighted(BucklingPropertyNames, OverrideBucklingStiffness, BucklingStiffness);
	}

	if (OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride)
	{
		if (OverrideBendingStiffnessWarp != EChaosClothAssetConstraintOverrideType::None)
		{
			PropertyHelper.OverridePropertiesWeighted({ FName(TEXT("XPBDAnisoBendingStiffnessWarp"))}, OverrideBendingStiffnessWarp, BendingStiffnessWarp);
		}
		if (OverrideBendingStiffnessWeft != EChaosClothAssetConstraintOverrideType::None)
		{
			PropertyHelper.OverridePropertiesWeighted({ FName(TEXT("XPBDAnisoBendingStiffnessWeft")) }, OverrideBendingStiffnessWeft, BendingStiffnessWeft);
		}
		if (OverrideBendingStiffnessBias != EChaosClothAssetConstraintOverrideType::None)
		{
			PropertyHelper.OverridePropertiesWeighted({ FName(TEXT("XPBDAnisoBendingStiffnessBias")) }, OverrideBendingStiffnessBias, BendingStiffnessBias);
		}
	}
	if ((OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride || !bApplyBendingStiffnessOverrideToBuckling)
		&& (OverrideBucklingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBucklingStiffnessOverride))
	{
		if (OverrideBucklingStiffnessWarp != EChaosClothAssetConstraintOverrideType::None)
		{
			PropertyHelper.OverridePropertiesWeighted({ FName(TEXT("XPBDAnisoBucklingStiffnessWarp")) }, OverrideBucklingStiffnessWarp, BucklingStiffnessWarp);
		}
		if (OverrideBucklingStiffnessWeft != EChaosClothAssetConstraintOverrideType::None)
		{
			PropertyHelper.OverridePropertiesWeighted({ FName(TEXT("XPBDAnisoBucklingStiffnessWeft")) }, OverrideBucklingStiffnessWeft, BucklingStiffnessWeft);
		}
		if (OverrideBucklingStiffnessBias != EChaosClothAssetConstraintOverrideType::None)
		{
			PropertyHelper.OverridePropertiesWeighted({ FName(TEXT("XPBDAnisoBucklingStiffnessBias")) }, OverrideBucklingStiffnessBias, BucklingStiffnessBias);
		}
	}
	if (OverrideBendingDamping != EChaosClothAssetConstraintOverrideType::None)
	{
		PropertyHelper.OverridePropertiesWeighted({
				FName(TEXT("XPBDBendingSpringDamping")),
				FName(TEXT("XPBDBendingElementDamping"))}, OverrideBendingDamping, BendingDamping);
	}
}
