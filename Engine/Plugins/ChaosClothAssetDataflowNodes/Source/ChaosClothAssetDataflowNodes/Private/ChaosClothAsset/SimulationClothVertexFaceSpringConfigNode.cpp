// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationClothVertexFaceSpringConfigNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Spatial/MeshAABBTree3.h"
#include "Algo/MaxElement.h"
#include "MeshQueries.h"
#include "TriangleTypes.h"
#include "CompGeom/Delaunay3.h"

#include "ChaosClothAsset/ClothDataflowTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationClothVertexFaceSpringConfigNode)

#define LOCTEXT_NAMESPACE "SimulationClothVertexFaceSpringConfigNode"

namespace UE::Chaos::ClothAsset::Private
{
	struct FVertexFaceConstraint
	{
		int32 SourceVertex;
		int32 TargetFace;
		enum struct EIntersectionType : uint8
		{
			Closest,
			Ray,
			RayFlipped
		} IntersectionType;

		bool operator==(const FVertexFaceConstraint& Other) const
		{
			return SourceVertex == Other.SourceVertex &&
				TargetFace == Other.TargetFace &&
				IntersectionType == Other.IntersectionType;
		}
	};

	struct FFaceSetMeshAdapter
	{
		const TSet<int32>& TargetFaces;
		TConstArrayView<FIntVector3> Elements;
		TConstArrayView<FVector3f> Positions; 
		int32 MaxTriangleId;

		FFaceSetMeshAdapter(const TSet<int32>& InTargetFaces, const FCollectionClothConstFacade& InCloth)
			: TargetFaces(InTargetFaces)
			, Elements(InCloth.GetSimIndices3D())
			, Positions(InCloth.GetSimPosition3D())
		{
			check(!TargetFaces.IsEmpty());
			MaxTriangleId = *Algo::MaxElement(TargetFaces);
		}

		bool IsTriangle(int32 Index) const
		{
			return TargetFaces.Contains(Index);
		}

		bool IsVertex(int32 Index) const
		{
			return Positions.IsValidIndex(Index);
		}

		int32 MaxTriangleID() const
		{
			return MaxTriangleId;
		}

		int32 TriangleCount() const
		{
			return TargetFaces.Num();
		}

		int32 VertexCount() const
		{
			return Positions.Num();
		}

		uint64 GetChangeStamp() const
		{
			return 1; // no concept of change.
		}

		UE::Geometry::FIndex3i GetTriangle(int32 Index) const
		{
			const FIntVector3& Element = Elements[Index];
			return UE::Geometry::FIndex3i(Element[0], Element[1], Element[2]);
		}

		FVector3d GetVertex(int32 Index) const
		{
			return FVector3d(Positions[Index]);
		}

		void GetTriVertices(int32 TriID, FVector3d& V0, FVector3d& V1, FVector3d& V2) const
		{
			const FIntVector3& Element = Elements[TriID];
			V0 = FVector3d(Positions[Element[0]]);
			V1 = FVector3d(Positions[Element[1]]);
			V2 = FVector3d(Positions[Element[2]]);
		}		
	};

	static uint32 GetTypeHash(const FVertexFaceConstraint& Constraint)
	{
		uint32 Hash = ::GetTypeHash(Constraint.SourceVertex);
		Hash = HashCombineFast(Hash, ::GetTypeHash(Constraint.TargetFace));
		Hash = HashCombineFast(Hash, ::GetTypeHash((uint8)Constraint.IntersectionType));
		return Hash;
	}

	static void AppendConstraintsSourceToClosestTarget(const TSet<int32>& SourceVertices, const TSet<int32>& TargetFaces, const FCollectionClothConstFacade& Cloth, TSet<FVertexFaceConstraint>& Constraints)
	{
		using namespace UE::Geometry;
		TConstArrayView<FIntVector3> Elements = Cloth.GetSimIndices3D();
		TConstArrayView<FVector3f> Positions = Cloth.GetSimPosition3D();

		FFaceSetMeshAdapter MeshAdapter(TargetFaces, Cloth);
		TMeshAABBTree3<FFaceSetMeshAdapter> Tree(&MeshAdapter);


		for (const int32 SourceVertex : SourceVertices)
		{
			if (!Positions.IsValidIndex(SourceVertex))
			{
				continue;
			}

			double NearestDistSq;
			const int32 HitFace = Tree.FindNearestTriangle(FVector3d(Positions[SourceVertex]), NearestDistSq,
				IMeshSpatial::FQueryOptions([&Elements, SourceVertex](int32 Element)
					{
						if (Elements[Element][0] == SourceVertex ||
							Elements[Element][1] == SourceVertex ||
							Elements[Element][2] == SourceVertex)
						{
							return false;
						}
						return true;
					}));


			if (HitFace != IndexConstants::InvalidID)
			{
				check(TargetFaces.Contains(HitFace));
				Constraints.Add({ SourceVertex, HitFace, FVertexFaceConstraint::EIntersectionType::Closest });
			}
		}
	}

	static void AppendConstraintsSourceToRayIntersectionTarget(const TSet<int32>& SourceVertices, const TSet<int32>& TargetFaces, bool bFlipRay, float MaxRadius, const FCollectionClothConstFacade& Cloth, TSet<FVertexFaceConstraint>& Constraints)
	{
		using namespace UE::Geometry;
		TConstArrayView<FIntVector3> Elements = Cloth.GetSimIndices3D();
		TConstArrayView<FVector3f> Positions = Cloth.GetSimPosition3D();
		TConstArrayView<FVector3f> Normals = Cloth.GetSimNormal();

		FFaceSetMeshAdapter MeshAdapter(TargetFaces, Cloth);
		TMeshAABBTree3<FFaceSetMeshAdapter> Tree(&MeshAdapter);


		for (const int32 SourceVertex : SourceVertices)
		{
			if (!Positions.IsValidIndex(SourceVertex))
			{
				continue;
			}

			const FRay3d Ray(FVector3d(Positions[SourceVertex]), FVector3d(bFlipRay ? -Normals[SourceVertex] : Normals[SourceVertex]));
			const int32 HitFace = Tree.FindNearestHitTriangle(Ray,
				IMeshSpatial::FQueryOptions(MaxRadius, [&Elements, SourceVertex](int32 Element)
					{
						if (Elements[Element][0] == SourceVertex ||
							Elements[Element][1] == SourceVertex ||
							Elements[Element][2] == SourceVertex)
						{
							return false;
						}
						return true;
					}));


			if (HitFace != IndexConstants::InvalidID)
			{
				check(TargetFaces.Contains(HitFace));
				Constraints.Add({ SourceVertex, HitFace, bFlipRay ? FVertexFaceConstraint::EIntersectionType::RayFlipped : FVertexFaceConstraint::EIntersectionType::Ray });
			}
		}
	}

	static void AppendConstraintsAllWithinRadius(const TSet<int32>& SourceVertices, const TSet<int32>& TargetFaces, float Radius, int32 DisableNeighborDistance, const FCollectionClothConstFacade& Cloth, TSet<FVertexFaceConstraint>& Constraints)
	{
		using namespace UE::Geometry;

		TConstArrayView<FIntVector3> Elements = Cloth.GetSimIndices3D();
		TConstArrayView<FVector3f> Positions = Cloth.GetSimPosition3D();

		::Chaos::FTriangleMesh TriangleMesh;
		static_assert(sizeof(::Chaos::TVec3<int32>) == sizeof(FIntVector3));
		TConstArrayView<::Chaos::TVec3<int32>> ChaosElements(reinterpret_cast<const ::Chaos::TVec3<int32>*>(Elements.GetData()), Elements.Num());
		TriangleMesh.Init(ChaosElements, 0, Positions.Num(), false);

		FFaceSetMeshAdapter MeshAdapter(TargetFaces, Cloth);
		TMeshAABBTree3<FFaceSetMeshAdapter> Tree(&MeshAdapter);

		const float RadiusSq = Radius * Radius;

		for (const int32 SourceVertex : SourceVertices)
		{
			if (!Positions.IsValidIndex(SourceVertex))
			{
				continue;
			}
			const FVector3d Position(Positions[SourceVertex]);

			const TSet<int32> DisabledNeighbors = TriangleMesh.GetNRing(SourceVertex, DisableNeighborDistance);

			TMeshAABBTree3<FFaceSetMeshAdapter>::FTreeTraversal Traversal;
			Traversal.NextBoxF = [RadiusSq, &Position](const FAxisAlignedBox3d& Box, int32 Depth)
				{
					return Box.DistanceSquared(Position) < RadiusSq;
				};
			Traversal.NextTriangleF = [RadiusSq, &Positions, SourceVertex, &Constraints, &Elements](int32 TriangleID)
				{
					const FIntVector3& Element = Elements[TriangleID];
					FTriangle3f Triangle;
					Triangle.V[0] = Positions[Element[0]];
					Triangle.V[1] = Positions[Element[1]];
					Triangle.V[2] = Positions[Element[2]];

					FDistPoint3Triangle3f Dist(Positions[SourceVertex], Triangle);
					if (Dist.ComputeResult() < RadiusSq)
					{
						Constraints.Add({ SourceVertex, TriangleID, FVertexFaceConstraint::EIntersectionType::Closest });
					}
				};

			IMeshSpatial::FQueryOptions QueryOptions([&DisabledNeighbors, Elements](int32 Element)
				{
					if (DisabledNeighbors.Contains(Elements[Element][0]) ||
						DisabledNeighbors.Contains(Elements[Element][1]) ||
						DisabledNeighbors.Contains(Elements[Element][2]))
					{
						return false;
					}
					return true;
				});

			Tree.DoTraversal(Traversal, QueryOptions);
		}
	}


	static void AppendConstraintsTetrahedralize(const TSet<int32>& SourceVertices, 
		const TSet<int32>& TargetFaces, 
		float Radius, 
		int32 DisableNeighborDistance, 
		const FCollectionClothConstFacade& Cloth, 
		bool bSkipZeroVolumeTets,
		TSet<FVertexFaceConstraint>& Constraints)
	{
		// Speed up vertex-triangle queries by constructing this map up front
		TMap<int32, TArray<int32>> VertexToTriangleMap;
		TConstArrayView<FIntVector3> SimTris = Cloth.GetSimIndices3D();
		for (const int32& TriIndex : TargetFaces)
		{
			const FIntVector3& SimTri = SimTris[TriIndex];

			for (int32 TriCornerIndex = 0; TriCornerIndex < 3; ++TriCornerIndex)
			{
				const int32 VertexIndex = SimTri[TriCornerIndex];
				if (!VertexToTriangleMap.Find(VertexIndex))
				{
					VertexToTriangleMap.Add(VertexIndex);
				}
				VertexToTriangleMap[VertexIndex].Add(TriIndex);
			}
		}

		// Check if the three input vertices correspond to a single triangle (with three distinct vertices) on the Cloth sim mesh
		auto IsValidSimTri = [](const FIntVector3& TestIndices, const TMap<int32, TArray<int32>>& VertexToTriangleMap, const FCollectionClothConstFacade& Cloth, int32& OutClothTriIndex)
		{
			if (TestIndices[0] == TestIndices[1] || TestIndices[0] == TestIndices[2] || TestIndices[1] == TestIndices[2])
			{
				return false;
			}

			// Check if the test triangle vertex is in the vertex-to-triangle map. 
			// If not, it means no triangles containing this vertex are in the TargetFaces set, which means our test triangle is not in the TargetFaces set either.
			if (!VertexToTriangleMap.Find(TestIndices[0]))
			{
				return false;
			}

			// Get triangles to check -- we'll use triangles incident on the first vertex
			const TArray<int32> CandidateTriangles = VertexToTriangleMap[TestIndices[0]];

			for (const int32 CandidateTriIndex : CandidateTriangles)
			{
				const FIntVector3& Tri = Cloth.GetSimIndices3D()[CandidateTriIndex];

				if (Tri[0] == Tri[1] || Tri[0] == Tri[2] || Tri[1] == Tri[2])
				{
					continue;
				}

				if ((Tri[0] == TestIndices[0] || Tri[0] == TestIndices[1] || Tri[0] == TestIndices[2]) &&
					(Tri[1] == TestIndices[0] || Tri[1] == TestIndices[1] || Tri[1] == TestIndices[2]) &&
					(Tri[2] == TestIndices[0] || Tri[2] == TestIndices[1] || Tri[2] == TestIndices[2]))
				{
					OutClothTriIndex = CandidateTriIndex;
					return true;
				}
			}

			return false;
		};


		const FFaceSetMeshAdapter MeshAdapter(TargetFaces, Cloth);

		// Throw all source vertices and target triangle vertices into the tet mesher
		TSet<int32> AllVertexIndices(SourceVertices);
		for (const int32 TargetFace : TargetFaces)
		{
			if (MeshAdapter.IsTriangle(TargetFace))
			{
				const UE::Geometry::FIndex3i Triangle = MeshAdapter.GetTriangle(TargetFace);
				AllVertexIndices.Add(Triangle[0]);
				AllVertexIndices.Add(Triangle[1]);
				AllVertexIndices.Add(Triangle[2]);
			}
		}
		TConstArrayView<FVector3f> Positions = Cloth.GetSimPosition3D();
		TArray<FVector3f> TetInputPoints;
		for (const int32 VertexIndex : AllVertexIndices)
		{
			check(Positions.IsValidIndex(VertexIndex));
			TetInputPoints.Add(Positions[VertexIndex]);
		}

		// Compute the tetrahedral mesh
		UE::Geometry::FDelaunay3 Delaunay;
		if (Delaunay.Triangulate(TetInputPoints))
		{
			const TArray<FIntVector4> Tets = Delaunay.GetTetrahedra();
			for (const FIntVector4& Tet : Tets)
			{
				for (int32 TetFaceIndex = 0; TetFaceIndex < 4; ++TetFaceIndex)
				{
					// For each tet face, if it corresponds to a triangle in the sim mesh, use it to create a constraint
					const FIntVector3 TetFace(Tet[TetFaceIndex], Tet[(TetFaceIndex + 1) % 4], Tet[(TetFaceIndex + 2) % 4]);
					int32 SimMeshTriIndex = -1;
					if (IsValidSimTri(TetFace, VertexToTriangleMap, Cloth, SimMeshTriIndex))
					{
						const int32 VertIndex = Tet[(TetFaceIndex + 3) % 4];
						check(0 <= VertIndex && VertIndex < Cloth.GetNumSimVertices3D());
						check(0 <= SimMeshTriIndex && SimMeshTriIndex < Cloth.GetNumSimFaces());

						const FIntVector3 SimMeshTri = Cloth.GetSimIndices3D()[SimMeshTriIndex];
						check(VertIndex != SimMeshTri[0] && VertIndex != SimMeshTri[1] && VertIndex != SimMeshTri[2]);

						// Only take constraints where the vertex is in the set of source vertices and triangle is in the set of target triangles
						if (!SourceVertices.Contains(VertIndex) || !TargetFaces.Contains(SimMeshTriIndex))
						{
							continue;
						}

						// Check if the vertex is coplanar to the triangle and skip this contraint if it is
						if (bSkipZeroVolumeTets)
						{
							const FVector3f VertPos = Cloth.GetSimPosition3D()[VertIndex];
							const FVector3f A = Cloth.GetSimPosition3D()[SimMeshTri[0]] - VertPos;
							const FVector3f B = Cloth.GetSimPosition3D()[SimMeshTri[1]] - VertPos;
							const FVector3f C = Cloth.GetSimPosition3D()[SimMeshTri[2]] - VertPos;
							const float Vol = FVector3f::DotProduct(A, FVector3f::CrossProduct(B, C));
							if (FMath::Abs(Vol) < UE_KINDA_SMALL_NUMBER)
							{
								continue;
							}
						}

						Constraints.Add({ VertIndex, SimMeshTriIndex, FVertexFaceConstraint::EIntersectionType::Closest });
					}
				}
			}
		}

	}


	static void StoreConstraintsEnds(const TSet<FVertexFaceConstraint>& Constraints, const FCollectionClothConstFacade& Cloth, TArray<int32>& SourceVertices, TArray<FIntVector>& TargetVertices, TArray<FVector3f>& TargetWeights)
	{
		using namespace UE::Geometry;

		TConstArrayView<FIntVector3> Elements = Cloth.GetSimIndices3D();
		TConstArrayView<FVector3f> Positions = Cloth.GetSimPosition3D();
		TConstArrayView<FVector3f> Normals = Cloth.GetSimNormal();
		const int32 NumConstraints = Constraints.Num();
		SourceVertices.Reset(NumConstraints);
		TargetVertices.Reset(NumConstraints);
		TargetWeights.Reset(NumConstraints);

		for (const FVertexFaceConstraint& Constraint : Constraints)
		{
			FIntVector3 Element = Elements[Constraint.TargetFace];

			FTriangle3f Triangle;
			Triangle.V[0] = Positions[Element[0]];
			Triangle.V[1] = Positions[Element[1]];
			Triangle.V[2] = Positions[Element[2]];

			const FVector3f ChaosNormal = -Triangle.Normal(); // Chaos triangles follow right-hand convention rather than UE left-hand.

			switch (Constraint.IntersectionType)
			{
			case FVertexFaceConstraint::EIntersectionType::Closest:
			{
				FDistPoint3Triangle3f Dist(Positions[Constraint.SourceVertex], Triangle);
				const float Distance = FMath::Sqrt(Dist.ComputeResult());

				FVector3f TargetWeight = Dist.TriangleBaryCoords;

				// Order Element so that Source Vertex is always on the Normal side of the triangle for repulsion constraints.
				if (FVector3f::DotProduct(Positions[Constraint.SourceVertex] - Dist.ClosestTrianglePoint, ChaosNormal) < 0.f)
				{
					Swap(Element[0], Element[1]);
					Swap(TargetWeight[0], TargetWeight[1]);
				}

				SourceVertices.Add(Constraint.SourceVertex);
				TargetVertices.Add(Element);
				TargetWeights.Add(TargetWeight);
			}
			break;
			case FVertexFaceConstraint::EIntersectionType::Ray: // fallthrough
			case FVertexFaceConstraint::EIntersectionType::RayFlipped:
			{
				const bool bFlipRay = Constraint.IntersectionType == FVertexFaceConstraint::EIntersectionType::RayFlipped;
				const FRay3f Ray(Positions[Constraint.SourceVertex], bFlipRay ? -Normals[Constraint.SourceVertex] : Normals[Constraint.SourceVertex]);
				FIntrRay3Triangle3f Intr(Ray, Triangle);
				Intr.Find();
				check(Intr.IntersectionType == EIntersectionType::Point);

				FVector3f TargetWeight = FVector3f(Intr.TriangleBaryCoords);
				const FVector3f IntersectionPoint = Triangle.BarycentricPoint(TargetWeight);

				// Order Element so that Source Vertex is always on the Normal side of the triangle for repulsion constraints.
				if (FVector3f::DotProduct(Positions[Constraint.SourceVertex] - IntersectionPoint, ChaosNormal) < 0.f)
				{
					Swap(Element[0], Element[1]);
					Swap(TargetWeight[0], TargetWeight[1]);
				}
				SourceVertices.Add(Constraint.SourceVertex);
				TargetVertices.Add(Element);
				TargetWeights.Add(TargetWeight);
			}
			break;
			default:
				checkNoEntry();
			}
		}
	}

	static void CalculateRestLengthsFromPositions(const FCollectionClothConstFacade& Cloth, const TArray<int32>& SourceVertices, const TArray<FIntVector>& TargetVertices, const TArray<FVector3f>& TargetWeights, const float RestLengthScale, TArray<float>& RestLengths)
	{
		TConstArrayView<FVector3f> Positions = Cloth.GetSimPosition3D();
		const int32 NumConstraints = SourceVertices.Num();
		check(TargetVertices.Num() == NumConstraints);
		check(TargetWeights.Num() == NumConstraints);
		RestLengths.SetNumUninitialized(NumConstraints);

		for (int32 ConstraintIdx = 0; ConstraintIdx < NumConstraints; ++ConstraintIdx)
		{
			const FVector3f& SourcePosition = Positions[SourceVertices[ConstraintIdx]];
			const FVector3f TargetPosition = Positions[TargetVertices[ConstraintIdx][0]] * TargetWeights[ConstraintIdx][0] +
				Positions[TargetVertices[ConstraintIdx][1]] * TargetWeights[ConstraintIdx][1] + Positions[TargetVertices[ConstraintIdx][2]] * TargetWeights[ConstraintIdx][2];
			RestLengths[ConstraintIdx] = FVector3f::Dist(SourcePosition, TargetPosition) * RestLengthScale;
		}
	}

	static void CalculateRestLengthsFromThickness(const ::Chaos::Softs::FPBDFlatWeightMapView& ThicknessView, const TArray<int32>& SourceVertices, const TArray<FIntVector>& TargetVertices, const TArray<FVector3f>& TargetWeights, TArray<float>& RestLengths)
	{
		const int32 NumConstraints = SourceVertices.Num();
		check(TargetVertices.Num() == NumConstraints);
		check(TargetWeights.Num() == NumConstraints);
		if (ThicknessView.HasWeightMap())
		{
			RestLengths.SetNumUninitialized(NumConstraints);

			for (int32 ConstraintIdx = 0; ConstraintIdx < NumConstraints; ++ConstraintIdx)
			{
				const float SourceThickness = ThicknessView.GetValue(SourceVertices[ConstraintIdx]);
				const float TargetThickness = ThicknessView.GetValue(TargetVertices[ConstraintIdx][0]) * TargetWeights[ConstraintIdx][0] +
					ThicknessView.GetValue(TargetVertices[ConstraintIdx][1]) * TargetWeights[ConstraintIdx][1] + ThicknessView.GetValue(TargetVertices[ConstraintIdx][2]) * TargetWeights[ConstraintIdx][2];
				RestLengths[ConstraintIdx] = SourceThickness + TargetThickness;
			}
		}
		else
		{
			RestLengths.Init(2.f * ThicknessView.GetLow(), NumConstraints);
		}
	}
}

FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::FChaosClothAssetSimulationClothVertexFaceSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
	, GenerateConstraints(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				CreateConstraints(Context);
			}))
{
	RegisterCollectionConnections();
	RegisterInputConnection(&Thickness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	// Start with one set of option pins.
	for (int32 Index = 0; Index < NumInitialConstructionSets; ++Index)
	{
		AddPins();
	}
	check(GetNumInputs() == NumRequiredInputs + NumInitialConstructionSets * 2); // Update NumRequiredInputs if you add more Inputs. This is used by Serialize.
}


TArray<UE::Dataflow::FPin> FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::AddPins()
{
	const int32 Index = ConstructionSets.AddDefaulted();
	TArray<UE::Dataflow::FPin> Pins;
	Pins.Reserve(2);
	{
		const FDataflowInput& Input = RegisterInputArrayConnection(GetSourceConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() });
	}
	{
		const FDataflowInput& Input = RegisterInputArrayConnection(GetTargetConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() });
	}
	return Pins;
}

TArray<UE::Dataflow::FPin> FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::GetPinsToRemove() const
{
	const int32 Index = ConstructionSets.Num() - 1;
	check(ConstructionSets.IsValidIndex(Index));
	TArray<UE::Dataflow::FPin> Pins;
	Pins.Reserve(2);
	if (const FDataflowInput* const Input = FindInput(GetSourceConnectionReference(Index)))
	{
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() });
	}
	if (const FDataflowInput* const Input = FindInput(GetTargetConnectionReference(Index)))
	{
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() });
	}
	return Pins;
}

void FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = ConstructionSets.Num() - 1;
	check(ConstructionSets.IsValidIndex(Index));
	const FDataflowInput* const FirstInput = FindInput(GetSourceConnectionReference(Index));
	const FDataflowInput* const SecondInput = FindInput(GetTargetConnectionReference(Index));
	check(FirstInput || SecondInput);
	const bool bIsFirstInput = FirstInput && FirstInput->GetName() == Pin.Name;
	const bool bIsSecondInput = SecondInput && SecondInput->GetName() == Pin.Name;
	if ((bIsFirstInput && !SecondInput) || (bIsSecondInput && !FirstInput))
	{
		// Both inputs removed. Remove array index.
		ConstructionSets.SetNum(Index);
	}
	return Super::OnPinRemoved(Pin);
}

void FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::PostSerialize(const FArchive& Ar)
{
	// Restore the pins when re-loading so they can get properly reconnected
	if (Ar.IsLoading())
	{
		check(ConstructionSets.Num() >= NumInitialConstructionSets);
		for (int32 Index = 0; Index < NumInitialConstructionSets; ++Index)
		{
			check(FindInput(GetSourceConnectionReference(Index)));
			check(FindInput(GetTargetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialConstructionSets; Index < ConstructionSets.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetSourceConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
			FindOrRegisterInputArrayConnection(GetTargetConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		}

		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialConstructionSets * 2);
			const int32 OrigNumConstructionSets = ConstructionSets.Num();
			const int32 OrigNumRegisteredConstructionSets = (OrigNumRegisteredInputs - NumRequiredInputs) / 2;

			if (OrigNumRegisteredConstructionSets > OrigNumConstructionSets)
			{
				ensure(Ar.IsTransacting());
				// Temporarily expand ConstructionSets so we can get connection references.
				ConstructionSets.SetNum(OrigNumRegisteredConstructionSets);
				for (int32 Index = OrigNumConstructionSets; Index < ConstructionSets.Num(); ++Index)
				{
					UnregisterInputConnection(GetTargetConnectionReference(Index));
					UnregisterInputConnection(GetSourceConnectionReference(Index));
				}
				ConstructionSets.SetNum(OrigNumConstructionSets);
			}
		}
		else
		{
			ensureAlways(ConstructionSets.Num() * 2 + NumRequiredInputs == GetNumInputs());
		}
	}
}

UE::Dataflow::TConnectionReference<FString> FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::GetSourceConnectionReference(int32 Index) const
{
	return { &ConstructionSets[Index].SourceVertexSelection.StringValue, Index, &ConstructionSets };
}

UE::Dataflow::TConnectionReference<FString> FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::GetTargetConnectionReference(int32 Index) const
{
	return { &ConstructionSets[Index].TargetFaceSelection.StringValue, Index, &ConstructionSets };
}

void FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	if (!bAppendToExisting)
	{
		if (bUseTetRepulsionConstraints)
		{
			PropertyHelper.SetProperty(this, &VertexFaceRepulsionStiffness, {}, Chaos::Softs::ECollectionPropertyFlags::Animatable);
			PropertyHelper.SetProperty(this, &VertexFaceMaxRepulsionIters, {}, Chaos::Softs::ECollectionPropertyFlags::Animatable);
		}
		else
		{
			PropertyHelper.SetPropertyWeighted(this, &VertexFaceSpringExtensionStiffness, {}, Chaos::Softs::ECollectionPropertyFlags::Animatable);
			PropertyHelper.SetPropertyWeighted(this, &VertexFaceSpringCompressionStiffness, {}, Chaos::Softs::ECollectionPropertyFlags::Animatable);
			PropertyHelper.SetPropertyWeighted(this, &VertexFaceSpringDamping, {}, Chaos::Softs::ECollectionPropertyFlags::Animatable);
		}
	}
}

void FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const
{
	using namespace UE::Chaos::ClothAsset;
	Chaos::Softs::FEmbeddedSpringFacade SpringFacade(ClothCollection.Get(), ClothCollectionGroup::SimVertices3D);

	auto CheckVertexIndicesValidity = [](const TArray<TArray<int32>>& IndicesArray, const int32 VertexCount) -> bool
		{
			for (const TArray<int32>& Indices : IndicesArray)
			{
				for (const int32& Index : Indices)
				{
					if (Index < 0 || Index >= VertexCount)
					{
						return false;
					}
				}
			}
			return true;
		};

	FCollectionClothFacade ClothFacade(ClothCollection);
	if (ClothFacade.IsValid() && SpringFacade.IsValid())
	{
		const int32 NumConstraints = FMath::Min(SourceVertices.Num(), FMath::Min(TargetVertices.Num(), FMath::Min(TargetWeights.Num(), RestLengths.Num())));
		TArray<TArray<int32>> SourceIndicesArray;
		TArray<TArray<float>> SourceWeightsArray;
		TArray<TArray<int32>> TargetIndicesArray;
		TArray<TArray<float>> TargetWeightsArray;
		SourceIndicesArray.SetNum(NumConstraints);
		SourceWeightsArray.Init(TArray<float>({ 1.f }), NumConstraints);
		TargetIndicesArray.SetNum(NumConstraints);
		TargetWeightsArray.SetNum(NumConstraints);
		for (int32 Index = 0; Index < NumConstraints; ++Index)
		{
			SourceIndicesArray[Index] = { SourceVertices[Index] };
			TargetIndicesArray[Index] = { TargetVertices[Index][0], TargetVertices[Index][1], TargetVertices[Index][2] };
			TargetWeightsArray[Index] = { TargetWeights[Index][0], TargetWeights[Index][1], TargetWeights[Index][2] };
		}

		const bool bSourceIndicesAreValid = CheckVertexIndicesValidity(SourceIndicesArray, ClothFacade.GetNumSimVertices3D());
		if (!bSourceIndicesAreValid)
		{
			Context.Error(LOCTEXT("VertexFaceSpring_InvalidSourceIndices", "invalid source indices detected, ignoring constraints, please regenerate constraints as the topology of the sim mesh may have changed"), this);
			return;
		}

		const bool bTargetIndicesAreValid = CheckVertexIndicesValidity(TargetIndicesArray, ClothFacade.GetNumSimVertices3D());
		if (!bTargetIndicesAreValid)
		{
			Context.Error(LOCTEXT("VertexFaceSpring_InvalidTargetIndices", "invalid target indices detected, ignoring constraints, please regenerate constraints as the topology of the sim mesh may have changed"), this);
			return;
		}

		const FString ConstraintName = bUseTetRepulsionConstraints ? TEXT("VertexFaceRepulsionConstraint") : TEXT("VertexFaceSpringConstraint");

		// Try to find existing constraint of this type.
		bool bFound = false;
		for (int32 ConstraintIndex = 0; ConstraintIndex < SpringFacade.GetNumSpringConstraints(); ++ConstraintIndex)
		{
			Chaos::Softs::FEmbeddedSpringConstraintFacade SpringConstraintFacade = SpringFacade.GetSpringConstraint(ConstraintIndex);
			if (SpringConstraintFacade.GetConstraintEndPointNumIndices() == FUintVector2(1, 3) && SpringConstraintFacade.GetConstraintName() == ConstraintName)
			{
				if (bAppendToExisting)
				{
					SpringConstraintFacade.Append(
						TConstArrayView<TArray<int32>>(SourceIndicesArray),
						TConstArrayView<TArray<float>>(SourceWeightsArray),
						TConstArrayView<TArray<int32>>(TargetIndicesArray),
						TConstArrayView<TArray<float>>(TargetWeightsArray),
						TConstArrayView<float>(RestLengths.GetData(), NumConstraints));
				}
				else
				{
					SpringConstraintFacade.Initialize(FUintVector2(1, 3),
						TConstArrayView<TArray<int32>>(SourceIndicesArray),
						TConstArrayView<TArray<float>>(SourceWeightsArray),
						TConstArrayView<TArray<int32>>(TargetIndicesArray),
						TConstArrayView<TArray<float>>(TargetWeightsArray),
						TConstArrayView<float>(RestLengths.GetData(), NumConstraints),
						TConstArrayView<float>(),
						TConstArrayView<float>(),
						TConstArrayView<float>(),
						ConstraintName
					);
				}
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			Chaos::Softs::FEmbeddedSpringConstraintFacade SpringConstraintFacade = SpringFacade.AddGetSpringConstraint();
			SpringConstraintFacade.Initialize(FUintVector2(1, 3),
				TConstArrayView<TArray<int32>>(SourceIndicesArray),
				TConstArrayView<TArray<float>>(SourceWeightsArray),
				TConstArrayView<TArray<int32>>(TargetIndicesArray),
				TConstArrayView<TArray<float>>(TargetWeightsArray),
				TConstArrayView<float>(RestLengths.GetData(), NumConstraints),
				TConstArrayView<float>(),
				TConstArrayView<float>(),
				TConstArrayView<float>(),
				ConstraintName
			);
		}
	}
}

TArray<FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::FConstructionSetData> FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::GetConstructionSetData(UE::Dataflow::FContext& Context) const
{
	TArray<FConstructionSetData> ConstructionSetData;
	ConstructionSetData.SetNumUninitialized(ConstructionSets.Num());
	for (int32 Index = 0; Index < ConstructionSets.Num(); ++Index)
	{
		ConstructionSetData[Index].SourceSetName = *GetValue(Context, GetSourceConnectionReference(Index));
		ConstructionSetData[Index].TargetSetName = *GetValue(Context, GetTargetConnectionReference(Index));
		ConstructionSetData[Index].ConstructionMethod = ConstructionSets[Index].ConstructionMethod;
		ConstructionSetData[Index].bFlipRayNormal = ConstructionSets[Index].bFlipRayNormal;
		ConstructionSetData[Index].MaxRayLength = ConstructionSets[Index].MaxRayLength;
		ConstructionSetData[Index].Radius = ConstructionSets[Index].Radius;
		ConstructionSetData[Index].DisableNeighborDistance = ConstructionSets[Index].DisableNeighborDistance;
		ConstructionSetData[Index].bSkipZeroVolumeTets = ConstructionSets[Index].bSkipZeroVolumeTets;
	}
	return ConstructionSetData;
}

void FChaosClothAssetSimulationClothVertexFaceSpringConfigNode::CreateConstraints(UE::Dataflow::FContext& Context)
{
	using namespace UE::Chaos::ClothAsset;
	FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

	FCollectionClothConstFacade ClothFacade(ClothCollection);
	FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);
	if (ClothFacade.IsValid() && SelectionFacade.IsValid())
	{
		TSet<Private::FVertexFaceConstraint> Constraints;
		const TArray<FConstructionSetData> ConstructionSetData = GetConstructionSetData(Context);
		const TConstArrayView<FVector3f> Positions = ClothFacade.GetSimPosition3D();
		for (const FConstructionSetData& Data : ConstructionSetData)
		{
			TSet<int32> SourceSet;
			TSet<int32> TargetSet;
			if (FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, Data.SourceSetName, ClothCollectionGroup::SimVertices3D, SourceSet)
				&& FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, Data.TargetSetName, ClothCollectionGroup::SimFaces, TargetSet))
			{
				if (SourceSet.IsEmpty() || TargetSet.IsEmpty())
				{
					continue;
				}

				switch (Data.ConstructionMethod)
				{
				case EChaosClothAssetClothVertexFaceSpringConstructionMethod::SourceToClosestTarget:
					Private::AppendConstraintsSourceToClosestTarget(SourceSet, TargetSet, ClothFacade, Constraints);
					break;

				case EChaosClothAssetClothVertexFaceSpringConstructionMethod::SourceToRayIntersectionTarget:
					Private::AppendConstraintsSourceToRayIntersectionTarget(SourceSet, TargetSet, Data.bFlipRayNormal, Data.MaxRayLength, ClothFacade, Constraints);
					break;
				case EChaosClothAssetClothVertexFaceSpringConstructionMethod::AllWithinRadius:
					Private::AppendConstraintsAllWithinRadius(SourceSet, TargetSet, Data.Radius, Data.DisableNeighborDistance, ClothFacade, Constraints);
					break;
				case EChaosClothAssetClothVertexFaceSpringConstructionMethod::Tetrahedralize:
					Private::AppendConstraintsTetrahedralize(SourceSet, TargetSet, Data.Radius, Data.DisableNeighborDistance, ClothFacade, Data.bSkipZeroVolumeTets, Constraints);
					break;
				default:
					checkNoEntry();
				}
			}
		}

		Private::StoreConstraintsEnds(Constraints, ClothFacade, SourceVertices, TargetVertices, TargetWeights);
		if (bUseThicknessMap)
		{
			TConstArrayView<float> ThicknessMap = ClothFacade.GetWeightMap(FName(*GetValue(Context, &Thickness.WeightMap)));
			const Chaos::Softs::FPBDFlatWeightMapView ThicknessView(Chaos::Softs::FSolverVec2(Thickness.Low, Thickness.High), ThicknessMap, ClothFacade.GetNumSimVertices3D());
			Private::CalculateRestLengthsFromThickness(ThicknessView, SourceVertices, TargetVertices, TargetWeights, RestLengths);
		}
		else
		{
			Private::CalculateRestLengthsFromPositions(ClothFacade, SourceVertices, TargetVertices, TargetWeights, RestLengthScale, RestLengths);
		}
	}
}

#undef LOCTEXT_NAMESPACE