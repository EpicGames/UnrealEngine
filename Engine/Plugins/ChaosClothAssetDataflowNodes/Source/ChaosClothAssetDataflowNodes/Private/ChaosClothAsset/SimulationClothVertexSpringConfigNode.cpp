// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationClothVertexSpringConfigNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Spatial/SparseDynamicPointOctree3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationClothVertexSpringConfigNode)


#define LOCTEXT_NAMESPACE "SimulationClothVertexSpringConfigNode"

namespace UE::Chaos::ClothAsset::Private
{
	static void AppendConstraintsSourceToClosestTarget(const TSet<int32>& SourceVertices, const TSet<int32>& TargetVertices, const TConstArrayView<FVector3f>& Positions, TSet<FIntVector2>& Constraints)
	{
		constexpr int32 MaxBruteForceTargetCount = 500;
		constexpr int32 MaxBruteForceCompareCount = 10000;
		if (TargetVertices.Num() <= MaxBruteForceTargetCount || SourceVertices.Num() * TargetVertices.Num() <= MaxBruteForceCompareCount)
		{
			// Do a brute force comparison for smaller numbers of points.
			for (const int32 SourceIndex : SourceVertices)
			{
				if (!Positions.IsValidIndex(SourceIndex))
				{
					continue;
				}
				const FVector3f& SourcePos = Positions[SourceIndex];
				int32 ClosestIndex = INDEX_NONE;
				float ClosestDistSq = UE_MAX_FLT;
				for (const int32 TargetIndex : TargetVertices)
				{
					if (SourceIndex != TargetIndex && Positions.IsValidIndex(TargetIndex))
					{
						const float DistSq = FVector3f::DistSquared(SourcePos, Positions[TargetIndex]);
						if (DistSq < ClosestDistSq)
						{
							ClosestDistSq = DistSq;
							ClosestIndex = TargetIndex;
						}
					}
				}
				if (ClosestIndex != INDEX_NONE)
				{
					Constraints.Add(SourceIndex < ClosestIndex ? FIntVector2(SourceIndex, ClosestIndex) : FIntVector2(ClosestIndex, SourceIndex));
				}				
			}
		}
		else
		{
			// Put target vertices in an acceleration structure for faster lookup.
			UE::Geometry::FSparseDynamicPointOctree3 Octree;
			UE::Geometry::FAxisAlignedBox3d BBox;
			for (const int32 TargetIndex : TargetVertices)
			{
				if (Positions.IsValidIndex(TargetIndex))
				{
					BBox.Contain(FVector3d(Positions[TargetIndex]));
				}
			}

			Octree.ConfigureFromPointCountEstimate(BBox.MaxDim(), TargetVertices.Num());

			for (const int32 TargetIndex : TargetVertices)
			{
				if (Positions.IsValidIndex(TargetIndex))
				{
					Octree.InsertPoint_DynamicExpand(TargetIndex,
						[&Positions](int32 Index)
						{
							return FVector3d(Positions[Index]);
						}
					);
				}
			}

			TArray<const UE::Geometry::FSparsePointOctreeCell*> Buffer;
			for (const int32 SourceIndex : SourceVertices)
			{
				if (!Positions.IsValidIndex(SourceIndex))
				{
					continue;
				}
				const FVector3f& SourcePos = Positions[SourceIndex];

				const int32 ClosestTargetIndex = Octree.FindClosestPoint(FVector3d(SourcePos), UE_BIG_NUMBER,
					[SourceIndex](int32 Index)
					{
						return Index != SourceIndex;
					},
					[&SourcePos, &Positions](int32 Index)
					{
						return (double)FVector3f::DistSquared(SourcePos, Positions[Index]);
					}, &Buffer);

				if (ensure(Positions.IsValidIndex(ClosestTargetIndex)))
				{
					Constraints.Add(SourceIndex < ClosestTargetIndex ? FIntVector2(SourceIndex, ClosestTargetIndex) : FIntVector2(ClosestTargetIndex, SourceIndex));
				}
			}
		}
	}

	static void AppendConstraintsSourceToAllTargets(const TSet<int32>& SourceVertices, const TSet<int32>& TargetVertices, const int32 NumPositions, TSet<FIntVector2>& Constraints)
	{
		auto IsValidIndex = [NumPositions](int32 Index)
			{
				return Index >= 0 && Index < NumPositions;
			};
		for (const int32 SourceIndex : SourceVertices)
		{
			for (const int32 TargetIndex : TargetVertices)
			{
				if (SourceIndex != TargetIndex && IsValidIndex(SourceIndex) && IsValidIndex(TargetIndex))
				{
					Constraints.Add(SourceIndex < TargetIndex ? FIntVector2(SourceIndex, TargetIndex) : FIntVector2(TargetIndex, SourceIndex));
				}
			}
		}
	}
}

FChaosClothAssetSimulationClothVertexSpringConfigNode::FChaosClothAssetSimulationClothVertexSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
	, GenerateConstraints(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				CreateConstraints(Context);
			}))
{
	RegisterCollectionConnections();
	// Start with one set of option pins.
	for (int32 Index = 0; Index < NumInitialConstructionSets; ++Index)
	{
		AddPins();
	}
	check(GetNumInputs() == NumRequiredInputs + NumInitialConstructionSets * 2); // Update NumRequiredInputs if you add more Inputs. This is used by Serialize.
}


TArray<UE::Dataflow::FPin> FChaosClothAssetSimulationClothVertexSpringConfigNode::AddPins()
{
	const int32 Index = ConstructionSets.AddDefaulted();
	TArray<UE::Dataflow::FPin> Pins;
	Pins.Reserve(2);
	{
		const FDataflowInput& Input = RegisterInputArrayConnection(GetSourceConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() });
	}
	{
		const FDataflowInput& Input = RegisterInputArrayConnection(GetTargetConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() });
	}
	return Pins;
}

TArray<UE::Dataflow::FPin> FChaosClothAssetSimulationClothVertexSpringConfigNode::GetPinsToRemove() const
{
	const int32 Index = ConstructionSets.Num() - 1;
	check(ConstructionSets.IsValidIndex(Index));
	TArray<UE::Dataflow::FPin> Pins;
	Pins.Reserve(2);
	if (const FDataflowInput* const Input = FindInput(GetSourceConnectionReference(Index)))
	{
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() });
	}
	if (const FDataflowInput* const Input = FindInput(GetTargetConnectionReference(Index)))
	{
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() });
	}
	return Pins;
}

void FChaosClothAssetSimulationClothVertexSpringConfigNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = ConstructionSets.Num() - 1;
	check(ConstructionSets.IsValidIndex(Index));
	const FDataflowInput* const FirstInput = FindInput(GetSourceConnectionReference(Index));
	const FDataflowInput* const SecondInput = FindInput(GetTargetConnectionReference(Index));
	check(FirstInput || SecondInput);
	const bool bIsFirstInput = FirstInput && FirstInput->GetName() == Pin.Name;
	const bool bIsSecondInput = SecondInput && SecondInput->GetName() == Pin.Name;
	if ((bIsFirstInput && !SecondInput) || (bIsSecondInput && !FirstInput))
	{
		// Both inputs removed. Remove array index.
		ConstructionSets.SetNum(Index);
	}
	return Super::OnPinRemoved(Pin);
}

void FChaosClothAssetSimulationClothVertexSpringConfigNode::PostSerialize(const FArchive& Ar)
{
	// Restore the pins when re-loading so they can get properly reconnected
	if (Ar.IsLoading())
	{
		check(ConstructionSets.Num() >= NumInitialConstructionSets);
		for (int32 Index = 0; Index < NumInitialConstructionSets; ++Index)
		{
			check(FindInput(GetSourceConnectionReference(Index)));
			check(FindInput(GetTargetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialConstructionSets; Index < ConstructionSets.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetSourceConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
			FindOrRegisterInputArrayConnection(GetTargetConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		}

		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialConstructionSets * 2);
			const int32 OrigNumConstructionSets = ConstructionSets.Num();
			const int32 OrigNumRegisteredConstructionSets = (OrigNumRegisteredInputs - NumRequiredInputs) / 2;

			if (OrigNumRegisteredConstructionSets > OrigNumConstructionSets)
			{
				ensure(Ar.IsTransacting());
				// Temporarily expand ConstructionSets so we can get connection references.
				ConstructionSets.SetNum(OrigNumRegisteredConstructionSets);
				for (int32 Index = OrigNumConstructionSets; Index < ConstructionSets.Num(); ++Index)
				{
					UnregisterInputConnection(GetTargetConnectionReference(Index));
					UnregisterInputConnection(GetSourceConnectionReference(Index));
				}
				ConstructionSets.SetNum(OrigNumConstructionSets);
			}
		}
		else
		{
			ensureAlways(ConstructionSets.Num() * 2 + NumRequiredInputs == GetNumInputs());
		}
	}
}

UE::Dataflow::TConnectionReference<FString> FChaosClothAssetSimulationClothVertexSpringConfigNode::GetSourceConnectionReference(int32 Index) const
{
	return { &ConstructionSets[Index].SourceVertexSelection.StringValue, Index, &ConstructionSets };
}

UE::Dataflow::TConnectionReference<FString> FChaosClothAssetSimulationClothVertexSpringConfigNode::GetTargetConnectionReference(int32 Index) const
{
	return { &ConstructionSets[Index].TargetVertexSelection.StringValue, Index, &ConstructionSets };
}

void FChaosClothAssetSimulationClothVertexSpringConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	if (!bAppendToExisting)
	{
		PropertyHelper.SetPropertyWeighted(this, &VertexSpringExtensionStiffness, {}, Chaos::Softs::ECollectionPropertyFlags::Animatable);
		PropertyHelper.SetPropertyWeighted(this, &VertexSpringCompressionStiffness, {}, Chaos::Softs::ECollectionPropertyFlags::Animatable);
		PropertyHelper.SetPropertyWeighted(this, &VertexSpringDamping, {}, Chaos::Softs::ECollectionPropertyFlags::Animatable);
	}
}

void FChaosClothAssetSimulationClothVertexSpringConfigNode::EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const
{
	using namespace UE::Chaos::ClothAsset;
	Chaos::Softs::FEmbeddedSpringFacade SpringFacade(ClothCollection.Get(), ClothCollectionGroup::SimVertices3D);

	auto CheckVertexIndicesValidity = [](const TArray<FIntVector2>& IndexPairs, int32 VertexCount)
		{
			for (const FIntVector2& IndexPair : IndexPairs)
			{
				if (IndexPair[0] < 0 || IndexPair[0] >= VertexCount)
				{
					return false;
				}
				if (IndexPair[1] < 0 || IndexPair[1] >= VertexCount)
				{
					return false;
				}
			}
			return true;
		};

	FCollectionClothFacade ClothFacade(ClothCollection);
	if (ClothFacade.IsValid() && SpringFacade.IsValid())
	{
		const bool bVertexIndicesAreValid = CheckVertexIndicesValidity(ConstraintVertices, ClothFacade.GetNumSimVertices3D());
		if (!bVertexIndicesAreValid)
		{
			Context.Error(LOCTEXT("VertexSpring_InvalidVertexIndices", "invalid vertex indices detected, ignoring constraints, please regenerate constraints as the topology of the sim mesh may have changed"), this);
			return;
		}

		const int32 NumConstraints = FMath::Min(ConstraintVertices.Num(), RestLengths.Num());

		// Try to find existing constraint of this type.
		bool bFound = false;
		for (int32 ConstraintIndex = 0; ConstraintIndex < SpringFacade.GetNumSpringConstraints(); ++ConstraintIndex)
		{
			Chaos::Softs::FEmbeddedSpringConstraintFacade SpringConstraintFacade = SpringFacade.GetSpringConstraint(ConstraintIndex);
			if (SpringConstraintFacade.GetConstraintEndPointNumIndices() == FUintVector2(1, 1) && SpringConstraintFacade.GetConstraintName() == TEXT("VertexSpringConstraint"))
			{
				if (bAppendToExisting)
				{
					SpringConstraintFacade.Append(
						TConstArrayView<FIntVector2>(ConstraintVertices.GetData(), NumConstraints),
						TConstArrayView<float>(RestLengths.GetData(), NumConstraints));
				}
				else
				{
					SpringConstraintFacade.Initialize(
						TConstArrayView<FIntVector2>(ConstraintVertices.GetData(), NumConstraints), 
						TConstArrayView<float>(RestLengths.GetData(), NumConstraints),
						TConstArrayView<float>(),
						TConstArrayView<float>(),
						TConstArrayView<float>(),
						TEXT("VertexSpringConstraint")
						);
				}
				bFound = true;
				break;
			}
		}
		if (!bFound)
		{
			Chaos::Softs::FEmbeddedSpringConstraintFacade SpringConstraintFacade = SpringFacade.AddGetSpringConstraint();
			SpringConstraintFacade.Initialize(
				TConstArrayView<FIntVector2>(ConstraintVertices.GetData(), NumConstraints),
				TConstArrayView<float>(RestLengths.GetData(), NumConstraints),
				TConstArrayView<float>(),
				TConstArrayView<float>(),
				TConstArrayView<float>(),
				TEXT("VertexSpringConstraint")
			);
		}
	}
}

TArray<FChaosClothAssetSimulationClothVertexSpringConfigNode::FConstructionSetData> FChaosClothAssetSimulationClothVertexSpringConfigNode::GetConstructionSetData(UE::Dataflow::FContext& Context) const
{
	TArray<FConstructionSetData> ConstructionSetData;
	ConstructionSetData.SetNumUninitialized(ConstructionSets.Num());
	for (int32 Index = 0; Index < ConstructionSets.Num(); ++Index)
	{
		ConstructionSetData[Index].SourceSetName = *GetValue(Context, GetSourceConnectionReference(Index));
		ConstructionSetData[Index].TargetSetName = *GetValue(Context, GetTargetConnectionReference(Index));
		ConstructionSetData[Index].ConstructionMethod = ConstructionSets[Index].ConstructionMethod;
	}
	return ConstructionSetData;
}

void FChaosClothAssetSimulationClothVertexSpringConfigNode::CreateConstraints(UE::Dataflow::FContext& Context)
{
	using namespace UE::Chaos::ClothAsset;
	FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

	FCollectionClothConstFacade ClothFacade(ClothCollection);
	FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);
	if (ClothFacade.IsValid() && SelectionFacade.IsValid())
	{
		TSet<FIntVector2> Constraints;
		const TArray<FConstructionSetData> ConstructionSetData = GetConstructionSetData(Context);
		const TConstArrayView<FVector3f> Positions = ClothFacade.GetSimPosition3D();
		for (const FConstructionSetData& Data : ConstructionSetData)
		{
			TSet<int32> SourceSet;
			TSet<int32> TargetSet;
			if (FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, Data.SourceSetName, ClothCollectionGroup::SimVertices3D, SourceSet)
				&& FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, Data.TargetSetName, ClothCollectionGroup::SimVertices3D, TargetSet))
			{
				switch (Data.ConstructionMethod)
				{
				case EChaosClothAssetClothVertexSpringConstructionMethod::SourceToClosestTarget:
					Private::AppendConstraintsSourceToClosestTarget(SourceSet, TargetSet, Positions, Constraints);
					break;

				case EChaosClothAssetClothVertexSpringConstructionMethod::ClosestSourceToClosestTarget:
					Private::AppendConstraintsSourceToClosestTarget(SourceSet, TargetSet, Positions, Constraints);
					Private::AppendConstraintsSourceToClosestTarget(TargetSet, SourceSet, Positions, Constraints);
					break;

				case EChaosClothAssetClothVertexSpringConstructionMethod::AllSourceToAllTargets:
					Private::AppendConstraintsSourceToAllTargets(SourceSet, TargetSet, Positions.Num(), Constraints);
					break;

				default:
					checkNoEntry();
				}
			}
		}

		ConstraintVertices = Constraints.Array();
		RestLengths.SetNumUninitialized(ConstraintVertices.Num());
		for (int32 Index = 0; Index < ConstraintVertices.Num(); ++Index)
		{
			RestLengths[Index] = FVector3f::Dist(Positions[ConstraintVertices[Index][0]], Positions[ConstraintVertices[Index][1]]);
		}
	}
}

#undef LOCTEXT_NAMESPACE 
