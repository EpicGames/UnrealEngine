// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationCollisionConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationCollisionConfigNode)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FChaosClothAssetSimulationCollisionConfigNode::FChaosClothAssetSimulationCollisionConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&FrictionCoefficientWeighted.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&ClothCollisionThicknessImported.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&InnerCollisionThickness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FChaosClothAssetSimulationCollisionConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyBool(this, &bUseCCD);
	PropertyHelper.SetPropertyWeighted(this, &InnerCollisionThickness);
	
	PropertyHelper.SetFabricProperty(FName(TEXT("CollisionThickness")), CollisionThicknessImported,
		[](UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
		{
			return FabricFacade.GetCollisionThickness();
		}, {});

	PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("FrictionCoefficient")), FrictionCoefficientWeighted,
		[](const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
		{
			return FabricFacade.GetFriction();
		}, {});

	PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("ClothCollisionThickness")), ClothCollisionThicknessImported,
		[](const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
		{
			return FabricFacade.GetCollisionThickness();
		}, {});

	PropertyHelper.SetPropertyBool(this, &bEnableSimpleColliders);
	PropertyHelper.SetPropertyBool(this, &bUsePlanarConstraintForSimpleColliders);
	PropertyHelper.SetPropertyBool(this, &bEnableComplexColliders);
	PropertyHelper.SetPropertyBool(this, &bUsePlanarConstraintForComplexColliders);
	PropertyHelper.SetPropertyBool(this, &bEnableSkinnedTriangleMeshCollisions);
	PropertyHelper.SetPropertyBool(this, &bUseSelfCollisionSubstepsForSkinnedTriangleMeshes);
}

void FChaosClothAssetSimulationCollisionConfigNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsLoading())
	{
#if WITH_EDITORONLY_DATA
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (FrictionCoefficient_DEPRECATED != FrictionCoefficientDeprecatedDefault)
		{
			FrictionCoefficientImported.ImportedValue = FrictionCoefficient_DEPRECATED;
			FrictionCoefficient_DEPRECATED = FrictionCoefficientDeprecatedDefault;
		}
		if (CollisionThickness_DEPRECATED != CollisionThicknessDeprecatedDefault)
		{
			CollisionThicknessImported.ImportedValue = CollisionThickness_DEPRECATED;
			CollisionThickness_DEPRECATED = CollisionThicknessDeprecatedDefault;
		}
		if (FrictionCoefficientImported.ImportedValue != UE::Chaos::ClothAsset::FDefaultFabric::Friction)
		{
			FrictionCoefficientWeighted.Low = FrictionCoefficientWeighted.High = FrictionCoefficientImported.ImportedValue;
			FrictionCoefficientImported.ImportedValue = UE::Chaos::ClothAsset::FDefaultFabric::Friction;
		}
		if (FrictionCoefficientImported.bUseImportedValue)
		{
			FrictionCoefficientWeighted.bImportFabricBounds = true;
			FrictionCoefficientImported.bUseImportedValue = false;
		}
		
		// Intentionally not upgrading ClothCollisionThickness. This property was using the incorrect string and was unused.
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}
}

