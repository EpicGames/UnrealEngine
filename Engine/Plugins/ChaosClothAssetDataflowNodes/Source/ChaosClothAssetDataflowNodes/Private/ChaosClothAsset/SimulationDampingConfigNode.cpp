// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationDampingConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationDampingConfigNode)

FChaosClothAssetSimulationDampingConfigNode::FChaosClothAssetSimulationDampingConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	 RegisterInputConnection(&DampingCoefficientWeighted.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationDampingConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyWeighted(TEXT("DampingCoefficient"), DampingCoefficientWeighted);
	PropertyHelper.SetPropertyEnum(this, &LocalDampingSpace);
	PropertyHelper.SetProperty(this, &LocalDampingLinearCoefficient);
	PropertyHelper.SetProperty(this, &LocalDampingAngularCoefficient);
}

void FChaosClothAssetSimulationDampingConfigNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (DampingCoefficient_DEPRECATED != DeprecatedDampingCoefficientValue)
	{
		DampingCoefficientWeighted.Low = DampingCoefficientWeighted.High = DampingCoefficient_DEPRECATED;
		DampingCoefficient_DEPRECATED = DeprecatedDampingCoefficientValue;
	}
	if (LocalDampingCoefficient_DEPRECATED != DeprecatedDampingCoefficientValue)
	{
		LocalDampingLinearCoefficient = LocalDampingAngularCoefficient = LocalDampingCoefficient_DEPRECATED;
		LocalDampingCoefficient_DEPRECATED = DeprecatedDampingCoefficientValue;
	}
}
