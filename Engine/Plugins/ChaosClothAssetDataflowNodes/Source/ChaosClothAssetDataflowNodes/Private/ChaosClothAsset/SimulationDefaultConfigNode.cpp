// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationDefaultConfigNode.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosCloth/ChaosClothConfig.h"
#include "ChaosCloth/ChaosClothingSimulationCloth.h"
#include "ChaosCloth/ChaosClothingSimulationConfig.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/PBDLongRangeConstraints.h"
#include "Chaos/PBDSphericalConstraint.h"
#include "Dataflow/DataflowInputOutput.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationDefaultConfigNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetSimulationDefaultConfigNode"

FChaosClothAssetSimulationDefaultConfigNode::FChaosClothAssetSimulationDefaultConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
	, SimulationConfig(NewObject<UChaosClothConfig>(InParam.OwningObject))
	, SharedSimulationConfig(NewObject<UChaosClothSharedSimConfig>(InParam.OwningObject))
	, OwningObject(InParam.OwningObject)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetSimulationDefaultConfigNode::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AddDataflowObjectSerialization)
	{
		if (!SimulationConfig)
		{
			SimulationConfig = NewObject<UChaosClothConfig>(OwningObject);
		}
		SimulationConfig->Serialize(Ar);

		if (!SharedSimulationConfig)
		{
			SharedSimulationConfig = NewObject<UChaosClothSharedSimConfig>(OwningObject);
		}
		SharedSimulationConfig->Serialize(Ar);
	}
}

void FChaosClothAssetSimulationDefaultConfigNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace ::Chaos;
	using namespace ::Chaos::Softs;
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		if (FCollectionClothFacade(ClothCollection).IsValid())  // Can only act on the collection if it is a valid cloth collection
		{
			if (!FCollectionPropertyConstFacade(ClothCollection).IsValid())  // Cannot already have any properties set before this node (it's using CopyTo and wouldn't warn when overriding)
			{
				FClothingSimulationConfig ClothingSimulationConfig;
				ClothingSimulationConfig.Initialize(SimulationConfig.Get(), SharedSimulationConfig.Get());

				ClothingSimulationConfig.GetPropertyCollection(0)->CopyTo(&ClothCollection.Get());

				// Generate tethers
				const FCollectionPropertyConstFacade& Properties = ClothingSimulationConfig.GetProperties(0);
				constexpr bool bUseGeodesicTethersDefault = true;
				const bool bUseGeodesicTethers = Properties.GetValue<bool>(FClothingSimulationCloth::UseGeodesicTethersName, bUseGeodesicTethersDefault);
				// Use the "MaxDistance" weight map to generate tethers. This follows legacy behavior.
				const FName& MaxDistanceName = FPBDSphericalConstraint::MaxDistanceName;
				const FVector2f MaxDistanceValue = Properties.GetWeightedFloatValue(MaxDistanceName, FVector2f(0.f, 1.f));

				UE::Chaos::ClothAsset::FClothEngineTools::GenerateTethers(ClothCollection, MaxDistanceName, bUseGeodesicTethers, MaxDistanceValue);
			}
			else
			{
				Context.Warning(
					LOCTEXT("CannotOverridePreExistingConfigDetails", "The Simulation Default Config node cannot be used to override any existing properties, and must be placed ahead of any other config nodes.")
					, this, Out
				);
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
