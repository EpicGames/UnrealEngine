// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationGravityConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "UObject/FortniteValkyrieBranchObjectVersion.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationGravityConfigNode)

FChaosClothAssetSimulationGravityConfigNode::FChaosClothAssetSimulationGravityConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&GravityScaleWeighted.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationGravityConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyBool(this, &bUseGravityOverride);
	PropertyHelper.SetPropertyWeighted(TEXT("GravityScale"), GravityScaleWeighted);
	
	PropertyHelper.SetSolverProperty(FName(TEXT("GravityOverride")), GravityOverrideImported,
		[](UE::Chaos::ClothAsset::FCollectionClothFacade& ClothFacade)-> FVector3f
		 {
			 return ClothFacade.GetSolverGravity();
		 }, {});
}

void FChaosClothAssetSimulationGravityConfigNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FFortniteValkyrieBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
#if WITH_EDITORONLY_DATA
		if (Ar.CustomVer(FFortniteValkyrieBranchObjectVersion::GUID) < FFortniteValkyrieBranchObjectVersion::ChaosClothAssetWeightedMassAndGravity)
		{
			GravityScaleWeighted.Low = GravityScaleWeighted.High = GravityScale_DEPRECATED;
		}
		if (GravityOverride_DEPRECATED != UE::Chaos::ClothAsset::FDefaultSolver::Gravity)
		{
			GravityOverrideImported.ImportedValue = GravityOverride_DEPRECATED;
			GravityOverride_DEPRECATED = UE::Chaos::ClothAsset::FDefaultSolver::Gravity;
		}
#endif
	}
}
