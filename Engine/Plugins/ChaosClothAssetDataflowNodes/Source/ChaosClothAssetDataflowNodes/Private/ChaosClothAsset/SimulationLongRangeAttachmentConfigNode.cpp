// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationLongRangeAttachmentConfigNode.h"
#include "ChaosClothAsset/ClothEngineTools.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationLongRangeAttachmentConfigNode)

FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2::FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&FixedEndSet.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterInputConnection(&TetherStiffness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&TetherScale.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);

	// Start with one set of option pins.
	for (int32 Index = 0; Index < NumInitialCustomTetherSets; ++Index)
	{
		AddPins();
	}

	check(GetNumInputs() == NumRequiredInputs + NumInitialCustomTetherSets * 2); // Update NumRequiredInputs if you add more Inputs. This is used by Serialize.
}

void FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyWeighted(this, &TetherStiffness);
	PropertyHelper.SetPropertyWeighted(this, &TetherScale);
	PropertyHelper.SetPropertyBool(this, &bUseGeodesicTethers, {}, ECollectionPropertyFlags::Intrinsic);  // Intrinsic since the tethers need to be recalculated.
	PropertyHelper.SetPropertyString(this, &FixedEndSet);
}

void FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2::EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const
{
	const FString InFixedEndSetString = GetValue<FString>(Context, &FixedEndSet.StringValue);
	const FName InFixedEndSet(InFixedEndSetString);
	if (bEnableCustomTetherGeneration)
	{
		TArray<TPair<FName, FName>> TetherEndSets;
		TetherEndSets.SetNumUninitialized(CustomTetherData.Num());
		for (int32 Index = 0; Index < CustomTetherData.Num(); ++Index)
		{
			TetherEndSets[Index] = MakeTuple<FName, FName>(FName(*GetValue(Context, GetDynamicEndConnectionReference(Index))), FName(*GetValue(Context, GetFixedEndConnectionReference(Index))));
		}
		UE::Chaos::ClothAsset::FClothEngineTools::GenerateTethersFromCustomSelectionSets(ClothCollection, InFixedEndSet, TetherEndSets, bUseGeodesicTethers);
	}
	else
	{
		UE::Chaos::ClothAsset::FClothEngineTools::GenerateTethersFromSelectionSet(ClothCollection, InFixedEndSet, bUseGeodesicTethers);
	}
}

TArray<UE::Dataflow::FPin> FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2::AddPins()
{
	const int32 Index = CustomTetherData.AddDefaulted();
	TArray<UE::Dataflow::FPin> Pins;
	Pins.Reserve(2);
	{
		const FDataflowInput& Input = RegisterInputArrayConnection(GetFixedEndConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() });
	}
	{
		const FDataflowInput& Input = RegisterInputArrayConnection(GetDynamicEndConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() });
	}
	return Pins;
}

TArray<UE::Dataflow::FPin> FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2::GetPinsToRemove() const
{
	const int32 Index = CustomTetherData.Num() - 1;
	check(CustomTetherData.IsValidIndex(Index));
	TArray<UE::Dataflow::FPin> Pins;
	Pins.Reserve(2);
	if (const FDataflowInput* const Input = FindInput(GetFixedEndConnectionReference(Index)))
	{
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() });
	}
	if (const FDataflowInput* const Input = FindInput(GetDynamicEndConnectionReference(Index)))
	{
		Pins.Emplace(UE::Dataflow::FPin{ UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() });
	}
	return Pins;
}

void FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = CustomTetherData.Num() - 1;
	check(CustomTetherData.IsValidIndex(Index));
	const FDataflowInput* const FirstInput = FindInput(GetFixedEndConnectionReference(Index));
	const FDataflowInput* const SecondInput = FindInput(GetDynamicEndConnectionReference(Index));
	check(FirstInput || SecondInput);
	const bool bIsFirstInput = FirstInput && FirstInput->GetName() == Pin.Name;
	const bool bIsSecondInput = SecondInput && SecondInput->GetName() == Pin.Name;
	if ((bIsFirstInput && !SecondInput) || (bIsSecondInput && !FirstInput))
	{
		// Both inputs removed. Remove array index.
		CustomTetherData.SetNum(Index);
	}
	return Super::OnPinRemoved(Pin);
}

void FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2::PostSerialize(const FArchive& Ar)
{
	// Restore the pins when re-loading so they can get properly reconnected
	if (Ar.IsLoading())
	{
		check(CustomTetherData.Num() >= NumInitialCustomTetherSets);
		for (int32 Index = 0; Index < NumInitialCustomTetherSets; ++Index)
		{
			check(FindInput(GetFixedEndConnectionReference(Index)));
			check(FindInput(GetDynamicEndConnectionReference(Index)));
		}

		for (int32 Index = NumInitialCustomTetherSets; Index < CustomTetherData.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetFixedEndConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
			FindOrRegisterInputArrayConnection(GetDynamicEndConnectionReference(Index), GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
		}

		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialCustomTetherSets * 2);
			const int32 OrigNumSets = CustomTetherData.Num();
			const int32 OrigNumRegisteredSets = (OrigNumRegisteredInputs - NumRequiredInputs) / 2;

			if (OrigNumRegisteredSets > OrigNumSets)
			{
				ensure(Ar.IsTransacting());
				// Temporarily expand SelectionFilterSets so we can get connection references.
				CustomTetherData.SetNum(OrigNumRegisteredSets);
				for (int32 Index = OrigNumSets; Index < CustomTetherData.Num(); ++Index)
				{
					UnregisterInputConnection(GetDynamicEndConnectionReference(Index));
					UnregisterInputConnection(GetFixedEndConnectionReference(Index));
				}
				CustomTetherData.SetNum(OrigNumRegisteredSets);
			}
		}
		else
		{
			ensureAlways(CustomTetherData.Num() * 2 + NumRequiredInputs == GetNumInputs());
		}
	}
}

UE::Dataflow::TConnectionReference<FString> FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2::GetFixedEndConnectionReference(int32 Index) const
{
	return { &CustomTetherData[Index].CustomFixedEndSet.StringValue, Index, &CustomTetherData };
}

UE::Dataflow::TConnectionReference<FString> FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2::GetDynamicEndConnectionReference(int32 Index) const
{
	return { &CustomTetherData[Index].CustomDynamicEndSet.StringValue, Index, &CustomTetherData };
}


FChaosClothAssetSimulationLongRangeAttachmentConfigNode::FChaosClothAssetSimulationLongRangeAttachmentConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&FixedEndWeightMap);
	RegisterInputConnection(&TetherStiffness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&TetherScale.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationLongRangeAttachmentConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // SetProperty functions are templated and cause deprecation warnings with the now deprecated v1
	PropertyHelper.SetPropertyWeighted(this, &TetherStiffness);
	PropertyHelper.SetPropertyWeighted(this, &TetherScale);
	PropertyHelper.SetPropertyBool(this, &bUseGeodesicTethers, {}, ECollectionPropertyFlags::Intrinsic);  // Intrinsic since the tethers need to be recalculated.
	PropertyHelper.SetPropertyString(this, &FixedEndWeightMap);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FChaosClothAssetSimulationLongRangeAttachmentConfigNode::EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const
{
	const FString InFixedEndWeightMapString = GetValue<FString>(Context, &FixedEndWeightMap);
	const FName InFixedEndWeightMap(InFixedEndWeightMapString);
	UE::Chaos::ClothAsset::FClothEngineTools::GenerateTethers(ClothCollection, InFixedEndWeightMap, bUseGeodesicTethers);
}
