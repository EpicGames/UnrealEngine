// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationMassConfigNode.h"
#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "UObject/FortniteValkyrieBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationMassConfigNode)

FChaosClothAssetSimulationMassConfigNode::FChaosClothAssetSimulationMassConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&UniformMassWeighted.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&DensityWeighted.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationMassConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyEnum(this, &MassMode, {}, ECollectionPropertyFlags::Intrinsic);
	switch (MassMode)
	{
	default:
	case EClothMassMode::UniformMass:
		{
			PropertyHelper.SetPropertyWeighted(FName(TEXT("MassValue")), UniformMassWeighted, {}, ECollectionPropertyFlags::Intrinsic);
		}
		break;
	case EClothMassMode::TotalMass:
		{
			PropertyHelper.SetProperty(FName(TEXT("MassValue")), TotalMass, {}, ECollectionPropertyFlags::Intrinsic);
		}
		break;
	case EClothMassMode::Density:
		{
			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("MassValue")), DensityWeighted, [](const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			 {
				 return FabricFacade.GetDensity();
			 }, {}, ECollectionPropertyFlags::Intrinsic);
		}
		break;
	}
	PropertyHelper.SetProperty(this, &MinPerParticleMass, {}, ECollectionPropertyFlags::Intrinsic);
}

void FChaosClothAssetSimulationMassConfigNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FFortniteValkyrieBranchObjectVersion::GUID);
	if (Ar.IsLoading())
	{
#if WITH_EDITORONLY_DATA
		if (Ar.CustomVer(FFortniteValkyrieBranchObjectVersion::GUID) < FFortniteValkyrieBranchObjectVersion::ChaosClothAssetWeightedMassAndGravity)
		{
			UniformMassWeighted.Low = UniformMassWeighted.High = UniformMass_DEPRECATED;
			DensityWeighted.Low = DensityWeighted.High = Density_DEPRECATED;
		}
#endif
	}
}
