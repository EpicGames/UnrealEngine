// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationMaxDistanceConfigNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Chaos/CollectionPropertyFacade.h"

#if WITH_EDITOR
#include "ChaosClothAsset/ClothDataflowViewModes.h"
#include "Dataflow/DataflowDebugDrawInterface.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationMaxDistanceConfigNode)

FChaosClothAssetSimulationMaxDistanceConfigNode::FChaosClothAssetSimulationMaxDistanceConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	KinematicVertices3D = TEXT("KinematicVertices3D");
	RegisterCollectionConnections();
	RegisterInputConnection(&MaxDistance.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(false);
	RegisterInputConnection(&InKinematic, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterOutputConnection(&KinematicVertices3D);
}

void FChaosClothAssetSimulationMaxDistanceConfigNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	Super::Evaluate(Context, Out);

	if (Out->IsA(&KinematicVertices3D))
	{
		SetValue(Context, KinematicVertices3D, &KinematicVertices3D);
	}
}

void FChaosClothAssetSimulationMaxDistanceConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyWeighted(this, &MaxDistance, {}, ECollectionPropertyFlags::Intrinsic);  // Intrinsic since the deformer weights needs to be recalculated
	PropertyHelper.SetPropertyString(this, &KinematicVertices3D);
}

void FChaosClothAssetSimulationMaxDistanceConfigNode::EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const
{
	using namespace UE::Chaos::ClothAsset;
	const FName MaxDistanceString(GetValue<FString>(Context, &MaxDistance.WeightMap)); //  Override for this is already set by AddProperties
	const FName InputKinematicString(GetValue<FString>(Context, &InKinematic.StringValue));

	FCollectionClothSelectionFacade SelectionFacade(ClothCollection);
	SelectionFacade.DefineSchema();
	SelectionFacade.FindOrAddSelectionSet(FName(KinematicVertices3D), ClothCollectionGroup::SimVertices3D) =
		FClothGeometryTools::GenerateKinematicVertices3D(ClothCollection, MaxDistanceString, FVector2f(MaxDistance.Low, MaxDistance.High), InputKinematicString);

}

#if WITH_EDITOR

bool FChaosClothAssetSimulationMaxDistanceConfigNode::CanDebugDraw() const
{
	return true;
}

bool FChaosClothAssetSimulationMaxDistanceConfigNode::CanDebugDrawViewMode(const FName& ViewModeName) const
{
	return ViewModeName == UE::Dataflow::FDataflowConstruction3DViewMode::Name || ViewModeName == UE::Chaos::ClothAsset::FCloth3DSimViewMode::Name;
}

void FChaosClothAssetSimulationMaxDistanceConfigNode::DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const
{
	using namespace UE::Chaos::ClothAsset;
	
	const FLinearColor DynamicColor(FColor::White);
	const FLinearColor KinematicColor(FColor::Purple);

	FNumberFormattingOptions NumberFormattingOptions;
	NumberFormattingOptions.AlwaysSign = false;
	NumberFormattingOptions.UseGrouping = false;
	NumberFormattingOptions.RoundingMode = ERoundingMode::HalfFromZero;
	NumberFormattingOptions.MinimumIntegralDigits = 1;
	NumberFormattingOptions.MaximumIntegralDigits = 6;
	NumberFormattingOptions.MinimumFractionalDigits = 2;
	NumberFormattingOptions.MaximumFractionalDigits = 2;

	const FManagedArrayCollection OutCollection = GetOutputValue(Context, &Collection, Collection);
	const FName MaxDistanceMapName(GetValue<FString>(Context, &MaxDistance.WeightMap));

	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(OutCollection);
	const FCollectionClothConstFacade ClothFacade(ClothCollection);
	const FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);
	if (ClothFacade.IsValid() && ClothFacade.HasWeightMap(MaxDistanceMapName))
	{
		TConstArrayView<float> MaxDistanceMap = ClothFacade.GetWeightMap(MaxDistanceMapName);
		const TSet<int32>* KinematicSet = SelectionFacade.FindSelectionSet(FName(KinematicVertices3D));
		const float MinValue = MaxDistance.Low;
		const float ValueRange = MaxDistance.High - MaxDistance.Low;
		for (int32 Index = 0; Index < MaxDistanceMap.Num(); ++Index)
		{
			const TConstArrayView<FVector3f> Positions = ClothFacade.GetSimPosition3D();
			if (Positions.IsValidIndex(Index))
			{
				const float MaxDistanceValue = (MinValue + (MaxDistanceMap[Index] * ValueRange));
				const FText Text = FText::AsNumber(MaxDistanceValue, &NumberFormattingOptions);
				const bool bIsKinematic = KinematicSet && KinematicSet->Contains(Index);
				DataflowRenderingInterface.SetColor(bIsKinematic ? KinematicColor: DynamicColor);
				DataflowRenderingInterface.DrawText3d(Text.ToString(), FVector(Positions[Index]));
			}
		}
	}
}
#endif // WITH_EDITOR