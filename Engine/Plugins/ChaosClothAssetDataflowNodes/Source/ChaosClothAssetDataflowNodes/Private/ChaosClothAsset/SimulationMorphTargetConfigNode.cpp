// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationMorphTargetConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationMorphTargetConfigNode)

FChaosClothAssetSimulationMorphTargetConfigNode::FChaosClothAssetSimulationMorphTargetConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
}

void FChaosClothAssetSimulationMorphTargetConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyAndString(FName(TEXT("ActiveMorphTarget")), ActiveMorphTarget.Weight, ActiveMorphTarget.Name);
}
