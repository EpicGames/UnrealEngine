// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationMultiResConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationMultiResConfigNode)

FChaosClothAssetSimulationMultiResConfigNode::FChaosClothAssetSimulationMultiResConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&MultiResStiffness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&MultiResVelocityTargetStiffness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationMultiResConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS  // SetProperty functions are templated and cause deprecation warnings with the now deprecated v1
	if (bIsFineLOD)
	{
		PropertyHelper.SetProperty(this, &MultiResCoarseLODIndex);
		PropertyHelper.SetPropertyBool(this, &bMultiResUseXPBD);
		PropertyHelper.SetPropertyWeighted(this, &MultiResStiffness);
		PropertyHelper.SetPropertyWeighted(this, &MultiResVelocityTargetStiffness);
	}
	if (bIsCoarseMultiResLOD)
	{
		PropertyHelper.SetPropertyBool(this, &bIsCoarseMultiResLOD);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

