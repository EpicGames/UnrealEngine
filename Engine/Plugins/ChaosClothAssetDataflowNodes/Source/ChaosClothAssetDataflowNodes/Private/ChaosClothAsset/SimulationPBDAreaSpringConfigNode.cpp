// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationPBDAreaSpringConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationPBDAreaSpringConfigNode)

FChaosClothAssetSimulationPBDAreaSpringConfigNode::FChaosClothAssetSimulationPBDAreaSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&AreaSpringStiffness.WeightMap);
}

void FChaosClothAssetSimulationPBDAreaSpringConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyWeighted(this,
		&AreaSpringStiffness,
		{
			FName(TEXT("XPBDAreaSpringStiffness"))  // Existing properties to warn against
		});
}
