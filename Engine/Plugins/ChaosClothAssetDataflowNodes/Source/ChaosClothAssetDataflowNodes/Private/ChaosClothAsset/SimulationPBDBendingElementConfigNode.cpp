// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationPBDBendingElementConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationPBDBendingElementConfigNode)

FChaosClothAssetSimulationPBDBendingElementConfigNode::FChaosClothAssetSimulationPBDBendingElementConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&FlatnessRatio.WeightMap);
	RegisterInputConnection(&RestAngle.WeightMap);
	RegisterInputConnection(&BendingElementStiffness.WeightMap);
	RegisterInputConnection(&BucklingStiffness.WeightMap);
}

void FChaosClothAssetSimulationPBDBendingElementConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyEnum(this,
		&RestAngleType,
		{
			FName(TEXT("XPBDRestAngleType")),      // Existing properties to warn against
			FName(TEXT("XPBDAnisoRestAngleType"))  //
		},
		ECollectionPropertyFlags::None);  // Non animatable
	PropertyHelper.SetPropertyWeighted(this,
		&FlatnessRatio,
		{
			FName(TEXT("XPBDFlatnessRatio")),      // Existing properties to warn against
			FName(TEXT("XPBDAnisoFlatnessRatio"))  //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&RestAngle,
		{
			FName(TEXT("XPBDRestAngle")),      // Existing properties to warn against
			FName(TEXT("XPBDAnisoRestAngle"))  //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&BendingElementStiffness,
		{
			FName(TEXT("BendingSpringStiffness")),        // Existing properties to warn against
			FName(TEXT("XPBDBendingSpringStiffness")),    //
			FName(TEXT("XPBDBendingElementStiffness")),   //
			FName(TEXT("XPBDAnisoBendingStiffnessWarp"))  //
		});
	PropertyHelper.SetProperty(this,
		&BucklingRatio,
		{
			FName(TEXT("XPBDBucklingRatio")),      // Existing properties to warn against
			FName(TEXT("XPBDAnisoBucklingRatio"))  //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&BucklingStiffness,
		{
			FName(TEXT("XPBDBucklingStiffness")),          // Existing properties to warn against
			FName(TEXT("XPBDAnisoBucklingStiffnessWarp"))  //
		});
}
