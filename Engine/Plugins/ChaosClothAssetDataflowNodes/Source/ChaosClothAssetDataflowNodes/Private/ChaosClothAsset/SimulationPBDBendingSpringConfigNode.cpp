// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationPBDBendingSpringConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationPBDBendingSpringConfigNode)

FChaosClothAssetSimulationPBDBendingSpringConfigNode::FChaosClothAssetSimulationPBDBendingSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&BendingSpringStiffness.WeightMap);
}

void FChaosClothAssetSimulationPBDBendingSpringConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyWeighted(this,
		&BendingSpringStiffness,
		{
			FName(TEXT("BendingElementStiffness")),       // Existing properties to warn against
			FName(TEXT("XPBDBendingSpringStiffness")),    //
			FName(TEXT("XPBDBendingElementStiffness")),   //
			FName(TEXT("XPBDAnisoBendingStiffnessWarp"))  //
		});
}
