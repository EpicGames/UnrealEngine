// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationPBDEdgeSpringConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationPBDEdgeSpringConfigNode)

FChaosClothAssetSimulationPBDEdgeSpringConfigNode::FChaosClothAssetSimulationPBDEdgeSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&EdgeSpringStiffness.WeightMap);
}

void FChaosClothAssetSimulationPBDEdgeSpringConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyWeighted(this,
		&EdgeSpringStiffness,
		{
			FName(TEXT("XPBDEdgeSpringStiffness")),       // Existing properties to warn against
			FName(TEXT("XPBDAnisoStretchStiffnessWarp"))  //
		});
}
