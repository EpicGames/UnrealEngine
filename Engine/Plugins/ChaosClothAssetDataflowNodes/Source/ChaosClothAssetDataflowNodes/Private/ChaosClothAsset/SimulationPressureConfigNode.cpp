// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationPressureConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationPressureConfigNode)

FChaosClothAssetSimulationPressureConfigNode::FChaosClothAssetSimulationPressureConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&Pressure.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationPressureConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("Pressure")), Pressure, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
	{
		return FabricFacade.GetPressure();
	}, {});
}
