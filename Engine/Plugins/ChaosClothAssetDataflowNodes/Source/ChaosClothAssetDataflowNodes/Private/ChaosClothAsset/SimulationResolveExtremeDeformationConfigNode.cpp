// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationResolveExtremeDeformationConfigNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationResolveExtremeDeformationConfigNode)

FChaosClothAssetSimulationResolveExtremeDeformationConfigNode::FChaosClothAssetSimulationResolveExtremeDeformationConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&InputSelection, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterOutputConnection(&ExtremeDeformationVertexSelection);
}

void FChaosClothAssetSimulationResolveExtremeDeformationConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetProperty(this, &ExtremeDeformationEdgeRatioThreshold);
	PropertyHelper.SetPropertyString(this, &ExtremeDeformationVertexSelection, {}, ECollectionPropertyFlags::None);
}

void FChaosClothAssetSimulationResolveExtremeDeformationConfigNode::EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const
{
	using namespace UE::Chaos::ClothAsset;
	TSet<int32> SelectionSet;
	const FName InSelectionName(GetValue(Context, &InputSelection.StringValue));
	const FName OutSelectionName(ExtremeDeformationVertexSelection);
	FCollectionClothSelectionFacade SelectionFacade(ClothCollection);
	SelectionFacade.DefineSchema();
	if (FClothGeometryTools::ConvertSelectionToNewGroupType(ClothCollection, InSelectionName, ClothCollectionGroup::SimVertices3D, SelectionSet))
	{
		SelectionFacade.FindOrAddSelectionSet(OutSelectionName, ClothCollectionGroup::SimVertices3D) = SelectionSet;
	}
}

void FChaosClothAssetSimulationResolveExtremeDeformationConfigNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	Super::Evaluate(Context, Out);

	if (Out->IsA(&ExtremeDeformationVertexSelection))
	{
		SetValue(Context, ExtremeDeformationVertexSelection, &ExtremeDeformationVertexSelection);
	}
}