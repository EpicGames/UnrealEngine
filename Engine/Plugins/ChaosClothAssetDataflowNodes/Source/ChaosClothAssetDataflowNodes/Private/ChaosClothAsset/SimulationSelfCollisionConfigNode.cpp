// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationSelfCollisionConfigNode.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationSelfCollisionConfigNode)


FChaosClothAssetSimulationSelfCollisionConfigNode_v2::FChaosClothAssetSimulationSelfCollisionConfigNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&SelfCollisionLayers.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&SelfCollisionDisabledFaces.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&SelfCollisionThickness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationSelfCollisionConfigNode_v2::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyBool(this, &bUseSelfCollisions);
	PropertyHelper.SetProperty(this, &SelfCollisionStiffness);
	PropertyHelper.SetProperty(this, &SelfCollisionDisableNeighborDistance, {}, ECollectionPropertyFlags::None); // Non animatable
	PropertyHelper.SetPropertyString(this, &SelfCollisionDisabledFaces);
	PropertyHelper.SetPropertyBool(this, &bUseSelfIntersections);
	PropertyHelper.SetPropertyBool(this, &bUseGlobalIntersectionAnalysis);
	PropertyHelper.SetPropertyBool(this, &bUseContourMinimization);
	PropertyHelper.SetProperty(this, &NumContourMinimizationPostSteps);
	PropertyHelper.SetPropertyBool(this, &bUseGlobalPostStepContours);

	PropertyHelper.SetFabricPropertyString<int32, FChaosClothAssetConnectableIStringValue>(FName(TEXT("SelfCollisionLayers")), SelfCollisionLayers,
		[](const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
		{
			return FabricFacade.GetLayer();
		}, {}, ECollectionPropertyFlags::None, UE::Chaos::ClothAsset::ClothCollectionGroup::SimFaces);

	PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("SelfCollisionThickness")), SelfCollisionThickness,
		[](const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
		{
			return FabricFacade.GetCollisionThickness();
		}, {});

	PropertyHelper.SetFabricProperty(FName(TEXT("SelfCollisionFriction")), SelfCollisionFriction,
		[](UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
		{
			return FabricFacade.GetFriction();
		}, {});
}

FChaosClothAssetSimulationSelfCollisionConfigNode::FChaosClothAssetSimulationSelfCollisionConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&SelfCollisionLayers.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&SelfCollisionDisabledFaces.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&SelfCollisionEnabledKinematicFaces.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&SelfCollisionKinematicColliderFrictionWeighted.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&SelfCollisionThicknessWeighted.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationSelfCollisionConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // SetProperty functions are templated and cause deprecation warnings with the now deprecated v1
	PropertyHelper.SetPropertyBool(this, &bUseSelfCollisions);
	PropertyHelper.SetProperty(this, &SelfCollisionStiffness);
	PropertyHelper.SetProperty(this, &SelfCollisionDisableNeighborDistance, {}, ECollectionPropertyFlags::None); // Non animatable
	PropertyHelper.SetPropertyString(this, &SelfCollisionDisabledFaces);
	PropertyHelper.SetPropertyBool(this, &bSelfCollideAgainstKinematicCollidersOnly);
	PropertyHelper.SetPropertyBool(this, &bSelfCollideAgainstAllKinematicVertices);
	PropertyHelper.SetPropertyString(this, &SelfCollisionEnabledKinematicFaces);
	PropertyHelper.SetProperty(this, &SelfCollisionKinematicColliderThickness);
	PropertyHelper.SetProperty(this, &SelfCollisionKinematicColliderStiffness);
	PropertyHelper.SetPropertyWeighted(TEXT("SelfCollisionKinematicColliderFriction"), SelfCollisionKinematicColliderFrictionWeighted);

	PropertyHelper.SetPropertyBool(this, &bUseSelfIntersections);
	PropertyHelper.SetPropertyBool(this, &bUseGlobalIntersectionAnalysis);
	PropertyHelper.SetPropertyBool(this, &bUseContourMinimization);
	PropertyHelper.SetProperty(this, &NumContourMinimizationPostSteps);
	PropertyHelper.SetPropertyBool(this, &bUseGlobalPostStepContours);
	PropertyHelper.SetProperty(this, &SelfCollisionProximityStiffness);

	PropertyHelper.SetFabricPropertyString<int32,FChaosClothAssetConnectableIStringValue>(FName(TEXT("SelfCollisionLayers")), SelfCollisionLayers,
		[](const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
		{
			return FabricFacade.GetLayer();
		}, {}, ECollectionPropertyFlags::None, UE::Chaos::ClothAsset::ClothCollectionGroup::SimFaces);
	
	PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("SelfCollisionThickness")), SelfCollisionThicknessWeighted,
		[](const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
		{
			return FabricFacade.GetCollisionThickness();
		}, {});

	PropertyHelper.SetFabricProperty(FName(TEXT("SelfCollisionFriction")), SelfCollisionFrictionImported,
		[](UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
		{
			return FabricFacade.GetFriction();
		}, {});
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FChaosClothAssetSimulationSelfCollisionConfigNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsLoading())
	{
#if WITH_EDITORONLY_DATA
		if (SelfCollisionFriction_DEPRECATED != SelfFrictionDeprecatedValue)
		{
			SelfCollisionFrictionImported.ImportedValue = SelfCollisionFriction_DEPRECATED;
			SelfCollisionFriction_DEPRECATED = SelfFrictionDeprecatedValue;
		}
		if (SelfCollisionKinematicColliderFriction_DEPRECATED != FrictionDeprecatedValue)
		{
			SelfCollisionKinematicColliderFrictionWeighted.Low = SelfCollisionKinematicColliderFrictionWeighted.High = SelfCollisionKinematicColliderFriction_DEPRECATED;
			SelfCollisionKinematicColliderFriction_DEPRECATED = FrictionDeprecatedValue;
		}
		if (SelfCollisionThickness_DEPRECATED != SelfCollisionThicknessDeprecatedValue)
		{
			SelfCollisionThicknessWeighted.Low = SelfCollisionThicknessWeighted.High = SelfCollisionThickness_DEPRECATED;
			SelfCollisionThickness_DEPRECATED = SelfCollisionThicknessDeprecatedValue;
		}
#endif
	}
}