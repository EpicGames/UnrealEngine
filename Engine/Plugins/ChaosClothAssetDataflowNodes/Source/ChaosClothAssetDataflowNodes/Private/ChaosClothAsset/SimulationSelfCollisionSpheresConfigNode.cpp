// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationSelfCollisionSpheresConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationSelfCollisionSpheresConfigNode)

FChaosClothAssetSimulationSelfCollisionSpheresConfigNode::FChaosClothAssetSimulationSelfCollisionSpheresConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterOutputConnection(&SelfCollisionSphereSetName);
}

void FChaosClothAssetSimulationSelfCollisionSpheresConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetProperty(this, &SelfCollisionSphereRadius);
	PropertyHelper.SetProperty(this, &SelfCollisionSphereRadiusCullMultiplier, {}, ECollectionPropertyFlags::Intrinsic); // Need to rebuild the sphere set to see this effect.
	PropertyHelper.SetProperty(this, &SelfCollisionSphereStiffness);
	PropertyHelper.SetPropertyString(this, &SelfCollisionSphereSetName, {}, ECollectionPropertyFlags::None);  // Non animatable
}

void FChaosClothAssetSimulationSelfCollisionSpheresConfigNode::EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const
{
	using namespace UE::Chaos::ClothAsset;
	FCollectionClothConstFacade Cloth(ClothCollection);
	const float CullDiameterSq = FMath::Square(SelfCollisionSphereRadius * SelfCollisionSphereRadiusCullMultiplier * 2.f);
	if (Cloth.IsValid() && CullDiameterSq > 0.f)
	{
		TConstArrayView<FVector3f> SimPositions = Cloth.GetSimPosition3D();
		TSet<int32> VertexSet;

		FClothGeometryTools::SampleVertices(SimPositions, CullDiameterSq, VertexSet);

		FCollectionClothSelectionFacade Selection(ClothCollection);
		Selection.DefineSchema();

		Selection.FindOrAddSelectionSet(FName(*SelfCollisionSphereSetName), ClothCollectionGroup::SimVertices3D) = VertexSet;
	}
}

void FChaosClothAssetSimulationSelfCollisionSpheresConfigNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FString>(&SelfCollisionSphereSetName))
	{
		SetValue(Context, SelfCollisionSphereSetName, &SelfCollisionSphereSetName);
	}
	else
	{
		Super::Evaluate(Context, Out);
	}
}
