// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationSolverConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationSolverConfigNode)

FChaosClothAssetSimulationSolverConfigNode::FChaosClothAssetSimulationSolverConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
}

void FChaosClothAssetSimulationSolverConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetProperty(this, &NumIterations);
	PropertyHelper.SetProperty(this, &MaxNumIterations);
	const float DynamicSubstepDeltaTimeValue = bEnableDynamicSubstepping ? DynamicSubstepDeltaTime : 0.f;
	PropertyHelper.SetProperty(TEXT("DynamicSubstepDeltaTime"), DynamicSubstepDeltaTimeValue);
	PropertyHelper.SetPropertyBool(this, &bEnableNumSelfCollisionSubsteps);
	PropertyHelper.SetProperty(this, &NumSelfCollisionSubsteps);
	
	PropertyHelper.SetSolverProperty(FName(TEXT("NumSubsteps")), NumSubstepsImported,
		[](UE::Chaos::ClothAsset::FCollectionClothFacade& ClothFacade)-> int32
		 {
			 return ClothFacade.GetSolverSubSteps();
		 }, {});
}

void FChaosClothAssetSimulationSolverConfigNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	if (Ar.IsLoading())
	{
#if WITH_EDITORONLY_DATA
		if (NumSubsteps_DEPRECATED != UE::Chaos::ClothAsset::FDefaultSolver::SubSteps)
		{
			NumSubstepsImported.ImportedValue = NumSubsteps_DEPRECATED;
			NumSubsteps_DEPRECATED = UE::Chaos::ClothAsset::FDefaultSolver::SubSteps;
		}
#endif
	}
}
