// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationStretchConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationStretchConfigNode)

FChaosClothAssetSimulationStretchConfigNode::FChaosClothAssetSimulationStretchConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&StretchStiffness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&StretchStiffnessWarp.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&StretchStiffnessWeft.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&StretchStiffnessBias.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&StretchDamping.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&StretchAnisoDamping.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&StretchWarpScale.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&StretchWeftScale.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&AreaStiffness.WeightMap)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}

void FChaosClothAssetSimulationStretchConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	if(SolverType == EChaosClothAssetConstraintSolverType::XPBD)
	{
		if (DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic)
		{
			PropertyHelper.SetPropertyBool(FName(TEXT("XPBDAnisoSpringUse3dRestLengths")), bStretchUse3dRestLengths, {
				FName(TEXT("XPBDAnisoStretchUse3dRestLengths"))}, ECollectionPropertyFlags::None);  // Non animatable

			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoSpringStiffnessWarp")), StretchStiffnessWarp, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			 {
				 return FabricFacade.GetStretchStiffness().Warp;
			 }, {
				FName(TEXT("EdgeSpringStiffness")),
				FName(TEXT("XPBDEdgeSpringStiffness")),
				FName(TEXT("XPBDAnisoStretchStiffnessWarp"))});

			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoSpringStiffnessWeft")), StretchStiffnessWeft, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			 {
				 return FabricFacade.GetStretchStiffness().Weft;
			 }, { FName(TEXT("XPBDAnisoStretchStiffnessWeft")) });
	
			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoSpringStiffnessBias")), StretchStiffnessBias, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			 {
				 return FabricFacade.GetStretchStiffness().Bias;
			 }, { FName(TEXT("XPBDAnisoStretchStiffnessBias")) });

			PropertyHelper.SetFabricPropertyWeighted(FName(TEXT("XPBDAnisoSpringDamping")), StretchAnisoDamping, [](
				const UE::Chaos::ClothAsset::FCollectionClothFabricFacade& FabricFacade)-> float
			 {
				 return FabricFacade.GetDamping();
			 }, { FName(TEXT("XPBDEdgeSpringDamping")),
				FName(TEXT("XPBDAnisoStretchDamping")) });

			PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDAnisoSpringWarpScale")), StretchWarpScale, 
				{ 
					FName(TEXT("XPBDAnisoStretchWarpScale")),
					FName(TEXT("EdgeSpringWarpScale"))
				});
			PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDAnisoSpringWeftScale")), StretchWeftScale, 
				{ 
					FName(TEXT("XPBDAnisoStretchWeftScale")),
					FName(TEXT("EdgeSpringWeftScale"))
				});
		}
		else
		{
			PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDEdgeSpringStiffness")), StretchStiffness, {
				FName(TEXT("EdgeSpringStiffness")),
				FName(TEXT("XPBDAnisoStretchStiffnessWarp")),
				FName(TEXT("XPBDAnisoSpringStiffnessWarp"))});

			PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDEdgeSpringDamping")), StretchDamping, {
				FName(TEXT("XPBDAnisoStretchDamping")),
				FName(TEXT("XPBDAnisoSpringDamping"))});

			if(bAddAreaConstraint)
			{
				PropertyHelper.SetPropertyWeighted(FName(TEXT("XPBDAreaSpringStiffness")), AreaStiffness,{
					FName(TEXT("AreaSpringStiffness"))});
			}
		}
	}
	else
	{
		PropertyHelper.SetPropertyWeighted(FName(TEXT("EdgeSpringStiffness")), StretchStiffness, {
			FName(TEXT("XPBDEdgeSpringStiffness")), 
			FName(TEXT("XPBDAnisoStretchStiffnessWarp")),
			FName(TEXT("XPBDAnisoSpringStiffnessWarp"))});

		if (bEnableStretchWarpAndWeftScale)
		{
			PropertyHelper.SetPropertyWeighted(FName(TEXT("EdgeSpringWarpScale")), StretchWarpScale,
				{
					FName(TEXT("XPBDAnisoStretchWarpScale")),
					FName(TEXT("XPBDAnisoSpringWarpScale"))
				});
			PropertyHelper.SetPropertyWeighted(FName(TEXT("EdgeSpringWeftScale")), StretchWeftScale,
				{
					FName(TEXT("XPBDAnisoStretchWeftScale")),
					FName(TEXT("XPBDAnisoSpringWeftScale"))
				});
		}
		
		if(bAddAreaConstraint)
		{
			PropertyHelper.SetPropertyWeighted(FName(TEXT("AreaSpringStiffness")), AreaStiffness,{
				FName(TEXT("XPBDAreaSpringStiffness"))});

			if (bEnableStretchWarpAndWeftScale)
			{
				PropertyHelper.SetPropertyWeighted(FName(TEXT("AreaSpringWarpScale")), StretchWarpScale);
				PropertyHelper.SetPropertyWeighted(FName(TEXT("AreaSpringWeftScale")), StretchWeftScale);
			}
		}
	}
}
