// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationStretchOverrideConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationStretchOverrideConfigNode)

FChaosClothAssetSimulationStretchOverrideConfigNode::FChaosClothAssetSimulationStretchOverrideConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
}

void FChaosClothAssetSimulationStretchOverrideConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	if (bOverrideStretchUse3dRestLengths)
	{
		PropertyHelper.OverridePropertiesBool({
			FName(TEXT("XPBDAnisoStretchUse3dRestLengths")),
			FName(TEXT("XPBDAnisoSpringUse3dRestLengths")) }, bStretchUse3dRestLengths);
	}
	if (OverrideStretchStiffness != EChaosClothAssetConstraintOverrideType::None)
	{
		TArray<FName> StretchPropertyNames =
		{
			FName(TEXT("EdgeSpringStiffness")),
			FName(TEXT("XPBDEdgeSpringStiffness")),
			FName(TEXT("AreaSpringStiffness")),
			FName(TEXT("XPBDAreaSpringStiffness"))
		};
		if (bApplyUniformStretchStiffnessOverride)
		{
			StretchPropertyNames.Append(
				{
					FName(TEXT("XPBDAnisoSpringStiffnessWarp")),
					FName(TEXT("XPBDAnisoStretchStiffnessWarp")),
					FName(TEXT("XPBDAnisoSpringStiffnessWeft")),
					FName(TEXT("XPBDAnisoStretchStiffnessWeft")),
					FName(TEXT("XPBDAnisoSpringStiffnessBias")),
					FName(TEXT("XPBDAnisoStretchStiffnessBias"))
				});
		}
		PropertyHelper.OverridePropertiesWeighted(StretchPropertyNames, OverrideStretchStiffness, StretchStiffness);
	}
	if (OverrideStretchStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformStretchStiffnessOverride)
	{
		if (OverrideStretchStiffnessWarp != EChaosClothAssetConstraintOverrideType::None)
		{
			PropertyHelper.OverridePropertiesWeighted({
					FName(TEXT("XPBDAnisoSpringStiffnessWarp")),
					FName(TEXT("XPBDAnisoStretchStiffnessWarp"))}, OverrideStretchStiffnessWarp, StretchStiffnessWarp);
		}
		if (OverrideStretchStiffnessWeft != EChaosClothAssetConstraintOverrideType::None)
		{
			PropertyHelper.OverridePropertiesWeighted({
					FName(TEXT("XPBDAnisoSpringStiffnessWeft")),
					FName(TEXT("XPBDAnisoStretchStiffnessWeft")) }, OverrideStretchStiffnessWeft, StretchStiffnessWeft);
		}
		if (OverrideStretchStiffnessBias != EChaosClothAssetConstraintOverrideType::None)
		{
			PropertyHelper.OverridePropertiesWeighted({
					FName(TEXT("XPBDAnisoSpringStiffnessBias")),
					FName(TEXT("XPBDAnisoStretchStiffnessBias")) }, OverrideStretchStiffnessBias, StretchStiffnessBias);
		}
	}
	if (OverrideStretchDamping != EChaosClothAssetConstraintOverrideType::None)
	{
		PropertyHelper.OverridePropertiesWeighted({
				FName(TEXT("XPBDAnisoSpringDamping")),
				FName(TEXT("XPBDAnisoStretchDamping")),
				FName(TEXT("XPBDEdgeSpringDamping"))}, OverrideStretchDamping, StretchDamping);
	}
	if (OverrideWarpScale != EChaosClothAssetConstraintOverrideType::None)
	{
		PropertyHelper.OverridePropertiesWeighted({
				FName(TEXT("XPBDAnisoSpringWarpScale")),
				FName(TEXT("XPBDAnisoStretchWarpScale"))}, OverrideWarpScale, WarpScale);
	}
	if (OverrideWeftScale != EChaosClothAssetConstraintOverrideType::None)
	{
		PropertyHelper.OverridePropertiesWeighted({
				FName(TEXT("XPBDAnisoSpringWeftScale")),
				FName(TEXT("XPBDAnisoStretchWeftScale")) }, OverrideWeftScale, WeftScale);
	}
}
