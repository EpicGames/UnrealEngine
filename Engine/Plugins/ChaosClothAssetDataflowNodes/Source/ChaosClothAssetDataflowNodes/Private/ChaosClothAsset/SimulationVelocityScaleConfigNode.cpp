// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationVelocityScaleConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationVelocityScaleConfigNode)

FChaosClothAssetSimulationVelocityScaleConfigNode::FChaosClothAssetSimulationVelocityScaleConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
}

void FChaosClothAssetSimulationVelocityScaleConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	static const FVector3f DefaultLinearClamp = { TNumericLimits<float>::Max(), TNumericLimits<float>::Max(), TNumericLimits<float>::Max() };
	PropertyHelper.SetPropertyEnum(this, &VelocityScaleSpace);
	PropertyHelper.SetProperty(this, &LinearVelocityScale);
	PropertyHelper.SetProperty(TEXT("MaxLinearVelocity"), bEnableLinearVelocityClamping ? MaxLinearVelocity : DefaultLinearClamp);
	PropertyHelper.SetProperty(TEXT("MaxLinearAcceleration"), bEnableLinearAccelerationClamping ? MaxLinearAcceleration : DefaultLinearClamp);
	PropertyHelper.SetProperty(this, &AngularVelocityScale);
	PropertyHelper.SetProperty(TEXT("MaxAngularVelocity"), bEnableAngularVelocityClamping ? MaxAngularVelocity : TNumericLimits<float>::Max());
	PropertyHelper.SetProperty(TEXT("MaxAngularAcceleration"), bEnableAngularAccelerationClamping ? MaxAngularAcceleration : TNumericLimits<float>::Max());
	PropertyHelper.SetProperty(this, &MaxVelocityScale);
	PropertyHelper.SetProperty(this, &FictitiousAngularScale);
	PropertyHelper.SetPropertyEnum(this, &FictitiousForcesModel);
}

void FChaosClothAssetSimulationVelocityScaleConfigNode::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);
	if (Ar.CustomVer(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::ChaosClothAddNewFictitiousForcesModes)
	{
		FictitiousForcesModel = EChaosClothingSimulationSolverFictitiousForcesModel::Legacy;
	}
}
