// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationXPBDAnisoBendingConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationXPBDAnisoBendingConfigNode)

FChaosClothAssetSimulationXPBDAnisoBendingConfigNode::FChaosClothAssetSimulationXPBDAnisoBendingConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&XPBDAnisoFlatnessRatio.WeightMap);
	RegisterInputConnection(&XPBDAnisoRestAngle.WeightMap);
	RegisterInputConnection(&XPBDAnisoBendingStiffnessWarp.WeightMap);
	RegisterInputConnection(&XPBDAnisoBendingStiffnessWeft.WeightMap);
	RegisterInputConnection(&XPBDAnisoBendingStiffnessBias.WeightMap);
	RegisterInputConnection(&XPBDAnisoBendingDamping.WeightMap);
	RegisterInputConnection(&XPBDAnisoBucklingStiffnessWarp.WeightMap);
	RegisterInputConnection(&XPBDAnisoBucklingStiffnessWeft.WeightMap);
	RegisterInputConnection(&XPBDAnisoBucklingStiffnessBias.WeightMap);
}

void FChaosClothAssetSimulationXPBDAnisoBendingConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyEnum(this,
		&XPBDAnisoRestAngleType,
		{
			FName(TEXT("XPBDRestAngleType")),  // Existing properties to warn against
			FName(TEXT("RestAngleType"))       //
		},
		ECollectionPropertyFlags::None);  // Non animatable
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDAnisoFlatnessRatio,
		{
			FName(TEXT("XPBDFlatnessRatio")),  // Existing properties to warn against
			FName(TEXT("FlatnessRatio"))       //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDAnisoRestAngle,
		{
			FName(TEXT("XPBDRestAngle")),  // Existing properties to warn against
			FName(TEXT("RestAngle"))       //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDAnisoBendingStiffnessWarp,
		{
			FName(TEXT("BendingSpringStiffness")),      // Existing properties to warn against
			FName(TEXT("BendingElementStiffness")),     //
			FName(TEXT("XPBDBendingSpringStiffness")),  //
			FName(TEXT("XPBDBendingElementStiffness"))  //
		});
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoBendingStiffnessWeft);
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoBendingStiffnessBias);
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDAnisoBendingDamping,
		{
			FName(TEXT("XPBDBendingSpringDamping")),  // Existing properties to warn against
			FName(TEXT("XPBDBendingElementDamping"))  //
		});
	PropertyHelper.SetProperty(this,
		&XPBDAnisoBucklingRatio,
		{
			FName(TEXT("BucklingRatio")),     // Existing properties to warn against
			FName(TEXT("XPBDBucklingRatio"))  //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDAnisoBucklingStiffnessWarp,
		{
			FName(TEXT("BucklingStiffness")),     // Existing properties to warn against
			FName(TEXT("XPBDBucklingStiffness"))  //
		});
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoBucklingStiffnessWeft);
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoBucklingStiffnessBias);
}
