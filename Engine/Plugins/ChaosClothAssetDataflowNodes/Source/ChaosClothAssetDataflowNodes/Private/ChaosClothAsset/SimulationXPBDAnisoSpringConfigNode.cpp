// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationXPBDAnisoSpringConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationXPBDAnisoSpringConfigNode)

FChaosClothAssetSimulationXPBDAnisoSpringConfigNode::FChaosClothAssetSimulationXPBDAnisoSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&XPBDAnisoSpringStiffnessWarp.WeightMap);
	RegisterInputConnection(&XPBDAnisoSpringStiffnessWeft.WeightMap);
	RegisterInputConnection(&XPBDAnisoSpringStiffnessBias.WeightMap);
	RegisterInputConnection(&XPBDAnisoSpringDamping.WeightMap);
	RegisterInputConnection(&XPBDAnisoSpringWarpScale.WeightMap);
	RegisterInputConnection(&XPBDAnisoSpringWeftScale.WeightMap);
}

void FChaosClothAssetSimulationXPBDAnisoSpringConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyBool(this, &bXPBDAnisoSpringUse3dRestLengths, {}, ECollectionPropertyFlags::None);  // Non animatable
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoSpringStiffnessWarp);
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoSpringStiffnessWeft);
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoSpringStiffnessBias);
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoSpringDamping);
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoSpringWarpScale);
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoSpringWeftScale);
}
