// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationXPBDAnisoStretchConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationXPBDAnisoStretchConfigNode)

FChaosClothAssetSimulationXPBDAnisoStretchConfigNode::FChaosClothAssetSimulationXPBDAnisoStretchConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&XPBDAnisoStretchStiffnessWarp.WeightMap);
	RegisterInputConnection(&XPBDAnisoStretchStiffnessWeft.WeightMap);
	RegisterInputConnection(&XPBDAnisoStretchStiffnessBias.WeightMap);
	RegisterInputConnection(&XPBDAnisoStretchDamping.WeightMap);
	RegisterInputConnection(&XPBDAnisoStretchWarpScale.WeightMap);
	RegisterInputConnection(&XPBDAnisoStretchWeftScale.WeightMap);
}

void FChaosClothAssetSimulationXPBDAnisoStretchConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyBool(this, &bXPBDAnisoStretchUse3dRestLengths, {}, ECollectionPropertyFlags::None);  // Non animatable
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDAnisoStretchStiffnessWarp,
		{
			FName(TEXT("EdgeSpringStiffness")),     // Existing properties to warn against
			FName(TEXT("XPBDEdgeSpringStiffness"))  //
		});
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoStretchStiffnessWeft);
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoStretchStiffnessBias);
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDAnisoStretchDamping,
		{
			FName(TEXT("XPBDEdgeSpringDamping"))  // Existing properties to warn against
		});
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoStretchWarpScale);
	PropertyHelper.SetPropertyWeighted(this, &XPBDAnisoStretchWeftScale);
}
