// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationXPBDAreaSpringConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationXPBDAreaSpringConfigNode)

FChaosClothAssetSimulationXPBDAreaSpringConfigNode::FChaosClothAssetSimulationXPBDAreaSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&XPBDAreaSpringStiffness.WeightMap);
}

void FChaosClothAssetSimulationXPBDAreaSpringConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDAreaSpringStiffness,
		{
			FName(TEXT("AreaSpringStiffness"))  // Existing properties to warn against
		});
}
