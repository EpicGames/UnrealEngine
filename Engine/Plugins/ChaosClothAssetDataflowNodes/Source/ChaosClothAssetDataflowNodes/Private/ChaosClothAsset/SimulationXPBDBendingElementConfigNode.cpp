// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationXPBDBendingElementConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationXPBDBendingElementConfigNode)

FChaosClothAssetSimulationXPBDBendingElementConfigNode::FChaosClothAssetSimulationXPBDBendingElementConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&XPBDFlatnessRatio.WeightMap);
	RegisterInputConnection(&XPBDRestAngle.WeightMap);
	RegisterInputConnection(&XPBDBendingElementStiffness.WeightMap);
	RegisterInputConnection(&XPBDBendingElementDamping.WeightMap);
	RegisterInputConnection(&XPBDBucklingStiffness.WeightMap);
}

void FChaosClothAssetSimulationXPBDBendingElementConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyEnum(this,
		&XPBDRestAngleType,
		{
			FName(TEXT("RestAngleType")),          // Existing properties to warn against
			FName(TEXT("XPBDAnisoRestAngleType"))  //
		},
		ECollectionPropertyFlags::None);  // Non animatable
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDFlatnessRatio,
		{
			FName(TEXT("FlatnessRatio")),          // Existing properties to warn against
			FName(TEXT("XPBDAnisoFlatnessRatio"))  //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDRestAngle,
		{
			FName(TEXT("RestAngle")),          // Existing properties to warn against
			FName(TEXT("XPBDAnisoRestAngle"))  //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDBendingElementStiffness,
		{
			FName(TEXT("BendingSpringStiffness")),        // Existing properties to warn against
			FName(TEXT("BendingElementStiffness")),       //
			FName(TEXT("XPBDBendingSpringStiffness")),    //
			FName(TEXT("XPBDAnisoBendingStiffnessWarp"))  //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDBendingElementDamping,
		{
			FName(TEXT("XPBDBendingSpringDamping")),  // Existing properties to warn against
			FName(TEXT("XPBDAnisoBendingDamping"))    //
		});
	PropertyHelper.SetProperty(this,
		&XPBDBucklingRatio,
		{
			FName(TEXT("BucklingRatio")),          // Existing properties to warn against
			FName(TEXT("XPBDAnisoBucklingRatio"))  //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDBucklingStiffness,
		{
			FName(TEXT("BucklingStiffness")),              // Existing properties to warn against
			FName(TEXT("XPBDAnisoBucklingStiffnessWarp"))  //
		});
}
