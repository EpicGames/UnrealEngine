// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationXPBDBendingSpringConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationXPBDBendingSpringConfigNode)

FChaosClothAssetSimulationXPBDBendingSpringConfigNode::FChaosClothAssetSimulationXPBDBendingSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&XPBDBendingSpringStiffness.WeightMap);
	RegisterInputConnection(&XPBDBendingSpringDamping.WeightMap);
}

void FChaosClothAssetSimulationXPBDBendingSpringConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDBendingSpringStiffness,
		{
			FName(TEXT("BendingSpringStiffness")),        // Existing properties to warn against
			FName(TEXT("BendingElementStiffness")),       //
			FName(TEXT("XPBDBendingElementStiffness")),   //
			FName(TEXT("XPBDAnisoBendingStiffnessWarp"))  //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDBendingSpringDamping,
		{
			FName(TEXT("XPBDBendingElementDamping")),  // Existing properties to warn against
			FName(TEXT("XPBDAnisoBendingDamping"))     //
		});
}
