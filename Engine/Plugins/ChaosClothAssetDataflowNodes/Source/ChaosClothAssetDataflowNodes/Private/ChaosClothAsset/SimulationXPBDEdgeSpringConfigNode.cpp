// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SimulationXPBDEdgeSpringConfigNode.h"
#include "Chaos/CollectionPropertyFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimulationXPBDEdgeSpringConfigNode)

FChaosClothAssetSimulationXPBDEdgeSpringConfigNode::FChaosClothAssetSimulationXPBDEdgeSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FChaosClothAssetSimulationBaseConfigNode(InParam, InGuid)
{
	RegisterCollectionConnections();
	RegisterInputConnection(&XPBDEdgeSpringStiffness.WeightMap);
	RegisterInputConnection(&XPBDEdgeSpringDamping.WeightMap);
}

void FChaosClothAssetSimulationXPBDEdgeSpringConfigNode::AddProperties(FPropertyHelper& PropertyHelper) const
{
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDEdgeSpringStiffness,
		{
			FName(TEXT("EdgeSpringStiffness")),           // Existing properties to warn against
			FName(TEXT("XPBDAnisoStretchStiffnessWarp"))  //
		});
	PropertyHelper.SetPropertyWeighted(this,
		&XPBDEdgeSpringDamping,
		{
			FName(TEXT("XPBDAnisoStretchDamping"))  // Existing properties to warn against
		});
}
