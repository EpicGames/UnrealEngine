// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/SkeletalMeshImportNode.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Animation/MorphTarget.h"
#include "Animation/Skeleton.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "Rendering/SkeletalMeshModel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SkeletalMeshImportNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetSkeletalMeshImportNode"

namespace UE::Chaos::ClothAsset::Private
{
	TMap<int32, int32> GenerateSKMToSim3DLookup(const FCollectionClothConstFacade& ClothFacade, const TArray<int32>& Sim2DToSKMIndex)
	{
		TConstArrayView<int32 > SimVertex3DLookup = ClothFacade.GetSimVertex3DLookup();
		check(Sim2DToSKMIndex.Num() == SimVertex3DLookup.Num());
	
		TMap<int32, int32> SKMToSim3DLookup;
		for (int32 Sim2DIndex = 0; Sim2DIndex < Sim2DToSKMIndex.Num(); ++Sim2DIndex)
		{
			const int32 Sim3DIndex = SimVertex3DLookup[Sim2DIndex];
			const int32 SKMIndex = Sim2DToSKMIndex[Sim2DIndex];
			SKMToSim3DLookup.Add(SKMIndex, Sim3DIndex);
		}
		return SKMToSim3DLookup;
	}
}

FChaosClothAssetSkeletalMeshImportNode_v2::FChaosClothAssetSkeletalMeshImportNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&SkeletalMesh);
	RegisterOutputConnection(&Collection);
}

void FChaosClothAssetSkeletalMeshImportNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
		FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.DefineSchema(EClothCollectionExtendedSchemas::Import);

		if (const USkeletalMesh* const InSkeletalMesh = GetValue(Context, &SkeletalMesh))
		{
			const FSkeletalMeshModel* ImportedModel = InSkeletalMesh->GetImportedModel();
			const bool bIsValidLOD = ImportedModel && ImportedModel->LODModels.IsValidIndex(LODIndex);
			if (!bIsValidLOD)
			{
				Context.Warning(
					FText::Format(
						LOCTEXT("InvalidLODDetails", "No valid LOD {0} found for skeletal mesh {1}."),
						LODIndex,
						FText::FromString(InSkeletalMesh->GetName()))
					, this, Out
				);

				SetValue(Context, MoveTemp(*ClothCollection), &Collection);
				return;
			}

			const FSkeletalMeshLODModel& LODModel = ImportedModel->LODModels[LODIndex];
			const int32 FirstSection = bImportSingleSection ? SectionIndex : 0;
			const int32 LastSection = bImportSingleSection ? SectionIndex : LODModel.Sections.Num() - 1;
			constexpr bool bImportSimMeshNormals = true;

			TArray<int32> Sim2DToSKMIndex;

			for (int32 Section = FirstSection; Section <= LastSection; ++Section)
			{
				const bool bIsValidSection = LODModel.Sections.IsValidIndex(Section);;
				if (!bIsValidSection)
				{
					Context.Warning(
						FText::Format(
							LOCTEXT("InvalidSectionDetails", "No valid section {0} found for skeletal mesh {1}."),
							Section,
							FText::FromString(InSkeletalMesh->GetName()))
						, this, Out
					);

					continue;
				}

				if (bImportSimMesh)
				{
					FClothDataflowTools::AddSimPatternsFromSkeletalMeshSection(ClothCollection, LODModel, Section, UVChannel, UVScale, bImportSimMeshNormals, &Sim2DToSKMIndex);
				}

				if (bImportRenderMesh)
				{
					const TArray<FSkeletalMaterial>& Materials = InSkeletalMesh->GetMaterials();

					const FSkeletalMeshLODInfo* const SkeletalMeshLODInfo = InSkeletalMesh->GetLODInfo(LODIndex);
					const int32 MaterialIndex =
						SkeletalMeshLODInfo &&
						SkeletalMeshLODInfo->LODMaterialMap.IsValidIndex(Section) &&
						SkeletalMeshLODInfo->LODMaterialMap[Section] != INDEX_NONE ?
							SkeletalMeshLODInfo->LODMaterialMap[Section] :
							LODModel.Sections[Section].MaterialIndex;

					const FString RenderMaterialPathName = Materials.IsValidIndex(MaterialIndex) && Materials[MaterialIndex].MaterialInterface ?
						 Materials[MaterialIndex].MaterialInterface->GetPathName() : "";

					FClothDataflowTools::AddRenderPatternFromSkeletalMeshSection(ClothCollection, LODModel, Section, RenderMaterialPathName);
				}
			}

			if (bImportSimMesh)
			{
				// Set SimImportVertexID
				TArrayView<int32> SimImportVertexID = ClothFacade.GetSimImportVertexID();
				for (int32 Vertex2DIndex = 0; Vertex2DIndex < SimImportVertexID.Num(); ++Vertex2DIndex)
				{
					if (Sim2DToSKMIndex.IsValidIndex(Vertex2DIndex) && LODModel.MeshToImportVertexMap.IsValidIndex(Sim2DToSKMIndex[Vertex2DIndex]))
					{
						SimImportVertexID[Vertex2DIndex] = LODModel.MeshToImportVertexMap[Sim2DToSKMIndex[Vertex2DIndex]];
					}
					else
					{
						SimImportVertexID[Vertex2DIndex] = INDEX_NONE;
					}
				}

				if (bImportSimMorphTargets)
				{
					// Use Sim2DToSKMIndex to get a map from SKMIndex to Sim3D index
					const TMap<int32, int32> SKMToSim3DLookup = Private::GenerateSKMToSim3DLookup(ClothFacade, Sim2DToSKMIndex);

					const TArray<TObjectPtr<UMorphTarget>>& MorphTargets = InSkeletalMesh->GetMorphTargets();
					for (const UMorphTarget* const MorphTarget : MorphTargets)
					{
						if (MorphTarget && MorphTarget->HasDataForLOD(LODIndex))
						{
							const FMorphTargetLODModel& MorphTargetLODModel = MorphTarget->GetMorphLODModels()[LODIndex];

							TArray<int32> MissingIndices;
							TArray<FVector3f> PositionDelta;
							TArray<FVector3f> NormalZDelta;
							TArray<int32> SimIndex;
							PositionDelta.Reserve(MorphTargetLODModel.NumVertices);
							NormalZDelta.Reserve(MorphTargetLODModel.NumVertices);
							SimIndex.Reserve(MorphTargetLODModel.NumVertices);
							for (const FMorphTargetDelta& Delta : MorphTargetLODModel.Vertices)
							{
								if (const int32* const SimIndex3D = SKMToSim3DLookup.Find(Delta.SourceIdx))
								{
									PositionDelta.Add(Delta.PositionDelta);
									NormalZDelta.Add(Delta.TangentZDelta);
									SimIndex.Add(*SimIndex3D);
								}
								else
								{
									MissingIndices.Add(Delta.SourceIdx);
								}
							}

							const int32 NumTargetVertices = PositionDelta.Num();
							check(NormalZDelta.Num() == NumTargetVertices);
							check(SimIndex.Num() == NumTargetVertices);
							if (NumTargetVertices > 0)
							{
								FCollectionClothSimMorphTargetFacade MorphTargetFacade = ClothFacade.AddGetSimMorphTarget();
								MorphTargetFacade.Initialize(MorphTarget->GetName(), PositionDelta, NormalZDelta, SimIndex);
								if (!MissingIndices.IsEmpty())
								{
									FString MissingIndicesString;
									for (int32 Index = 0; Index < MissingIndices.Num(); ++Index)
									{
										MissingIndicesString += FString::FromInt(MissingIndices[Index]);
										if (Index != MissingIndices.Num() - 1)
										{
											MissingIndicesString += ", ";
										}
									}
									Context.Warning(FString::Printf(TEXT("Failed to find corresponding sim vertex index for the following indices for morph target '%s': %s"), *MorphTarget->GetName(), *MissingIndicesString), this, Out);
								}
							}
							else
							{
								Context.Warning(FString::Printf(TEXT("Failed to import morph target '%s': no valid vertices found"), *MorphTarget->GetName()), this, Out);
							}
						}
					}
				}
			}

			FClothGeometryTools::CleanupAndCompactMesh(ClothCollection);

			if (const UPhysicsAsset* PhysicsAsset = bSetPhysicsAsset ? InSkeletalMesh->GetPhysicsAsset() : nullptr)
			{
				ClothFacade.SetPhysicsAssetSoftObjectPathName(PhysicsAsset->GetPathName());
			}

			ClothFacade.SetSkeletalMeshSoftObjectPathName(InSkeletalMesh->GetPathName());
		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}



FChaosClothAssetSkeletalMeshImportNode::FChaosClothAssetSkeletalMeshImportNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&SkeletalMesh);
	RegisterOutputConnection(&Collection);
}

void FChaosClothAssetSkeletalMeshImportNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
		FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.DefineSchema();

		if (const USkeletalMesh* const InSkeletalMesh = GetValue(Context, &SkeletalMesh))
		{
			const FSkeletalMeshModel* ImportedModel = InSkeletalMesh->GetImportedModel();
			const bool bIsValidLOD = ImportedModel && ImportedModel->LODModels.IsValidIndex(LODIndex);
			if (!bIsValidLOD)
			{
				Context.Warning(
					FText::Format(
						LOCTEXT("InvalidLODDetails", "No valid LOD {0} found for skeletal mesh {1}."),
						LODIndex,
						FText::FromString(InSkeletalMesh->GetName()))
					, this, Out
				);

				SetValue(Context, MoveTemp(*ClothCollection), &Collection);
				return;
			}

			const FSkeletalMeshLODModel &LODModel = ImportedModel->LODModels[LODIndex];
			const int32 FirstSection = bImportSingleSection ? SectionIndex : 0;
			const int32 LastSection = bImportSingleSection ? SectionIndex : LODModel.Sections.Num() - 1;
			constexpr bool bImportSimMeshNormals = false;

			for (int32 Section = FirstSection; Section <= LastSection; ++Section)
			{
				const bool bIsValidSection = LODModel.Sections.IsValidIndex(Section);;
				if (!bIsValidSection)
				{
					Context.Warning(
						FText::Format(
							LOCTEXT("InvalidSectionDetails", "No valid section {0} found for skeletal mesh {1}."),
							Section,
							FText::FromString(InSkeletalMesh->GetName()))
						, this, Out
					);

					continue;
				}

				if (bImportSimMesh)
				{
					FClothDataflowTools::AddSimPatternsFromSkeletalMeshSection(ClothCollection, LODModel, Section, UVChannel, UVScale, bImportSimMeshNormals);
				}

				if (bImportRenderMesh)
				{
					const TArray<FSkeletalMaterial>& Materials = InSkeletalMesh->GetMaterials();
					check(Section < Materials.Num());
					const FString RenderMaterialPathName = Materials[Section].MaterialInterface ? Materials[Section].MaterialInterface->GetPathName() : "";
					FClothDataflowTools::AddRenderPatternFromSkeletalMeshSection(ClothCollection, LODModel, Section, RenderMaterialPathName);
				}
			}

			if (const UPhysicsAsset* PhysicsAsset = bSetPhysicsAsset ? InSkeletalMesh->GetPhysicsAsset() : nullptr)
			{
				ClothFacade.SetPhysicsAssetSoftObjectPathName(PhysicsAsset->GetPathName());
			}

			// In order to retain existing behavior, flip the sim normals.
			constexpr bool bReverseSimMeshNormals = true;
			constexpr bool bReverseFalse = false;
			FClothGeometryTools::ReverseMesh(ClothCollection, bReverseSimMeshNormals, bReverseFalse, bReverseFalse, bReverseFalse, TArray<int32>(), TArray<int32>());

			ClothFacade.SetSkeletalMeshSoftObjectPathName(InSkeletalMesh->GetPathName());
		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

void FChaosClothAssetSkeletalMeshImportNode::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ClothAssetSkeletalMeshMultiSectionImport)
	{
		bImportSingleSection = true;
		bSetPhysicsAsset = true;
	}
}

#undef LOCTEXT_NAMESPACE
