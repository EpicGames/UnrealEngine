// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosClothAsset/SkinningBlendNode.h"
#include "ChaosClothAsset/ClothCollectionAttribute.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Utils/ClothingMeshUtils.h"
#include "PointWeightMap.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(SkinningBlendNode)
#define LOCTEXT_NAMESPACE "ChaosClothAssetSkinningBlendNode"
namespace UE::Chaos::ClothAsset::Private
{
	struct FSkinningBlendDataGenerator
	{
		FPointWeightMap PointWeightMap;
		TConstArrayView<TArray<FVector4f>> RenderDeformerPositionBaryCoordsAndDist;
		TConstArrayView<TArray<FIntVector3>> RenderDeformerSimIndices3D;
		TConstArrayView<TArray<float>> RenderDeformerWeight;
		TArrayView<float> RenderDeformerSkinningBlend;
		void Generate(bool bUseSmoothTransition)
		{
			const int32 NumPositions = RenderDeformerPositionBaryCoordsAndDist.Num();
			const int32 NumInfluences = NumPositions ? RenderDeformerPositionBaryCoordsAndDist[0].Num() : 0;
			// Rebuild MeshToMeshVertData, only needs PositionBaryCoordsAndDist, SourceMeshVertIndices, and Weight
			TArray<FMeshToMeshVertData> MeshToMeshVertData;
			MeshToMeshVertData.Reserve(NumPositions * NumInfluences);
			for (int32 Index = 0; Index < NumPositions; ++Index)
			{
				check(RenderDeformerPositionBaryCoordsAndDist[Index].Num() == NumInfluences);
				check(RenderDeformerSimIndices3D[Index].Num() == NumInfluences);
				check(RenderDeformerWeight[Index].Num() == NumInfluences);
				for (int32 Influence = 0; Influence < NumInfluences; ++Influence)
				{
					FMeshToMeshVertData MeshToMeshVertDatum;
					MeshToMeshVertDatum.PositionBaryCoordsAndDist = RenderDeformerPositionBaryCoordsAndDist[Index][Influence];
					for (int32 Vertex = 0; Vertex < 3; ++Vertex)
					{
						MeshToMeshVertDatum.SourceMeshVertIndices[Vertex] = RenderDeformerSimIndices3D[Index][Influence][Vertex];
					}
					MeshToMeshVertDatum.Weight = RenderDeformerWeight[Index][Influence];

					MeshToMeshVertData.Emplace(MoveTemp(MeshToMeshVertDatum));
				}
			}
			// Re-generate vertex contributions
			const bool bUseMultipleInfluences = (NumInfluences > 1);
			ClothingMeshUtils::ComputeVertexContributions(
				MeshToMeshVertData,
				&PointWeightMap,
				bUseSmoothTransition,
				bUseMultipleInfluences);
			// Copy back to the RenderDeformerSkinningBlend weight map
			for (int32 Index = 0; Index < NumPositions; ++Index)
			{
				RenderDeformerSkinningBlend[Index] = 0.f;
				for (int32 Influence = 0; Influence < NumInfluences; ++Influence)
				{
					const FMeshToMeshVertData& MeshToMeshVertDatum = MeshToMeshVertData[Index * NumInfluences + Influence];
					RenderDeformerSkinningBlend[Index] += MeshToMeshVertDatum.Weight * (float)MeshToMeshVertDatum.SourceMeshVertIndices[3] / (float)TNumericLimits<uint16>::Max();
				}
			}
		}
	};
	static FPointWeightMap KinematicSelectionToPointWeightMap(const FCollectionClothConstFacade& ClothFacade, const FCollectionClothSelectionConstFacade& SelectionFacade, const FName& KinematicSelectionName)
	{
		// Default init to dynamic/selected
		constexpr float SelectedValue = 1.f;
		FPointWeightMap PointWeightMap(ClothFacade.GetNumSimVertices3D(), SelectedValue);
		// Mark selected points as kinematic/unselected in the PointWeightMap
		if (const TSet<int32>* KinematicSelectionSet = SelectionFacade.IsValid() ? SelectionFacade.FindSelectionSet(KinematicSelectionName) : nullptr)
		{
			constexpr float UnSelectedValue = 0.f;
			const FName SelectionGroup = SelectionFacade.GetSelectionGroup(KinematicSelectionName);
			if (SelectionGroup == ClothCollectionGroup::SimVertices3D)
			{
				for (const int32 VertexIndex : *KinematicSelectionSet)
				{
					if (VertexIndex >= 0 && VertexIndex < PointWeightMap.Num())
					{
						PointWeightMap[VertexIndex] = UnSelectedValue;
					}
				}
				return PointWeightMap;
			}
			else if (SelectionGroup == ClothCollectionGroup::SimVertices2D)
			{
				const TConstArrayView<int32> Vertex2DTo3D = ClothFacade.GetSimVertex3DLookup();
				for (const int32 VertexIndex : *KinematicSelectionSet)
				{
					if (Vertex2DTo3D.IsValidIndex(VertexIndex))
					{
						// Indices in Vertex2DTo3D  are assumed to be valid for accessing PointWeightMap
						PointWeightMap[Vertex2DTo3D[VertexIndex]] = UnSelectedValue;
					}
				}
				return PointWeightMap;
			}
			else if (SelectionGroup == ClothCollectionGroup::SimFaces)
			{
				const TConstArrayView<FIntVector3> SimIndices3D = ClothFacade.GetSimIndices3D();
				for (const int32 FaceIndex : *KinematicSelectionSet)
				{
					if (SimIndices3D.IsValidIndex(FaceIndex))
					{
						// Indices in SimIndices3D are assumed to be valid to index the PointWeightMap array
						PointWeightMap[SimIndices3D[FaceIndex][0]] = UnSelectedValue;
						PointWeightMap[SimIndices3D[FaceIndex][1]] = UnSelectedValue;
						PointWeightMap[SimIndices3D[FaceIndex][2]] = UnSelectedValue;
					}
				}
				return PointWeightMap;
			}
		}
		// Invalid or no selection found, all points are dynamic (selected)
		return PointWeightMap;
	}
}  // End namespace UE::Chaos::ClothAsset::Private
FChaosClothAssetSkinningBlendNode::FChaosClothAssetSkinningBlendNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	using namespace UE::Chaos::ClothAsset;
	SkinningBlendName = ClothCollectionAttribute::RenderDeformerSkinningBlend.ToString();
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&KinematicVertices3D.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue));
	RegisterOutputConnection(&Collection)
		.SetPassthroughInput(&Collection);
	RegisterOutputConnection(&SkinningBlendName);
}
void FChaosClothAssetSkinningBlendNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;
		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		// Always check for a valid cloth collection/facade/sim mesh to avoid processing non cloth collections or pure render mesh cloth assets
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid() && ClothFacade.HasValidData())
		{
			FCollectionClothSelectionConstFacade SelectionFacade(ClothCollection);
			// Check for the optional render deformer schema
			if (!ClothFacade.IsValid(EClothCollectionExtendedSchemas::RenderDeformer))
			{
				Context.Warning(
					LOCTEXT("NoProxyDefromerDetails", "There isn't any Proxy Deformer mapping data on the input Cloth Collection to generate the Skinning Blend weight map.")
					, this, Out
				);
			}
			else
			{
				// Retrieve the SimVertexSelection name
				FName KinematicSelectionName = FName(*GetValue<FString>(Context, &KinematicVertices3D.StringValue));
				if (KinematicSelectionName != NAME_None && (!SelectionFacade.IsValid() || !SelectionFacade.FindSelectionSet(KinematicSelectionName)))
				{
					Context.Warning(
						LOCTEXT("HasSimVertexSelectionDetails", "The specified KinematicVertices3D selection does't exist within the input Cloth Collection. An empty selection of kinematic vertices will be used instead.")
						, this, Out
					);
					KinematicSelectionName = NAME_None;
				}
				// Create the render weight map for storing the skinning blend weights pattern per pattern, as the number of influences could vary
				for (int32 RenderPatternIndex = 0; RenderPatternIndex < ClothFacade.GetNumRenderPatterns(); ++RenderPatternIndex)
				{
					FCollectionClothRenderPatternFacade RenderPatternFacade = ClothFacade.GetRenderPattern(RenderPatternIndex);
					Private::FSkinningBlendDataGenerator DeformerMappingDataGenerator;
					DeformerMappingDataGenerator.PointWeightMap = Private::KinematicSelectionToPointWeightMap(ClothFacade, SelectionFacade, KinematicSelectionName);
					DeformerMappingDataGenerator.RenderDeformerPositionBaryCoordsAndDist = RenderPatternFacade.GetRenderDeformerPositionBaryCoordsAndDist();
					DeformerMappingDataGenerator.RenderDeformerSimIndices3D = RenderPatternFacade.GetRenderDeformerSimIndices3D();
					DeformerMappingDataGenerator.RenderDeformerWeight = RenderPatternFacade.GetRenderDeformerWeight();
					DeformerMappingDataGenerator.RenderDeformerSkinningBlend = RenderPatternFacade.GetRenderDeformerSkinningBlend();
					DeformerMappingDataGenerator.Generate(bUseSmoothTransition);
				}
			}
		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
	else if (Out->IsA<FString>(&SkinningBlendName))
	{
		SetValue(Context, SkinningBlendName, &SkinningBlendName);
	}
}
#undef LOCTEXT_NAMESPACE
