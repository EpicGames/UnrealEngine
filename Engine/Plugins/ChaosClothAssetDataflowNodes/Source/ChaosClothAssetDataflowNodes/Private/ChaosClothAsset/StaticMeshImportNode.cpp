// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/StaticMeshImportNode.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Engine/StaticMesh.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "SkeletalMeshAttributes.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshImportNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetStaticMeshImportNode"

namespace UE::Chaos::ClothAsset::Private
{
	static bool InitializeSimMeshFromSkeletalMeshModel(
		const FSkeletalMeshLODModel& SkeletalMeshLODModel,
		const TSharedRef<FManagedArrayCollection>& ClothCollection,
		int32 InSectionIndex,
		int32 UVChannelIndex,
		const FVector2f& UVScale,
		bool bImportNormals)
	{
		if (InSectionIndex == INDEX_NONE || SkeletalMeshLODModel.Sections.IsValidIndex(InSectionIndex))
		{
			const int32 StartSection = (InSectionIndex != INDEX_NONE) ? InSectionIndex : 0;
			const int32 EndSection = (InSectionIndex != INDEX_NONE) ? InSectionIndex + 1 : SkeletalMeshLODModel.Sections.Num();
			for (int32 SectionIndex = StartSection; SectionIndex < EndSection; ++SectionIndex)
			{
				FClothDataflowTools::AddSimPatternsFromSkeletalMeshSection(ClothCollection, SkeletalMeshLODModel, SectionIndex, UVChannelIndex, UVScale, bImportNormals);
			}
			return true;
		}
		return false;
	}

	static bool InitializeRenderMeshFromSkeletalMeshModel(
		const FSkeletalMeshLODModel& SkeletalMeshLODModel,
		const TSharedRef<FManagedArrayCollection>& ClothCollection,
		int32 InSectionIndex,
		const TFunctionRef<FString(int32 /*SectionIndex*/)>& GetMaterialPathNameFunction)
	{
		if (InSectionIndex == INDEX_NONE || SkeletalMeshLODModel.Sections.IsValidIndex(InSectionIndex))
		{
			const int32 StartSection = (InSectionIndex != INDEX_NONE) ? InSectionIndex : 0;
			const int32 EndSection = (InSectionIndex != INDEX_NONE) ? InSectionIndex + 1 : SkeletalMeshLODModel.Sections.Num();
			for (int32 SectionIndex = StartSection; SectionIndex < EndSection; ++SectionIndex)
			{
				const FString MaterialPathName = GetMaterialPathNameFunction(SectionIndex);
				FClothDataflowTools::AddRenderPatternFromSkeletalMeshSection(ClothCollection, SkeletalMeshLODModel, SectionIndex, MaterialPathName);
			}
			return true;
		}
		return false;
	}
} // namespace UE::Chaos::ClothAsset::Private


FChaosClothAssetStaticMeshImportNode_v2::FChaosClothAssetStaticMeshImportNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&StaticMesh);
	RegisterOutputConnection(&Collection);
}

void FChaosClothAssetStaticMeshImportNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;
	using namespace UE::Chaos::ClothAsset::Private;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Evaluate out collection
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
		FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.DefineSchema();

		const UStaticMesh* const InStaticMesh = GetValue(Context, &StaticMesh);

		if (InStaticMesh && (bImportSimMesh || bImportRenderMesh))
		{
			const int32 NumLods = InStaticMesh->GetNumSourceModels();
			if (LODIndex < NumLods)
			{
				const FMeshDescription* const MeshDescription = InStaticMesh->GetMeshDescription(LODIndex);
				check(MeshDescription);

				constexpr bool bImportSimMeshNormals = true;
				if (bImportSimMesh && SimMeshSection == INDEX_NONE)
				{
					// Keep legacy welding behavior when the entire static mesh is imported
					FMeshDescriptionToDynamicMesh Converter;
					Converter.bPrintDebugMessages = false;
					Converter.bEnableOutputGroups = false;
					Converter.bVIDsFromNonManifoldMeshDescriptionAttr = true;
					UE::Geometry::FDynamicMesh3 DynamicMesh;
					Converter.Convert(MeshDescription, DynamicMesh);
					constexpr bool bAppend = false;
					FClothGeometryTools::BuildSimMeshFromDynamicMesh(ClothCollection, DynamicMesh, UVChannel, UVScale, bAppend, bImportSimMeshNormals);
				}
				const bool bNeedsSkeletalMeshModel = bImportRenderMesh || (bImportSimMesh && SimMeshSection != INDEX_NONE);

				FSkeletalMeshLODModel SkeletalMeshLODModel;
				const FMeshBuildSettings& BuildSettings = InStaticMesh->GetSourceModel(LODIndex).BuildSettings;
				if (bNeedsSkeletalMeshModel &&
					FClothDataflowTools::BuildSkeletalMeshModelFromMeshDescription(MeshDescription, InStaticMesh->GetSourceModel(LODIndex).BuildSettings, SkeletalMeshLODModel))
				{
					if (bImportSimMesh && SimMeshSection != INDEX_NONE)
					{
						if (!InitializeSimMeshFromSkeletalMeshModel(SkeletalMeshLODModel, ClothCollection, SimMeshSection, UVChannel, UVScale, bImportSimMeshNormals))
						{
							Context.Warning(
								FText::Format(
									LOCTEXT("InvalidRenderMeshDetails", "The input static mesh {0} failed to convert to a valid render mesh."),
									FText::FromString(InStaticMesh->GetName()))
								, this, Out
							);
						}
					}
					if (bImportRenderMesh)
					{
						const FStaticMeshConstAttributes MeshAttributes(*MeshDescription);
						const TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();
						auto GetMaterialPathName = [&InStaticMesh, &SkeletalMeshLODModel, &MaterialSlotNames](int32 SectionIndex) -> FString
						{
							// Section MaterialIndex refers to the polygon group index. Look up which material this corresponds with.
							const FName& MaterialSlotName = MaterialSlotNames[SkeletalMeshLODModel.Sections[SectionIndex].MaterialIndex];
							const int32 MaterialIndex = InStaticMesh->GetMaterialIndexFromImportedMaterialSlotName(MaterialSlotName);
							const TArray<FStaticMaterial>& StaticMaterials = InStaticMesh->GetStaticMaterials();
							if (StaticMaterials.IsValidIndex(MaterialIndex) && StaticMaterials[MaterialIndex].MaterialInterface)
							{
								return StaticMaterials[MaterialIndex].MaterialInterface->GetPathName();
							}
							return FString();
						};

						if (!InitializeRenderMeshFromSkeletalMeshModel(SkeletalMeshLODModel, ClothCollection, RenderMeshSection, GetMaterialPathName))
						{
							Context.Warning(
								FText::Format(
									LOCTEXT("InvalidRenderMeshDetails", "The input static mesh {0} failed to convert to a valid render mesh."),
									FText::FromString(InStaticMesh->GetName()))
								, this, Out
							);
						}
					}
				}
				
				// Bind to root bone by default
				FClothGeometryTools::CleanupAndCompactMesh(ClothCollection);
				FClothGeometryTools::BindMeshToRootBone(ClothCollection, bImportSimMesh, bImportRenderMesh);

			}
			else
			{
				Context.Warning(
					FText::Format(LOCTEXT("InvalidLODIndexDetails",
						"{0} is not a valid LOD index for static mesh {1}.\n"
						"This static mesh has {2} LOD(s)."),
						FText::FromString(InStaticMesh->GetName()),
						LODIndex,
						NumLods)
					, this, Out
				);
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}


FChaosClothAssetStaticMeshImportNode::FChaosClothAssetStaticMeshImportNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&StaticMesh);
	RegisterOutputConnection(&Collection);
}

void FChaosClothAssetStaticMeshImportNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;
	using namespace UE::Chaos::ClothAsset::Private;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Evaluate out collection
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();
		FCollectionClothFacade ClothFacade(ClothCollection);
		ClothFacade.DefineSchema();

		const UStaticMesh* const InStaticMesh = GetValue(Context, &StaticMesh);

		if (InStaticMesh && (bImportSimMesh || bImportRenderMesh))
		{
			const int32 NumLods = InStaticMesh->GetNumSourceModels();
			if (LODIndex >= 0 && LODIndex < NumLods)
			{
				const FMeshDescription* const MeshDescription = InStaticMesh->GetMeshDescription(LODIndex);
				check(MeshDescription);

				constexpr bool bImportSimMeshNormals = false;
				if (bImportSimMesh && SimMeshSection == INDEX_NONE)
				{
					// Keep legacy welding behavior when the entire static mesh is imported
					FMeshDescriptionToDynamicMesh Converter;
					Converter.bPrintDebugMessages = false;
					Converter.bEnableOutputGroups = false;
					Converter.bVIDsFromNonManifoldMeshDescriptionAttr = true;
					UE::Geometry::FDynamicMesh3 DynamicMesh;
					Converter.Convert(MeshDescription, DynamicMesh);
					constexpr bool bAppend = false;
					FClothGeometryTools::BuildSimMeshFromDynamicMesh(ClothCollection, DynamicMesh, UVChannel, UVScale, bAppend, bImportSimMeshNormals);
				}
				const bool bNeedsSkeletalMeshModel = bImportRenderMesh || (bImportSimMesh && SimMeshSection != INDEX_NONE);

				FSkeletalMeshLODModel SkeletalMeshLODModel;
				const FMeshBuildSettings& BuildSettings = InStaticMesh->GetSourceModel(LODIndex).BuildSettings;
				if (bNeedsSkeletalMeshModel && 
					FClothDataflowTools::BuildSkeletalMeshModelFromMeshDescription(MeshDescription, InStaticMesh->GetSourceModel(LODIndex).BuildSettings, SkeletalMeshLODModel))
				{
					if (bImportSimMesh && SimMeshSection != INDEX_NONE)
					{
						if (!InitializeSimMeshFromSkeletalMeshModel(SkeletalMeshLODModel, ClothCollection, SimMeshSection, UVChannel, UVScale, bImportSimMeshNormals))
						{
							Context.Warning(
								FText::Format(
									LOCTEXT("InvalidRenderMeshDetails", "The input static mesh {0} failed to convert to a valid render mesh."),
									FText::FromString(InStaticMesh->GetName()))
								, this, Out
							);
						}
					}
					if (bImportRenderMesh)
					{
						const FStaticMeshConstAttributes MeshAttributes(*MeshDescription);
						const TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();
						auto GetMaterialPathName = [&InStaticMesh, &SkeletalMeshLODModel, &MaterialSlotNames](int32 SectionIndex) -> FString
							{
								// Section MaterialIndex refers to the polygon group index. Look up which material this corresponds with.
								const FName& MaterialSlotName = MaterialSlotNames[SkeletalMeshLODModel.Sections[SectionIndex].MaterialIndex];
								const int32 MaterialIndex = InStaticMesh->GetMaterialIndexFromImportedMaterialSlotName(MaterialSlotName);
								const TArray<FStaticMaterial>& StaticMaterials = InStaticMesh->GetStaticMaterials();
								if (StaticMaterials.IsValidIndex(MaterialIndex) && StaticMaterials[MaterialIndex].MaterialInterface)
								{
									return StaticMaterials[MaterialIndex].MaterialInterface->GetPathName();
								}
								return FString();
							};

						if (!InitializeRenderMeshFromSkeletalMeshModel(SkeletalMeshLODModel, ClothCollection, RenderMeshSection, GetMaterialPathName))
						{
							Context.Warning(
								FText::Format(
									LOCTEXT("InvalidRenderMeshDetails", "The input static mesh {0} failed to convert to a valid render mesh."),
									FText::FromString(InStaticMesh->GetName()))
								, this, Out
							);
						}
					}
				}
				// Bind to root bone by default
				FClothGeometryTools::BindMeshToRootBone(ClothCollection, bImportSimMesh, bImportRenderMesh);
				
				// In order to retain existing behavior, flip the sim normals.
				constexpr bool bReverseSimMeshNormals = true;
				constexpr bool bReverseFalse = false;
				FClothGeometryTools::ReverseMesh(ClothCollection, bReverseSimMeshNormals, bReverseFalse, bReverseFalse, bReverseFalse, TArray<int32>(), TArray<int32>());
				
			}
			else
			{
				Context.Warning(
					FText::Format(LOCTEXT("InvalidLODIndexDetails",
						"{0} is not a valid LOD index for static mesh {1}.\n"
						"This static mesh has {2} LOD(s)."),
						FText::FromString(InStaticMesh->GetName()),
						LODIndex,
						NumLods)
					, this, Out
				);
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
