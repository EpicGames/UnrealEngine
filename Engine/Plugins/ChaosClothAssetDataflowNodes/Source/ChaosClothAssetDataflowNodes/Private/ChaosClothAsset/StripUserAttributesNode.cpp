// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosClothAsset/StripUserAttributesNode.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StripUserAttributesNode)

FChaosClothStripUserAttributesNode::FChaosClothStripUserAttributesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	using namespace UE::Chaos::ClothAsset;
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection)
		.SetPassthroughInput(&Collection);
	RegisterInputConnection(&ExtraSimWeightMaps)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&ExtraSets)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&ExtraSimFaceIntMaps)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&ExtraRenderWeightMaps)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
}
void FChaosClothStripUserAttributesNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;
		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		// Always check for a valid cloth collection/facade/sim mesh to avoid processing non cloth collections or pure render mesh cloth assets
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid() && ClothFacade.HasValidData())
		{
			auto ConvertToFNameArray = [](const TArray<FString>& InStringArray)->TArray<FName>
				{
					TArray<FName> OutFNameArray;
					OutFNameArray.Reset(InStringArray.Num());
					for (const FString& InString : InStringArray)
					{
						OutFNameArray.Emplace(FName(InString));
					}
					return OutFNameArray;
				};

			UE::Chaos::ClothAsset::FClothGeometryTools::StripUnusedUserAttributes(ClothCollection,
				bStripUnusedSimWeightMaps, ConvertToFNameArray(GetValue<TArray<FString>>(Context, &ExtraSimWeightMaps)),
				bStripUnusedSets, ConvertToFNameArray(GetValue<TArray<FString>>(Context, &ExtraSets)),
				bStripUnusedSimFaceIntMaps, ConvertToFNameArray(GetValue<TArray<FString>>(Context, &ExtraSimFaceIntMaps)),
				bStripUnusedRenderWeightMaps, ConvertToFNameArray(GetValue<TArray<FString>>(Context, &ExtraRenderWeightMaps))
			);

			ClothFacade.SetCopyAllUserAttributesToSimModel(bCopyAllUserAttributesToSimModel);
		}
		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}
