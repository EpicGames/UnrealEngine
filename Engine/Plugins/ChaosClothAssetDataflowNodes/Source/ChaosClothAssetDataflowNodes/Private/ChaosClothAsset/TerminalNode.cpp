// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/TerminalNode.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/ClothLodTransitionDataCache.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Animation/Skeleton.h"
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Materials/Material.h"
#include "PhysicsEngine/PhysicsAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TerminalNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetTerminalNode"

namespace UE::Chaos::ClothAsset::Private
{
	uint32 CalculateClothChecksum(const TArray<TSharedRef<const FManagedArrayCollection>>& InClothCollections)
	{
		uint32 Checksum = 0;
		for (const TSharedRef<const FManagedArrayCollection>& ClothCollection : InClothCollections)
		{
			constexpr bool bIncludeWeightMapsTrue = true; // Currently, editing weight maps is destructive
			FCollectionClothConstFacade Cloth(ClothCollection);
			if (Cloth.HasValidRenderData())  // The cloth collection must at least have a render mesh
			{
				Checksum = Cloth.CalculateTypeHash(bIncludeWeightMapsTrue, Checksum);

				const TArray<FName> GroupNames = ClothCollection->GroupNames();
				for (const FName& GroupName : GroupNames)
				{
					Checksum = Cloth.CalculateUserDefinedAttributesTypeHash<int32>(GroupName, Checksum);
					Checksum = Cloth.CalculateUserDefinedAttributesTypeHash<float>(GroupName, Checksum);
					Checksum = Cloth.CalculateUserDefinedAttributesTypeHash<FVector3f>(GroupName, Checksum);
				}
			}
			FCollectionClothSelectionConstFacade Selection(ClothCollection);
			if (Selection.IsValid())
			{
				// Just checksum the sets that are SimVertex3D and SimFace sets since those are the only ones we care about right now
				const TArray<FName> SelectionNames = Selection.GetNames();
				for (const FName& SelectionName : SelectionNames)
				{
					const FName SelectionGroup = Selection.GetSelectionGroup(SelectionName);
					if (SelectionGroup == ClothCollectionGroup::SimVertices3D ||
						SelectionGroup == ClothCollectionGroup::SimFaces)
					{
						const TArray<int32> SelectionAsArray = Selection.GetSelectionSet(SelectionName).Array();
						Checksum = HashCombineFast(Checksum, GetTypeHash(SelectionName));
						Checksum = GetArrayHash(SelectionAsArray.GetData(), SelectionAsArray.Num(), Checksum);
					}
				}
			}
			const ::Chaos::Softs::FEmbeddedSpringFacade SpringFacade(const_cast<const FManagedArrayCollection&>(ClothCollection.Get()), ClothCollectionGroup::SimVertices3D);
			if (SpringFacade.IsValid())
			{
				Checksum = SpringFacade.CalculateTypeHash(Checksum);
			}
		}
		return Checksum;
	}

	bool ArePropertiesPreserved(const TArray<TSharedRef<const FManagedArrayCollection>>& Collections0, const TArray<TSharedRef<const FManagedArrayCollection>>& Collections1, bool& bNeedsReset)
	{
		using namespace ::Chaos::Softs;
		bNeedsReset = false;
		if (Collections0.Num() != Collections1.Num())
		{
			return false;
		}
		for (int32 LODIndex = 0; LODIndex < Collections0.Num(); ++LODIndex)
		{
			FCollectionPropertyConstFacade Property0(Collections0[LODIndex].ToSharedPtr());
			FCollectionPropertyConstFacade Property1(Collections1[LODIndex].ToSharedPtr());
			if (Property0.Num() != Property1.Num())
			{
				return false;
			}
			for (int32 PropertyIndex = 0; PropertyIndex < Property0.Num(); ++PropertyIndex)
			{
				if (Property0.GetKeyName(PropertyIndex) != Property1.GetKeyName(PropertyIndex))
				{
					return false;
				}
				if (Property0.IsIntrinsic(PropertyIndex) || Property1.IsIntrinsic(PropertyIndex))
				{
					if (!FCollectionPropertyConstFacade::AreValuesEqual(Property0, Property1, PropertyIndex))
					{
						return false;  // Intrinsic property change needs full rebuild
					}
				}
				if (!Property0.IsAnimatable(PropertyIndex) || !Property1.IsAnimatable(PropertyIndex))
				{
					if (!FCollectionPropertyConstFacade::AreValuesEqual(Property0, Property1, PropertyIndex))
					{
						bNeedsReset = true;  // Non animatable property change needs hard reset
					}
				}
			}
		}
		return true;
	}
}

FChaosClothAssetTerminalNode_v2::FChaosClothAssetTerminalNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
	, Refresh(FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& /*Context*/) { bClothCollectionChecksumValid = false; }))
{
	// Start with Lod0
	for (int32 Index = 0; Index < NumInitialCollectionLods; ++Index)
	{
		AddPins();
	}
	check(GetNumInputs() == NumRequiredInputs + NumInitialCollectionLods);  // Update NumRequiredInputs if you add more Inputs. This is used by Serialize.
}

void FChaosClothAssetTerminalNode_v2::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	if (UChaosClothAsset* ClothAsset = Cast<UChaosClothAsset>(Asset.Get()))
	{
		using namespace UE::Chaos::ClothAsset;

		// Check whether the context has just been cleared up, in which case force a rebuild by invalidating the checksum
		if (Context.IsEmpty())
		{
			bClothCollectionChecksumValid = false;
		}

		TArray<TSharedRef<const FManagedArrayCollection>> InClothCollections = GetCleanedCollectionLodValues(Context);
		TArray<TSharedRef<const FManagedArrayCollection>> ClothCollections = static_cast<const UChaosClothAsset*>(ClothAsset)->GetClothCollections();

		const uint32 PreviousChecksum = ClothColllectionChecksum;
		const bool bPreviousChecksumsValid = bClothCollectionChecksumValid;
		ClothColllectionChecksum = Private::CalculateClothChecksum(InClothCollections);
		bClothCollectionChecksumValid = InClothCollections.Num() > 0;

		bool bNeedsReset;
		if (bPreviousChecksumsValid && PreviousChecksum == ClothColllectionChecksum &&
			Private::ArePropertiesPreserved(InClothCollections, ClothCollections, bNeedsReset))
		{
			// Cloth and property keys match. Just update property values.
			check(InClothCollections.Num() == ClothCollections.Num());
			check(ClothCollections.Num() > 0);
			for (int32 LODIndex = 0; LODIndex < InClothCollections.Num(); ++LODIndex)
			{
				TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(*ClothCollections[LODIndex]);
				Chaos::Softs::FCollectionPropertyFacade Properties(ClothCollection);
				Properties.UpdateProperties(InClothCollections[LODIndex].ToSharedPtr());
				ClothCollections[LODIndex] = MoveTemp(ClothCollection);
			}

			ClothAsset->SetClothCollections(MoveTemp(ClothCollections), bNeedsReset);

			// Asset must be resaved
			ClothAsset->MarkPackageDirty();

			return;
		}

		FText ErrorText;
		FText VerboseText;
		ClothAsset->Build(InClothCollections, &LODTransitionDataCache, &ErrorText, &VerboseText);

		if (!VerboseText.IsEmpty())
		{
			Context.Warning(VerboseText, this);
		}

		// Asset must be resaved
		ClothAsset->MarkPackageDirty();
	}
}

TArray<UE::Dataflow::FPin> FChaosClothAssetTerminalNode_v2::AddPins()
{
	const int32 Index = CollectionLods.AddDefaulted();
	const FDataflowInput& Input = RegisterInputArrayConnection(GetConnectionReference(Index));
	return { { UE::Dataflow::FPin::EDirection::INPUT, Input.GetType(), Input.GetName() } };
}

TArray<UE::Dataflow::FPin> FChaosClothAssetTerminalNode_v2::GetPinsToRemove() const
{
	const int32 Index = CollectionLods.Num() - 1;
	check(CollectionLods.IsValidIndex(Index));
	if (const FDataflowInput* const Input = FindInput(GetConnectionReference(Index)))
	{
		return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
	}
	return Super::GetPinsToRemove();
}

void FChaosClothAssetTerminalNode_v2::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	const int32 Index = CollectionLods.Num() - 1;
	check(CollectionLods.IsValidIndex(Index));
#if DO_CHECK
	const FDataflowInput* const Input = FindInput(GetConnectionReference(Index));
	check(Input);
	check(Input->GetName() == Pin.Name);
	check(Input->GetType() == Pin.Type);
#endif
	CollectionLods.SetNum(Index);

	return Super::OnPinRemoved(Pin);
}

TArray<TSharedRef<const FManagedArrayCollection>> FChaosClothAssetTerminalNode_v2::GetCleanedCollectionLodValues(UE::Dataflow::FContext& Context) const
{
	using namespace UE::Chaos::ClothAsset;

	TArray<TSharedRef<const FManagedArrayCollection>> CollectionLodValues;
	CollectionLodValues.Reserve(CollectionLods.Num());

	int32 LastValidLodIndex = INDEX_NONE;
	for (int32 LodIndex = 0; LodIndex < CollectionLods.Num(); ++LodIndex)
	{
		TSharedRef<FManagedArrayCollection> CollectionLodValue = MakeShared<FManagedArrayCollection>(GetValue<FManagedArrayCollection>(Context, GetConnectionReference(LodIndex)));

		FCollectionClothFacade ClothFacade(CollectionLodValue);
		if (ClothFacade.HasValidRenderData() || ClothFacade.HasValidSimulationData())  // The cloth collection must at least have a mesh
		{
			FClothGeometryTools::CleanupAndCompactMesh(CollectionLodValue);
			LastValidLodIndex = LodIndex;
		}
		else if (LastValidLodIndex >= 0)
		{
			ClothFacade.DefineSchema();
			ClothFacade.Initialize(FCollectionClothConstFacade(CollectionLodValues[LastValidLodIndex]));

			Context.Warning(
				FText::Format(
					LOCTEXT("InvalidInputLodNDetails",
						"Invalid or empty input LOD for LOD {0}.\n"
						"Using the previous valid LOD {1} instead."),
					LodIndex,
					LastValidLodIndex)
				, this
			);
		}
		else
		{
			Context.Warning(
				LOCTEXT("InvalidInputLod0Details",
					"Invalid or empty input LOD for LOD 0.\n"
					"LOD 0 cannot be empty in order to construct a valid Cloth Asset.")
				, this
			);
			break;
		}
		CollectionLodValues.Emplace(MoveTemp(CollectionLodValue));
	}
	return CollectionLodValues;
}

UE::Dataflow::TConnectionReference<FManagedArrayCollection> FChaosClothAssetTerminalNode_v2::GetConnectionReference(int32 Index) const
{
	return { &CollectionLods[Index], Index, &CollectionLods };
}

void FChaosClothAssetTerminalNode_v2::PostSerialize(const FArchive& Ar)
{
	// because we add pins we need to make sure we restore them when loading
	// to make sure they can get properly reconnected

	if (Ar.IsLoading())
	{
		if (CollectionLods.Num() < NumInitialCollectionLods)
		{
			CollectionLods.SetNum(NumInitialCollectionLods);  // In case the FManagedArrayCollection wasn't serialized with the node (pre the WithSerializer trait)
		}
		for (int32 Index = 0; Index < NumInitialCollectionLods; ++Index)
		{
			check(FindInput(GetConnectionReference(Index)));
		}

		for (int32 Index = NumInitialCollectionLods; Index < CollectionLods.Num(); ++Index)
		{
			FindOrRegisterInputArrayConnection(GetConnectionReference(Index));
		}
		if (Ar.IsTransacting())
		{
			const int32 OrigNumRegisteredInputs = GetNumInputs();
			check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialCollectionLods);
			const int32 OrigNumCollections = CollectionLods.Num();
			const int32 OrigNumRegisteredCollections = (OrigNumRegisteredInputs - NumRequiredInputs);
			if (OrigNumRegisteredCollections > OrigNumCollections)
			{
				// Inputs have been removed.
				// Temporarily expand Collections so we can get connection references.
				CollectionLods.SetNum(OrigNumRegisteredCollections);
				for (int32 Index = OrigNumCollections; Index < CollectionLods.Num(); ++Index)
				{
					UnregisterInputConnection(GetConnectionReference(Index));
				}
				CollectionLods.SetNum(OrigNumCollections);
			}
		}
		else
		{
			ensureAlways(CollectionLods.Num() + NumRequiredInputs == GetNumInputs());
		}
	}
}




FChaosClothAssetTerminalNode::FChaosClothAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	RegisterInputConnection(&CollectionLod0);
	check(NumInitialCollectionLods + NumRequiredInputs == GetNumInputs()); // Update NumRequiredInputs if you add more Inputs. This is used by Serialize.
}

void FChaosClothAssetTerminalNode::SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const
{
	if (UChaosClothAsset* ClothAsset = Cast<UChaosClothAsset>(Asset.Get()))
	{
		using namespace UE::Chaos::ClothAsset;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (RefreshAsset.bRefreshAsset)
		{
			bClothCollectionChecksumValid = false;
			RefreshAsset.bRefreshAsset = false;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		TArray<TSharedRef<const FManagedArrayCollection>> InClothCollections = GetCleanedCollectionLodValues(Context);
		const TArray<TSharedRef<const FManagedArrayCollection>>& RefClothCollections = static_cast<const UChaosClothAsset*>(ClothAsset)->GetClothCollections();

		const uint32 PreviousChecksum = ClothColllectionChecksum;
		const bool bPreviousChecksumsValid = bClothCollectionChecksumValid;
		ClothColllectionChecksum = Private::CalculateClothChecksum(InClothCollections);
		bClothCollectionChecksumValid = InClothCollections.Num() > 0;

		bool bNeedsReset;
		if (bPreviousChecksumsValid && PreviousChecksum == ClothColllectionChecksum &&
			Private::ArePropertiesPreserved(InClothCollections, RefClothCollections, bNeedsReset))
		{
			// Cloth and property keys match. Just update property values.
			TArray<TSharedRef<const FManagedArrayCollection>> ClothCollections = RefClothCollections;  // Copy on write
			check(InClothCollections.Num() == ClothCollections.Num());
			check(ClothCollections.Num() > 0);
			for (int32 LODIndex = 0; LODIndex < InClothCollections.Num(); ++LODIndex)
			{
				TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(*ClothCollections[LODIndex]);
				Chaos::Softs::FCollectionPropertyFacade Properties(ClothCollection);
				Properties.UpdateProperties(InClothCollections[LODIndex].ToSharedPtr());
				ClothCollections[LODIndex] = MoveTemp(ClothCollection);
			}

			ClothAsset->SetClothCollections(MoveTemp(ClothCollections), bNeedsReset);

			// Asset must be resaved
			ClothAsset->MarkPackageDirty();

			return;
		}

		FText ErrorText;
		FText VerboseText;
		ClothAsset->Build(InClothCollections, &LODTransitionDataCache, &ErrorText, &VerboseText);

		if (!VerboseText.IsEmpty())
		{
			Context.Warning(VerboseText, this);
		}

		// Asset must be resaved
		ClothAsset->MarkPackageDirty();
	}
}

TArray<UE::Dataflow::FPin> FChaosClothAssetTerminalNode::AddPins()
{
	auto AddInput = [this](const FManagedArrayCollection* Collection) -> TArray<UE::Dataflow::FPin>
		{
			RegisterInputConnection(Collection);
			const FDataflowInput* const Input = FindInput(Collection);
			return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
		};

	switch (NumLods)
	{
	case 1: ++NumLods; return AddInput(&CollectionLod1);
	case 2: ++NumLods; return AddInput(&CollectionLod2);
	case 3: ++NumLods; return AddInput(&CollectionLod3);
	case 4: ++NumLods; return AddInput(&CollectionLod4);
	case 5: ++NumLods; return AddInput(&CollectionLod5);
	default: break;
	}

	return Super::AddPins();
}

TArray<UE::Dataflow::FPin> FChaosClothAssetTerminalNode::GetPinsToRemove() const
{
	auto PinToRemove = [this](const FManagedArrayCollection* Collection) -> TArray<UE::Dataflow::FPin>
		{
			const FDataflowInput* const Input = FindInput(Collection);
			check(Input);
			return { { UE::Dataflow::FPin::EDirection::INPUT, Input->GetType(), Input->GetName() } };
		};

	switch (NumLods - 1)
	{
	case 1: return PinToRemove(&CollectionLod1);
	case 2: return PinToRemove(&CollectionLod2);
	case 3: return PinToRemove(&CollectionLod3);
	case 4: return PinToRemove(&CollectionLod4);
	case 5: return PinToRemove(&CollectionLod5);
	default: break;
	}
	return Super::GetPinsToRemove();
}

void FChaosClothAssetTerminalNode::OnPinRemoved(const UE::Dataflow::FPin& Pin)
{
	auto CheckPinRemoved = [this, &Pin](const FManagedArrayCollection* Collection)
	{
		check(Pin.Direction == UE::Dataflow::FPin::EDirection::INPUT);
#if DO_CHECK
		const FDataflowInput* const Input = FindInput(Collection);
		check(Input);
		check(Input->GetName() == Pin.Name);
		check(Input->GetType() == Pin.Type);
#endif
	};

	switch (NumLods - 1)
	{
	case 1:
		CheckPinRemoved(&CollectionLod1);
		--NumLods; 
		break;
	case 2:
		CheckPinRemoved(&CollectionLod2);
		--NumLods;
		break;
	case 3:
		CheckPinRemoved(&CollectionLod3);
		--NumLods;
		break;
	case 4:
		CheckPinRemoved(&CollectionLod4);
		--NumLods;
		break;
	case 5:
		CheckPinRemoved(&CollectionLod5);
		--NumLods;
		break;
	default: 
		checkNoEntry();
		break;
	}

	return Super::OnPinRemoved(Pin);
}

TArray<const FManagedArrayCollection*> FChaosClothAssetTerminalNode::GetCollectionLods() const
{
	TArray<const FManagedArrayCollection*> CollectionLods;
	CollectionLods.SetNumUninitialized(NumLods);

	for (int32 LodIndex = 0; LodIndex < NumLods; ++LodIndex)
	{
		switch (LodIndex)
		{
		case 0: CollectionLods[LodIndex] = &CollectionLod0; break;
		case 1: CollectionLods[LodIndex] = &CollectionLod1; break;
		case 2: CollectionLods[LodIndex] = &CollectionLod2; break;
		case 3: CollectionLods[LodIndex] = &CollectionLod3; break;
		case 4: CollectionLods[LodIndex] = &CollectionLod4; break;
		case 5: CollectionLods[LodIndex] = &CollectionLod5; break;
		default: check(false); break;
		}
	}
	return CollectionLods;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // Unexpected deprecation message on some platforms otherwise
const FManagedArrayCollection* FChaosClothAssetTerminalNode::GetCollectionLod(int32 LodIndex) const
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	switch (LodIndex)
	{
	case 0: return &CollectionLod0;
	case 1: return &CollectionLod1; 
	case 2: return &CollectionLod2; 
	case 3: return &CollectionLod3; 
	case 4: return &CollectionLod4; 
	case 5: return &CollectionLod5; 
	default: check(false); return nullptr;
	}
}

TArray<TSharedRef<const FManagedArrayCollection>> FChaosClothAssetTerminalNode::GetCleanedCollectionLodValues(UE::Dataflow::FContext& Context) const
{
	using namespace UE::Chaos::ClothAsset;

	const TArray<const FManagedArrayCollection*> CollectionLods = GetCollectionLods();
	TArray<TSharedRef<const FManagedArrayCollection>> CollectionLodValues;
	CollectionLodValues.Reserve(CollectionLods.Num());

	int32 LastValidLodIndex = INDEX_NONE;
	for (int32 LodIndex = 0; LodIndex < CollectionLods.Num(); ++LodIndex)
	{
		TSharedRef<FManagedArrayCollection> CollectionLodValue = MakeShared<FManagedArrayCollection>(GetValue<FManagedArrayCollection>(Context, CollectionLods[LodIndex]));

		FCollectionClothFacade ClothFacade(CollectionLodValue);
		if (ClothFacade.HasValidRenderData())  // The cloth collection must at least have a render mesh
		{
			FClothGeometryTools::CleanupAndCompactMesh(CollectionLodValue);
			LastValidLodIndex = LodIndex;
		}
		else if (LastValidLodIndex >= 0)
		{
			ClothFacade.DefineSchema();
			ClothFacade.Initialize(FCollectionClothConstFacade(CollectionLodValues[LastValidLodIndex]));

			Context.Warning(
				FText::Format(
					LOCTEXT("InvalidInputLodNDetails",
						"Invalid or empty input LOD for LOD {0}.\n"
						"Using the previous valid LOD {1} instead."),
					LodIndex,
					LastValidLodIndex)
				, this
			);
		}
		else
		{
			Context.Warning(
				LOCTEXT("InvalidInputLod0Details",
					"Invalid or empty input LOD for LOD 0.\n"
					"LOD 0 cannot be empty in order to construct a valid Cloth Asset.")
				, this
			);
			break;
		}
		CollectionLodValues.Emplace(MoveTemp(CollectionLodValue));
	}
	return CollectionLodValues;
}

void FChaosClothAssetTerminalNode::PostSerialize(const FArchive& Ar)
{
	// because we add pins we need to make sure we restore them when loading
	// to make sure they can get properly reconnected

	if (Ar.IsLoading())
	{
		const int32 OrigNumRegisteredInputs = GetNumInputs();
		check(OrigNumRegisteredInputs >= NumRequiredInputs + NumInitialCollectionLods);
		const int32 OrigNumLods = NumLods;
		const int32 OrigNumRegisteredLods = OrigNumRegisteredInputs - NumRequiredInputs;
		const int32 NumLodToAdd = (OrigNumLods - OrigNumRegisteredLods);
		check(Ar.IsTransacting() || OrigNumRegisteredLods == NumInitialCollectionLods);
		if (NumLodToAdd > 0)
		{
			NumLods = OrigNumRegisteredLods;  // AddPin will increment it again
			for (int32 LodIndex = 0; LodIndex < NumLodToAdd; ++LodIndex)
			{
				AddPins();
			}
		}
		else if (NumLodToAdd < 0)
		{
			check(Ar.IsTransacting());
			for (int32 Index = NumLods; Index < OrigNumRegisteredLods; ++Index)
			{
				UnregisterInputConnection(GetCollectionLod(Index));
			}

		}
		check(NumLods + NumRequiredInputs == GetNumInputs());
	}
}
#undef LOCTEXT_NAMESPACE
