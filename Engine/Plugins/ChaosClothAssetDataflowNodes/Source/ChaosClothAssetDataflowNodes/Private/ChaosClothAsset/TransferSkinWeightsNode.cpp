// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/TransferSkinWeightsNode.h"
#include "ChaosClothAsset/ClothPatternToDynamicMesh.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Animation/Skeleton.h"
#include "Dataflow/DataflowInputOutput.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/NonManifoldMappingSupport.h"
#include "Engine/SkeletalMesh.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Operations/TransferBoneWeights.h"
#include "Rendering/SkeletalMeshLODImporterData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "SkeletalMeshAttributes.h"
#include "Selections/MeshConnectedComponents.h"
#include "BoneWeights.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SkeletalMeshLODRenderDataToDynamicMesh.h"
#include "Tasks/Task.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransferSkinWeightsNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetTransferSkinWeightsNode"

static const FName InpaintWeightMaskName(TEXT("_InpaintWeightMask"));

namespace UE::Geometry
{
	class FDynamicMesh3;
	template<typename MeshType> class TMeshAABBTree3;
	typedef TMeshAABBTree3<FDynamicMesh3> FDynamicMeshAABBTree3;
}

namespace UE::Chaos::ClothAsset::Private
{	
	/** Helper struct to pass the transfer settings around */
	struct TransferBoneWeightsSettings
	{
		/** 
		 * Settings for controlling which meshes to transfer to and from. 
		 */
		bool bTransferToSim = true; 	// if true, transfer to sim mesh, otherwise skip sim mesh
		bool bTransferToRender = true;  // if true, transfer to render mesh, otherwise skip render mesh
		bool bTransferToRenderFromSim = true; // if true, for render mesh only, transfer from the sim mesh, otherwise transfer from the source skeletal mesh

		/** 
		 * Shared Transfer Operator Settings 
		 */
		bool bUseParallel = false;
		int32 MaxNumInfluences = 8;
		UE::Geometry::FTransferBoneWeights::ETransferBoneWeightsMethod TransferMethod = UE::Geometry::FTransferBoneWeights::ETransferBoneWeightsMethod::ClosestPointOnSurface;

		/** 
		 * Settings for the ETransferBoneWeightsMethod::InpaintWeights transfer method
		 */
		double NormalThreshold = 0;
		double RadiusPercentage = 0;
		bool LayeredMeshSupport = false;
		int32 NumSmoothingIterations = 0;
		double SmoothingStrength = 0;
		FString InpaintMaskWeightMapName;
	};

	UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute* GetOrCreateSkinWeightsAttribute(FDynamicMesh3& InMesh, const FName& InProfileName="Default")
	{
		checkSlow(InMesh.HasAttributes());
		UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute* Attribute = InMesh.Attributes()->GetSkinWeightsAttribute(InProfileName);
		if (Attribute == nullptr)
		{
			Attribute = new UE::Geometry::FDynamicMeshVertexSkinWeightsAttribute(&InMesh);
			InMesh.Attributes()->AttachSkinWeightsAttribute(InProfileName, Attribute);
		}
		return Attribute;
	}

	/** 
	 * Convert the USkeletalMesh to DynamicMesh. If requested LOD was auto-generated, will convert render data instead of the mesh description.
	 */
	static bool SkeletalMeshToDynamicMesh(const USkeletalMesh* SkeletalMesh, const int32 LodIndex, FDynamicMesh3& ToDynamicMesh)
	{
		if (SkeletalMesh->HasMeshDescription(LodIndex))
		{
			const FMeshDescription* SourceMesh = SkeletalMesh->GetMeshDescription(LodIndex);
			if (!SourceMesh)
			{
				return false;
			}

			FMeshDescriptionToDynamicMesh Converter;
			Converter.Convert(SourceMesh, ToDynamicMesh);
		}
		else
		{
			const FSkeletalMeshRenderData* RenderData = SkeletalMesh->GetResourceForRendering();
			if (!RenderData)
			{
				return false;
			}

			if (!RenderData->LODRenderData.IsValidIndex(LodIndex))
			{
				return false;
			}

			const FSkeletalMeshLODRenderData* SkeletalMeshLODRenderData = &(RenderData->LODRenderData[LodIndex]);

			UE::Geometry::FSkeletalMeshLODRenderDataToDynamicMesh::ConversionOptions ConversionOptions;
			ConversionOptions.bWantUVs = false;
			ConversionOptions.bWantVertexColors = false;
			ConversionOptions.bWantMaterialIDs = false;
			ConversionOptions.bWantSkinWeights = true;

			UE::Geometry::FSkeletalMeshLODRenderDataToDynamicMesh::Convert(SkeletalMeshLODRenderData, SkeletalMesh->GetRefSkeleton(), ConversionOptions, ToDynamicMesh);
		}

		return true;
	}

	/** Convert the ClothCollection to DynamicMesh. */
	static bool ClothToDynamicMesh(
		const TSharedRef<const FManagedArrayCollection>& ClothCollection,
		const FReferenceSkeleton& TargetRefSkeleton, // the reference skeleton to add to the dynamic mesh
		const bool bIsSim, // if true, Mesh will contain the sim mesh, otherwise the render mesh
		UE::Geometry::FDynamicMesh3& Mesh) // the resulting sim or render mesh
	{
		using namespace UE::Geometry;
		using namespace UE::Chaos::ClothAsset;

		// Check if ClothCollection is empty
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		const int32 NumVertices = bIsSim ? ClothFacade.GetNumSimVertices3D() : ClothFacade.GetNumRenderVertices();
		const int32 NumFaces = bIsSim ? ClothFacade.GetNumSimFaces() :  ClothFacade.GetNumRenderFaces();

		if (NumVertices <= 0 || NumFaces <= 0)
		{
			UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Failed to convert the Cloth Collection to Dynamic Mesh. Cloth Collection is empty.");
			return false;
		}

		// Convert the sim mesh to DynamicMesh. 
		FClothPatternToDynamicMesh PatternToDynamicMesh;
		constexpr bool bDisableAttributes = false;
		const EClothPatternVertexType PatternType = bIsSim ? EClothPatternVertexType::Sim3D : EClothPatternVertexType::Render;
		PatternToDynamicMesh.Convert(ClothCollection, INDEX_NONE, PatternType, Mesh, bDisableAttributes);

		// Setup the skeleton.
		// @note we can't simply copy the bone attributes from the source USkeletalMesh because the cloth  
		// asset reference skeleton comes from the USkeleton not the USkeletalMesh
		Mesh.Attributes()->EnableBones(TargetRefSkeleton.GetRawBoneNum());
		for (int32 BoneIdx = 0; BoneIdx < TargetRefSkeleton.GetRawBoneNum(); ++BoneIdx)
		{
			Mesh.Attributes()->GetBoneNames()->SetValue(BoneIdx, TargetRefSkeleton.GetRawRefBoneInfo()[BoneIdx].Name);
		}

		return true;
	}

	/** Copy the skin weights from DynamicMesh to Render Collection. */
	static void CopySkinWeightsFromDynamicMeshToRenderCloth(const FDynamicMesh3& RenderMesh,
															const bool bUseParallel,
													  	 	const TSharedRef<FManagedArrayCollection>& ClothCollection)
	{
		using namespace UE::Geometry;

		UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(ClothCollection);

		ParallelFor(RenderMesh.MaxVertexID(), [&ClothFacade, &RenderMesh](int32 VertexID)
		{
			const FDynamicMeshVertexSkinWeightsAttribute* OutAttribute = RenderMesh.Attributes()->GetSkinWeightsAttribute(FSkeletalMeshAttributes::DefaultSkinWeightProfileName);

			checkSlow(OutAttribute);
			checkSlow(RenderMesh.IsVertex(VertexID));
			checkSlow(VertexID < ClothFacade.GetNumRenderVertices());
			OutAttribute->GetValue(VertexID, ClothFacade.GetRenderBoneIndices()[VertexID],
				ClothFacade.GetRenderBoneWeights()[VertexID]);

		}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
	}

	/** Copy the skin weights from DynamicMesh to Sim Collection, handling split vertices. */
	static void CopySkinWeightsFromDynamicMeshToSimCloth(const FDynamicMesh3& WeldedSimMesh, 
													  const bool bUseParallel,
													  const TSharedRef<FManagedArrayCollection>& ClothCollection,
													  const int32 MaxNumInfluences)
	{
		using namespace UE::Geometry;

		UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(ClothCollection);

		//
		// Copy the new bone weight data from the welded sim mesh back to the cloth patterns.
		//
		const FNonManifoldMappingSupport NonManifoldMapping(WeldedSimMesh);
		TArray<TArray<int32>> SimMeshToDynamicMesh;
		if (NonManifoldMapping.IsNonManifoldVertexInSource())
		{
			// WeldedSimMesh indices don't match cloth collection. 
			SimMeshToDynamicMesh.SetNum(ClothFacade.GetNumSimVertices3D());
			for (int32 DynamicMeshVert = 0; DynamicMeshVert < WeldedSimMesh.VertexCount(); ++DynamicMeshVert)
			{
				SimMeshToDynamicMesh[NonManifoldMapping.GetOriginalNonManifoldVertexID(DynamicMeshVert)].Add(DynamicMeshVert);
			}

			ParallelFor(ClothFacade.GetNumSimVertices3D(), [&ClothFacade, &WeldedSimMesh, &SimMeshToDynamicMesh, MaxNumInfluences](int32 SimVertexID)
			{
				const FDynamicMeshVertexSkinWeightsAttribute* const OutAttribute = WeldedSimMesh.Attributes()->GetSkinWeightsAttribute(FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
				checkSlow(OutAttribute);
				if (!ensure(SimMeshToDynamicMesh[SimVertexID].Num() > 0))
				{
					ClothFacade.GetSimBoneIndices()[SimVertexID].Reset();
					ClothFacade.GetSimBoneWeights()[SimVertexID].Reset();
					return;
				}
				if (SimMeshToDynamicMesh[SimVertexID].Num() == 1)
				{
					// Simple most common case, one-to-one correspondence. Just copy over.
					const int32 WeldedID = SimMeshToDynamicMesh[SimVertexID][0];
					checkSlow(WeldedSimMesh.IsVertex(WeldedID));
					OutAttribute->GetValue(WeldedID, ClothFacade.GetSimBoneIndices()[SimVertexID],
						ClothFacade.GetSimBoneWeights()[SimVertexID]);
				}
				else
				{					
					// Need to merge data because dynamic mesh split the original vertex
					TMap<int32, TPair<float, int32>> CombinedData;
					for (const int32 WeldedID : SimMeshToDynamicMesh[SimVertexID])
					{
						TArray<int32> Indices;
						TArray<float> Weights;
						checkSlow(WeldedSimMesh.IsVertex(WeldedID));
						OutAttribute->GetValue(WeldedID, Indices, Weights);
						check(Indices.Num() == Weights.Num());
						for (int32 Idx = 0; Idx < Indices.Num(); ++Idx)
						{
							TPair<float, int32>& WeightedFloat = CombinedData.FindOrAdd(Indices[Idx]);
							WeightedFloat.Get<0>() += Weights[Idx];
							WeightedFloat.Get<1>() += 1;
						}
					}
					TArray<int32>& BoneIndices = ClothFacade.GetSimBoneIndices()[SimVertexID];
					TArray<float>& BoneWeights = ClothFacade.GetSimBoneWeights()[SimVertexID];
					BoneIndices.Reset(CombinedData.Num());
					BoneWeights.Reset(CombinedData.Num());
					float WeightsSum = 0.f;
					for (TMap<int32, TPair<float, int32>>::TConstIterator CombinedDataIter = CombinedData.CreateConstIterator(); CombinedDataIter; ++CombinedDataIter)
					{
						check(CombinedDataIter.Value().Get<1>() > 0);
						BoneIndices.Add(CombinedDataIter.Key());
						const float FloatVal = CombinedDataIter.Value().Get<0>() / (float)CombinedDataIter.Value().Get<1>();
						BoneWeights.Add(FloatVal);
						WeightsSum += FloatVal;
					}
					if (BoneIndices.Num() > MaxNumInfluences)
					{
						// Choose MaxNumInfluences highest weighted bones.
						TArray<TPair<float, int32>> SortableData;
						SortableData.Reserve(BoneIndices.Num());
						for (int32 Idx = 0; Idx < BoneIndices.Num(); ++Idx)
						{
							SortableData.Emplace(BoneWeights[Idx], BoneIndices[Idx]);
						}
						SortableData.Sort([](const TPair<float, int32>& A, const TPair<float, int32>& B) { return A > B; });

						BoneIndices.SetNum(MaxNumInfluences);
						BoneWeights.SetNum(MaxNumInfluences);
						WeightsSum = 0.f;
						for (int32 Idx = 0; Idx < MaxNumInfluences; ++Idx)
						{
							BoneIndices[Idx] = SortableData[Idx].Get<1>();
							BoneWeights[Idx] = SortableData[Idx].Get<0>();
							WeightsSum += SortableData[Idx].Get<0>();
						}
					}

					// Normalize weights
					const float WeightsSumRecip = WeightsSum > UE_SMALL_NUMBER ? 1.f / WeightsSum : 0.f;
					for (float& Weight : BoneWeights)
					{
						Weight *= WeightsSumRecip;
					}
				}
			}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
		}
		else
		{
			ParallelFor(WeldedSimMesh.MaxVertexID(), [&ClothFacade, &WeldedSimMesh, &NonManifoldMapping](int32 WeldedID)
			{
				const FDynamicMeshVertexSkinWeightsAttribute* OutAttribute = WeldedSimMesh.Attributes()->GetSkinWeightsAttribute(FSkeletalMeshAttributes::DefaultSkinWeightProfileName);

				checkSlow(OutAttribute);
				checkSlow(WeldedSimMesh.IsVertex(WeldedID));
				checkSlow(WeldedID < ClothFacade.GetNumSimVertices3D());
				OutAttribute->GetValue(WeldedID, ClothFacade.GetSimBoneIndices()[WeldedID],
					ClothFacade.GetSimBoneWeights()[WeldedID]);
			}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
		}
	}

	/** Copy the inpaint mask array to Collection, handling split vertices. */
	static void CopyInpaintMapFromDynamicMeshToSimCloth(const FDynamicMesh3& WeldedSimMesh, 
													 const FName WeightMapName,
													 const bool bUseParallel, 
													 const TArray<bool>& MatchedVertices,
													 const TSharedRef<FManagedArrayCollection>& ClothCollection)
	{

		using namespace UE::Geometry;

		UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(ClothCollection);

		TArrayView<float> InpaintWeightMask = ClothFacade.GetWeightMap(WeightMapName);

		const FNonManifoldMappingSupport NonManifoldMapping(WeldedSimMesh);
		TArray<TArray<int32>> SimMeshToDynamicMesh;
		if (NonManifoldMapping.IsNonManifoldVertexInSource())
		{
			// WeldedSimMesh indices don't match cloth collection. 
			SimMeshToDynamicMesh.SetNum(ClothFacade.GetNumSimVertices3D());
			for (int32 DynamicMeshVert = 0; DynamicMeshVert < WeldedSimMesh.VertexCount(); ++DynamicMeshVert)
			{
				SimMeshToDynamicMesh[NonManifoldMapping.GetOriginalNonManifoldVertexID(DynamicMeshVert)].Add(DynamicMeshVert);
			}

			ParallelFor(ClothFacade.GetNumSimVertices3D(), [&ClothFacade, &WeldedSimMesh, &SimMeshToDynamicMesh, &InpaintWeightMask, &MatchedVertices](int32 SimVertexID)
			{
				if (!ensure(SimMeshToDynamicMesh[SimVertexID].Num() > 0))
				{
					return;
				}
				if (SimMeshToDynamicMesh[SimVertexID].Num() == 1)
				{
					const int32 WeldedID = SimMeshToDynamicMesh[SimVertexID][0];
					checkSlow(WeldedSimMesh.IsVertex(WeldedID));
					InpaintWeightMask[SimVertexID] = MatchedVertices[WeldedID] ? 1.0f : 0.0f;
				}
				else
				{
					const int32 WeldedID = SimMeshToDynamicMesh[SimVertexID][0]; // Any welded id can be used here
					InpaintWeightMask[SimVertexID] = MatchedVertices[WeldedID] ? 1.0f : 0.0f;
				}
			}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
		}
		else
		{
			ParallelFor(WeldedSimMesh.MaxVertexID(), [&ClothFacade, &WeldedSimMesh, &NonManifoldMapping, &InpaintWeightMask, &MatchedVertices](int32 WeldedID)
			{
				checkSlow(WeldedSimMesh.IsVertex(WeldedID));
				checkSlow(WeldedID < ClothFacade.GetNumSimVertices3D());
				InpaintWeightMask[WeldedID] = MatchedVertices[WeldedID] ? 1.0f : 0.0f;
			}, bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
		}
	}
	

	/** 
	 * Transfer skin weights from the source to the target dynamic mesh. 
	 * The target mesh is split into connected components and the transfer is run on each component separately.
	 */
	static bool TransferInpaintWeightsToMeshComponents(const FReferenceSkeleton& TargetRefSkeleton,
													   const UE::Geometry::FDynamicMesh3& SourceDynamicMesh,
													   UE::Geometry::FDynamicMesh3& TargetDynamicMesh,
													   const TransferBoneWeightsSettings& TransferSettings)
	{
		using namespace UE::Geometry;


		// Find connected-components
		FMeshConnectedComponents ConnectedComponents(&TargetDynamicMesh);
		ConnectedComponents.FindConnectedTriangles();

		// Pointer to the weight layer containing force inpaint mask (if one exists)
		const FDynamicMeshWeightAttribute* ForceInpaintWeightLayer = nullptr; 
		if (!TransferSettings.InpaintMaskWeightMapName.IsEmpty())
		{
			for (int32 Idx = 0; Idx < TargetDynamicMesh.Attributes()->NumWeightLayers(); ++Idx)
			{	
				const FDynamicMeshWeightAttribute* WeightLayer = TargetDynamicMesh.Attributes()->GetWeightLayer(Idx);
				if (WeightLayer && WeightLayer->GetName() == FName(TransferSettings.InpaintMaskWeightMapName))
				{
					ForceInpaintWeightLayer = WeightLayer;
					break;
				}
			}
		}

		// Iterate over each component and perform per-component skin weight transfer
		const int32 NumComponents = ConnectedComponents.Num();
		const int32 NumTasks = FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), NumComponents), 1);
		constexpr int32 MinComponentByTask = 1;
		const int32 ComnponentsByTask = FMath::Max(FMath::Max(FMath::DivideAndRoundUp(NumComponents, NumTasks), MinComponentByTask), 1);
		const int32 NumBatches = FMath::DivideAndRoundUp(NumComponents, ComnponentsByTask);
		TArray<UE::Tasks::FTask> PendingTasks;
		PendingTasks.Reserve(NumBatches);

		
		TArray<FDynamicMesh3> Submeshes;
		Submeshes.SetNum(NumComponents);
		TArray<TArray<int32>> SubmeshToBaseVs;
		SubmeshToBaseVs.SetNum(NumComponents);
		std::atomic<bool> bTransferParametersInvalid(false);

		// Creating the BVH here in single threaded which is expensive allow to avoid every thread to pay the expensive cost
		FDynamicMeshAABBTree3 SourceBVH(&SourceDynamicMesh);

		for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
		{
			const int32 StartIndex = BatchIndex * ComnponentsByTask;
			int32 EndIndex = (BatchIndex + 1) * ComnponentsByTask;
			EndIndex = BatchIndex == NumBatches - 1 ? FMath::Min(NumComponents, EndIndex) : EndIndex;

			UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [StartIndex, EndIndex, &TargetRefSkeleton, &ConnectedComponents, 
				&TargetDynamicMesh, &ForceInpaintWeightLayer, &TransferSettings, &SourceDynamicMesh, &SourceBVH, &Submeshes, &SubmeshToBaseVs, &bTransferParametersInvalid]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_TransferInpaintWeightsToMeshComponentsTask);
				if (bTransferParametersInvalid.load(std::memory_order_relaxed))
				{
					return;
				}
				// Setup transfer weights operator
				FTransferBoneWeights TransferBoneWeights(&SourceDynamicMesh, FSkeletalMeshAttributes::DefaultSkinWeightProfileName, &SourceBVH);
				TransferBoneWeights.bUseParallel = TransferSettings.bUseParallel;
				TransferBoneWeights.MaxNumInfluences = TransferSettings.MaxNumInfluences;
				TransferBoneWeights.TransferMethod = TransferSettings.TransferMethod;
				TransferBoneWeights.NormalThreshold = FMathd::DegToRad * TransferSettings.NormalThreshold;
				TransferBoneWeights.SearchRadius = TransferSettings.RadiusPercentage * TargetDynamicMesh.GetBounds().DiagonalLength();
				TransferBoneWeights.NumSmoothingIterations = TransferSettings.NumSmoothingIterations;
				TransferBoneWeights.SmoothingStrength = TransferSettings.SmoothingStrength;
				TransferBoneWeights.LayeredMeshSupport = TransferSettings.LayeredMeshSupport; // multilayerd clothing
				if (TransferBoneWeights.Validate() != EOperationValidationResult::Ok)
				{
					bTransferParametersInvalid.store(true, std::memory_order_relaxed);
					return;
				}

				for (int32 ComponentIdx = StartIndex; ComponentIdx < EndIndex; ++ComponentIdx)
				{
					// Mesh containing the geometry of the current component
					FDynamicMesh3& Submesh = Submeshes[ComponentIdx];
					Submesh.EnableAttributes();
					Submesh.Attributes()->EnableBones(TargetRefSkeleton.GetRawBoneNum());
					for (int32 BoneIdx = 0; BoneIdx < TargetRefSkeleton.GetRawBoneNum(); ++BoneIdx)
					{
						Submesh.Attributes()->GetBoneNames()->SetValue(BoneIdx, TargetRefSkeleton.GetRawRefBoneInfo()[BoneIdx].Name);
					}

					// Keep track of the maps between Mesh and the component submesh vertex/triangle indices
					TMap<int32, int32> BaseToSubmeshV;
					TArray<int32>& SubmeshToBaseV = SubmeshToBaseVs[ComponentIdx];
					TArray<int32> SubmeshToBaseT;

					const TArray<int32>& ComponentTris = ConnectedComponents[ComponentIdx].Indices;

					for (const int32 TID : ComponentTris)
					{
						const FIndex3i Triangle = TargetDynamicMesh.GetTriangle(TID);
						FIndex3i NewTriangle;
						for (int32 TriCornerIdx = 0; TriCornerIdx < 3; ++TriCornerIdx)
						{
							const int32* FoundIdx = BaseToSubmeshV.Find(Triangle[TriCornerIdx]);
							if (FoundIdx)
							{
								NewTriangle[TriCornerIdx] = *FoundIdx;
							}
							else
							{
								const FVector3d Position = TargetDynamicMesh.GetVertex(Triangle[TriCornerIdx]);
								const int32 NewVtxID = Submesh.AppendVertex(Position);
								check(NewVtxID == SubmeshToBaseV.Num());
								SubmeshToBaseV.Add(Triangle[TriCornerIdx]);
								BaseToSubmeshV.Add(Triangle[TriCornerIdx], NewVtxID);
								NewTriangle[TriCornerIdx] = NewVtxID;
							}
						}

						const int32 NewTriID = Submesh.AppendTriangle(NewTriangle);
						check(NewTriID == SubmeshToBaseT.Num());
						SubmeshToBaseT.Add(TID);
					}

					// Copy over the force inpaint values
					if (ForceInpaintWeightLayer != nullptr)
					{
						TArray<float> ForceInpaint;
						ForceInpaint.SetNum(Submesh.MaxVertexID());

						for (int32 VID = 0; VID < Submesh.MaxVertexID(); ++VID)
						{
							float Value = 0;
							ForceInpaintWeightLayer->GetValue(SubmeshToBaseV[VID], &Value);
							ForceInpaint[VID] = Value;
						}

						// Set the mask in the transfer operator
						TransferBoneWeights.ForceInpaint = MoveTemp(ForceInpaint);
					}

					// Transfer weights to the current submesh only. If transfer using the inpaint method fails, fall back to  
					// the closest point method which should always succeed. Common reason for the failure is if we didn't find  
					// any matches on the source at all with the current transfer settings.
					if (!TransferBoneWeights.TransferWeightsToMesh(Submesh, FSkeletalMeshAttributes::DefaultSkinWeightProfileName))
					{
						UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Failed to transfer skin weights to some of the vertices of the render mesh using inpaint method, falling back to closest point method.");

						// We can reuse the same operator but change the method type
						TransferBoneWeights.TransferMethod = FTransferBoneWeights::ETransferBoneWeightsMethod::ClosestPointOnSurface;

						// Ignore radius and normal settings, so all points on the source are considered
						const double OldSearchRadius = TransferBoneWeights.SearchRadius;
						const double OldNormalThreshold = TransferBoneWeights.NormalThreshold;
						TransferBoneWeights.SearchRadius = -1;
						TransferBoneWeights.NormalThreshold = -1;

						// This should always succeed
						if (!ensure(TransferBoneWeights.TransferWeightsToMesh(Submesh, FSkeletalMeshAttributes::DefaultSkinWeightProfileName)))
						{
							UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode:  Failed to transfer skin weights to some of the vertices of the render mesh.");
						}

						// Revert back the settings
						TransferBoneWeights.TransferMethod = TransferSettings.TransferMethod;
						TransferBoneWeights.SearchRadius = OldSearchRadius;
						TransferBoneWeights.NormalThreshold = OldNormalThreshold;
					}
				}
			});
			PendingTasks.Add(PendingTask);
		}
		UE::Tasks::Wait(PendingTasks);

		if (bTransferParametersInvalid)
		{
			UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Transfer method parameters are invalid.");
			return false;
		}

		for (int32 ComponentIdx = 0; ComponentIdx < NumComponents; ++ComponentIdx)
		{
			TArray<int32>& SubmeshToBaseV = SubmeshToBaseVs[ComponentIdx];
			// Copy over the data from the submesh to the base mesh
			FDynamicMeshVertexSkinWeightsAttribute* SubMeshSkinWeights = UE::Chaos::ClothAsset::Private::GetOrCreateSkinWeightsAttribute(Submeshes[ComponentIdx]);
			FDynamicMeshVertexSkinWeightsAttribute* BaseMeshSkinWeights = UE::Chaos::ClothAsset::Private::GetOrCreateSkinWeightsAttribute(TargetDynamicMesh);

			for (int32 SubMeshVID = 0; SubMeshVID < SubmeshToBaseV.Num(); ++SubMeshVID)
			{
				const int32 BaseMeshVID = SubmeshToBaseV[SubMeshVID];

				UE::AnimationCore::FBoneWeights Weights;
				SubMeshSkinWeights->GetValue(SubMeshVID, Weights);
				BaseMeshSkinWeights->SetValue(BaseMeshVID, Weights);
			}
		}

		return true;
	}

	/** Transfer skin weights to sim cloth. */
	static bool TransferInpaintWeightsToSim(
		const FReferenceSkeleton& TargetRefSkeleton,
		const UE::Geometry::FDynamicMesh3& SourceDynamicMesh,
		const TSharedRef<FManagedArrayCollection>& ClothCollection,
		const TransferBoneWeightsSettings& TransferSettings,
		UE::Geometry::FDynamicMesh3& WeldedSimMesh)
	{
		using namespace UE::Geometry;

		UE::Chaos::ClothAsset::FCollectionClothFacade ClothFacade(ClothCollection);

		// Convert cloth sim mesh LOD to the welded dynamic sim mesh.
		if (!ClothToDynamicMesh(ClothCollection, TargetRefSkeleton, true, WeldedSimMesh))
		{
			UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Failed to weld the simulation mesh for LOD.");
			return false;
		}

		// Transfer the weights from the body to the welded sim mesh.
		// TODO: run the transfer on components instead
		FTransferBoneWeights TransferBoneWeights(&SourceDynamicMesh, FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
		TransferBoneWeights.bUseParallel = TransferSettings.bUseParallel;
		TransferBoneWeights.MaxNumInfluences = TransferSettings.MaxNumInfluences;
		TransferBoneWeights.TransferMethod = TransferSettings.TransferMethod;
		TransferBoneWeights.NormalThreshold = FMathd::DegToRad * TransferSettings.NormalThreshold;
		TransferBoneWeights.SearchRadius = TransferSettings.RadiusPercentage * WeldedSimMesh.GetBounds().DiagonalLength();
		TransferBoneWeights.NumSmoothingIterations = TransferSettings.NumSmoothingIterations;
		TransferBoneWeights.SmoothingStrength = TransferSettings.SmoothingStrength;
		TransferBoneWeights.LayeredMeshSupport = TransferSettings.LayeredMeshSupport; // multilayerd clothing
		TransferBoneWeights.ForceInpaintWeightMapName = FName(TransferSettings.InpaintMaskWeightMapName);

		if (TransferBoneWeights.Validate() != EOperationValidationResult::Ok)
		{
			UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Transfer method parameters are invalid.");
			return false;
		}
		if (!TransferBoneWeights.TransferWeightsToMesh(WeldedSimMesh, FSkeletalMeshAttributes::DefaultSkinWeightProfileName))
		{
			UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Transferring skin weights failed.");
			return false;
		}

		
		ClothFacade.AddWeightMap(InpaintWeightMaskName);

		// Copy the new bone weight data and inpaint mask from the welded sim mesh back to the sim cloth patterns.
		CopySkinWeightsFromDynamicMeshToSimCloth(WeldedSimMesh, true, ClothCollection, TransferSettings.MaxNumInfluences);
		CopyInpaintMapFromDynamicMeshToSimCloth(WeldedSimMesh, InpaintWeightMaskName, true, TransferBoneWeights.MatchedVertices, ClothCollection);

		return true;
	}

	static bool TransferClosestPointOnSurface(
		const FReferenceSkeleton& TargetRefSkeleton,
		const FDynamicMesh3& SkeletalDynamicMesh,
		const TSharedRef<FManagedArrayCollection>& ClothCollection,
		const TransferBoneWeightsSettings& TransferSettings,
		const TSharedPtr<const FManagedArrayCollection>& SimClothCollection);

	/** Transfer skin weights to render cloth. */
	static bool TransferInpaintWeightsToRender(
		const FReferenceSkeleton& TargetRefSkeleton,
		const UE::Geometry::FDynamicMesh3& SourceDynamicMesh,
		const TSharedRef<FManagedArrayCollection>& ClothCollection,
		const TransferBoneWeightsSettings& TransferSettings)
	{
		using namespace UE::Geometry;

		// Convert cloth render mesh LOD to the dynamic render mesh.
		FDynamicMesh3 RenderDynamicMesh;
		if (!ClothToDynamicMesh(ClothCollection, TargetRefSkeleton, false, RenderDynamicMesh))
		{
			UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Failed to create the render dynamic mesh for LOD.");
			return false;
		}

		const FNonManifoldMappingSupport NonManifoldMapping(RenderDynamicMesh);
		if (NonManifoldMapping.IsNonManifoldVertexInSource())
		{
			UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Render mesh inpaint method failed due to non manifold geometry, falling back to closest point method instead.");
			checkf(!TransferSettings.bTransferToRenderFromSim, TEXT("Logic error, TransferClosestPointOnSurface should have been called instead transferring from Sim mesh."));
			TransferBoneWeightsSettings NewTransferSettings = TransferSettings;
			NewTransferSettings.bTransferToSim = false;
			NewTransferSettings.TransferMethod = FTransferBoneWeights::ETransferBoneWeightsMethod::ClosestPointOnSurface;
			TSharedPtr<const FManagedArrayCollection> EmptySimClothCollection;
			return TransferClosestPointOnSurface(TargetRefSkeleton, SourceDynamicMesh, ClothCollection, NewTransferSettings, EmptySimClothCollection);
		}
		else
		{
			// Transfer weights to render mesh
			if (!TransferInpaintWeightsToMeshComponents(TargetRefSkeleton, SourceDynamicMesh, RenderDynamicMesh, TransferSettings))
			{
				UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Failed to transfer skin weights to render mesh.");
				return false;
			}
		}

		CopySkinWeightsFromDynamicMeshToRenderCloth(RenderDynamicMesh, TransferSettings.bUseParallel, ClothCollection);

		return true;
	}

	/** Transfer skin weights to sim and render cloth. */
	static bool TransferInpaintWeights(
		const FReferenceSkeleton& TargetRefSkeleton,
		const UE::Geometry::FDynamicMesh3& SourceDynamicMesh,
		const TSharedRef<FManagedArrayCollection>& ClothCollection,
		const TransferBoneWeightsSettings& TransferSettings,
		const TSharedPtr<const FManagedArrayCollection>& SimClothCollection)
	{
		using namespace UE::Geometry;

		FDynamicMesh3 WeldedSimMesh;

		if (TransferSettings.bTransferToSim)
		{	
			//
			// Convert cloth sim mesh LOD to the welded dynamic sim mesh and transfer weights.
			//
			if (!TransferInpaintWeightsToSim(TargetRefSkeleton, SourceDynamicMesh, ClothCollection, TransferSettings, WeldedSimMesh))
			{
				return false;
			}
		}

		if (TransferSettings.bTransferToRender)
		{
			//
			// Compute the bone weights for the render mesh by transferring weights from the sim mesh
			//
			if (TransferSettings.bTransferToRenderFromSim)
			{
				ensureMsgf(TransferSettings.bTransferToSim, TEXT("The UI shouldn't allow a sim mesh InpaintWeights transfer without the sim mesh set to transfer, check the EditConditions."));

				// Transfers from the sim mesh most likely mean that a SkeletalMesh InpaintWeights transfer is not working for this render mesh,
				// therefore it is best to assume that a closest point transfer will always provide a better result in this particular case.
				TransferBoneWeightsSettings NewTransferSettings = TransferSettings;
				NewTransferSettings.bTransferToSim = false;
				NewTransferSettings.TransferMethod = FTransferBoneWeights::ETransferBoneWeightsMethod::ClosestPointOnSurface;
				
				return TransferClosestPointOnSurface(TargetRefSkeleton, SourceDynamicMesh, ClothCollection, NewTransferSettings, SimClothCollection);
			}
			else 
			{
				// Transfer skin weights to render cloth from the skeletal asset mesh
				if (!TransferInpaintWeightsToRender(TargetRefSkeleton, SourceDynamicMesh, ClothCollection, TransferSettings))
				{
					return false;
				}
			}
		}

		return true;
	}

	static bool TransferClosestPointOnSurface(
		const FReferenceSkeleton& TargetRefSkeleton,
		const FDynamicMesh3& SkeletalDynamicMesh,
		const TSharedRef<FManagedArrayCollection>& ClothCollection,
		const TransferBoneWeightsSettings& TransferSettings,
		const TSharedPtr<const FManagedArrayCollection>& SimClothCollection)
	{
		using namespace UE::Geometry;

		FCollectionClothFacade ClothFacade(ClothCollection);

		//
		// Compute the bone index mappings. This allows the transfer operator to retarget weights to the correct skeleton.
		//
		TMap<FName, FBoneIndexType> TargetBoneToIndex;
		TargetBoneToIndex.Reserve(TargetRefSkeleton.GetRawBoneNum());
		for (int32 BoneIdx = 0; BoneIdx < TargetRefSkeleton.GetRawBoneNum(); ++BoneIdx)
		{
			TargetBoneToIndex.Add(TargetRefSkeleton.GetRawRefBoneInfo()[BoneIdx].Name, BoneIdx);
		}

		if (TransferSettings.bTransferToSim)
		{
			//
			// Transfer weights to the sim mesh.
			//

			FTransferBoneWeights TransferBoneWeights(&SkeletalDynamicMesh, FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
			TransferBoneWeights.bUseParallel = TransferSettings.bUseParallel;
			TransferBoneWeights.MaxNumInfluences = TransferSettings.MaxNumInfluences;
			TransferBoneWeights.TransferMethod = TransferSettings.TransferMethod;

			if (TransferBoneWeights.Validate() != EOperationValidationResult::Ok)
			{
				UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Transfer method parameters are invalid.");
				return false;
			}
		
			ParallelFor(ClothFacade.GetNumSimVertices3D(), [&TransferBoneWeights, &TargetBoneToIndex, &ClothFacade](int32 VertexID)
			{
				TransferBoneWeights.TransferWeightsToPoint(ClothFacade.GetSimBoneIndices()[VertexID],
					ClothFacade.GetSimBoneWeights()[VertexID],
					ClothFacade.GetSimPosition3D()[VertexID],
					&TargetBoneToIndex);

			}, TransferSettings.bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
		}

		if (TransferSettings.bTransferToRender)
		{
			//
			// Transfer weights to the render mesh.
			//

			const FDynamicMesh3* SourceMeshToTransferFrom = &SkeletalDynamicMesh; // transfer from body
			FDynamicMesh3 WeldedSimMesh;
			if (TransferSettings.bTransferToRenderFromSim)
			{
				// Convert sim cloth to dynamic mesh, use a different sim cloth collection if needed
				constexpr bool bIsSimMesh = true;
				if (!ClothToDynamicMesh(SimClothCollection.IsValid() ? SimClothCollection.ToSharedRef() : ClothCollection, TargetRefSkeleton, bIsSimMesh, WeldedSimMesh))
				{
					return false;
				}

				SourceMeshToTransferFrom = &WeldedSimMesh; // transfer from sim mesh instead
			}

			FTransferBoneWeights TransferBoneWeights(SourceMeshToTransferFrom, FSkeletalMeshAttributes::DefaultSkinWeightProfileName);
			TransferBoneWeights.bUseParallel = TransferSettings.bUseParallel;
			TransferBoneWeights.TransferMethod = TransferSettings.TransferMethod;
			TransferBoneWeights.MaxNumInfluences = TransferSettings.MaxNumInfluences;

			if (TransferBoneWeights.Validate() != EOperationValidationResult::Ok)
			{
				UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Transfer method parameters are invalid.");
				return false;
			}

			ParallelFor(ClothFacade.GetNumRenderVertices(), [&TransferBoneWeights, &TargetBoneToIndex, &ClothFacade](int32 VertexID)
			{
				TransferBoneWeights.TransferWeightsToPoint(ClothFacade.GetRenderBoneIndices()[VertexID],
					ClothFacade.GetRenderBoneWeights()[VertexID],
					ClothFacade.GetRenderPosition()[VertexID],
					&TargetBoneToIndex);

			}, TransferSettings.bUseParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
		}
		return true;
	}
}



FChaosClothAssetTransferSkinWeightsNode::FChaosClothAssetTransferSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&SkeletalMesh);
	RegisterInputConnection(&LodIndex);
	RegisterInputConnection(&SimCollection)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&InpaintMask.WeightMap, GET_MEMBER_NAME_CHECKED(FChaosClothAssetWeightedValueNonAnimatableNoLowHighRange, WeightMap))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetTransferSkinWeightsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FChaosClothAssetTransferSkinWeightsNode_Evaluate);
	using namespace UE::Chaos::ClothAsset;
	using namespace UE::Chaos::ClothAsset::Private;
	using namespace UE::Geometry;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Evaluate inputs
		FManagedArrayCollection InputCollection = GetValue<FManagedArrayCollection>(Context, &Collection);

		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InputCollection));
		
		const TSharedPtr<FManagedArrayCollection> SimClothCollection = IsConnected(&SimCollection) ?
			MakeShared<FManagedArrayCollection>(GetValue(Context, &SimCollection)) :
			TSharedPtr<FManagedArrayCollection>();

		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid())  // Can only act on the collection if it is a valid cloth collection
		{
			TStrongObjectPtr<const USkeletalMesh> InSkeletalMesh;
			const FReferenceSkeleton* TargetRefSkeleton = nullptr;
			FDynamicMesh3 SourceDynamicMesh;

			const bool bNeedsSkeletalMesh =
				TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render ||
				RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SkeletalMesh;

			if (bNeedsSkeletalMesh)
			{
				InSkeletalMesh = TStrongObjectPtr<const USkeletalMesh>(GetValue(Context, &SkeletalMesh));

				if (!InSkeletalMesh)
				{
					Context.Warning(
						FText::Format(
							LOCTEXT("InvalidSkeletalMeshDetails", "No skeletal mesh has been specified and one is required for this type of transfer."),
							LodIndex)
						, this, Out
					);
					SetValue(Context, MoveTemp(*ClothCollection), &Collection);
					return;
				}

				int32 InLodIndex = GetValue(Context, &LodIndex);
				if (!InSkeletalMesh->IsValidLODIndex(InLodIndex))
				{
					const int32 LastLODIndex = InSkeletalMesh->GetLODNum() - 1;
					if (InSkeletalMesh->IsValidLODIndex(LastLODIndex))
					{
						Context.Warning(
							FText::Format(
								LOCTEXT("InvalidLodIndexSubstituteDetails", "LOD index {0} is not a valid LOD for skeletal mesh {1}. Using LOD index {2} instead."),
								InLodIndex,
								FText::FromString(InSkeletalMesh->GetName()),
								LastLODIndex)
						, this, Out);

						InLodIndex = LastLODIndex;
					}
					else
					{
						Context.Warning(
							FText::Format(
								LOCTEXT("InvalidLodIndexDetails", "LOD index {0} is not a valid LOD for skeletal mesh {1}."),
								InLodIndex,
								FText::FromString(InSkeletalMesh->GetName()))
							, this, Out
						);
						SetValue(Context, MoveTemp(*ClothCollection), &Collection);
						return;
					}
				}

				if (!SkeletalMeshToDynamicMesh(InSkeletalMesh.Get(), InLodIndex, SourceDynamicMesh))
				{
					Context.Warning(
						FText::Format(
							LOCTEXT("InvalidLodDetails", "Could not convert LOD index {0} of the skeletal mesh {1} to dyanamic mesh."),
							InLodIndex,
							FText::FromString(InSkeletalMesh->GetName()))
						, this, Out
					);
					SetValue(Context, MoveTemp(*ClothCollection), &Collection);
					return;
				}

				MeshTransforms::ApplyTransform(SourceDynamicMesh, Transform, true);
				TargetRefSkeleton = &InSkeletalMesh->GetRefSkeleton();

				ClothFacade.SetSkeletalMeshSoftObjectPathName(InSkeletalMesh->GetPathName());
			}
			else
			{
				// Reuse the input sim mesh skeleton
				auto GetCollectionRefSkeleton = [&InSkeletalMesh](const FCollectionClothConstFacade& ClothFacade) -> const FReferenceSkeleton*
					{
						const FSoftObjectPath& SkeletalMeshPathName = ClothFacade.GetSkeletalMeshSoftObjectPathName();
						InSkeletalMesh = TStrongObjectPtr<const USkeletalMesh>(Cast<USkeletalMesh>(SkeletalMeshPathName.TryLoad()));
						return InSkeletalMesh ? &InSkeletalMesh->GetRefSkeleton() : nullptr;
					};

				if (SimClothCollection.IsValid())
				{
					FCollectionClothConstFacade SimClothFacade(SimClothCollection.ToSharedRef());
					if (SimClothFacade.IsValid())
					{
						TargetRefSkeleton = GetCollectionRefSkeleton(SimClothFacade);
					}
					if (!TargetRefSkeleton)
					{
						Context.Warning(LOCTEXT("InvalidSimRefSkeletonDetails", "Couldn't find a valid reference skeleton from the input sim collection."), this, Out);
						SetValue(Context, MoveTemp(*ClothCollection), &Collection);
						return;
					}
				}
				else
				{
					TargetRefSkeleton = GetCollectionRefSkeleton(ClothFacade);

					if (!TargetRefSkeleton)
					{
						Context.Warning(LOCTEXT("InvalidRefSkeletonDetails", "Couldn't find a valid reference skeleton from the input collection."), this, Out);
						SetValue(Context, MoveTemp(*ClothCollection), &Collection);
						return;
					}
				}
			}

			// Clean up orphaned vertices
			UE::Chaos::ClothAsset::FClothGeometryTools::CleanupAndCompactMesh(ClothCollection);
			if (SimClothCollection.IsValid())
			{
				UE::Chaos::ClothAsset::FClothGeometryTools::CleanupAndCompactMesh(SimClothCollection.ToSharedRef());
			}

			//
			// Setup the bone weight transfer settings.
			//
			TransferBoneWeightsSettings TransferSettings;

			TransferSettings.bTransferToSim = TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render;
			TransferSettings.bTransferToRender = TargetMeshType != EChaosClothAssetTransferTargetMeshType::Simulation;
			TransferSettings.bTransferToRenderFromSim = RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SimulationMesh;
			
			TransferSettings.bUseParallel = true;
			TransferSettings.MaxNumInfluences = static_cast<int32>(FChaosClothAssetTransferSkinWeightsNode::MaxNumInfluences);
			TransferSettings.TransferMethod = static_cast<FTransferBoneWeights::ETransferBoneWeightsMethod>(TransferMethod);

			TransferSettings.NormalThreshold = NormalThreshold;
			TransferSettings.RadiusPercentage = RadiusPercentage;
			TransferSettings.LayeredMeshSupport = LayeredMeshSupport;
			TransferSettings.NumSmoothingIterations = NumSmoothingIterations;
			TransferSettings.SmoothingStrength = SmoothingStrength;
			TransferSettings.InpaintMaskWeightMapName = GetValue<FString>(Context, &InpaintMask.WeightMap);
			
			bool bTransferResult = false;
			switch (TransferMethod)
			{
			case EChaosClothAssetTransferSkinWeightsMethod::InpaintWeights:
				bTransferResult = TransferInpaintWeights(*TargetRefSkeleton, SourceDynamicMesh, ClothCollection, TransferSettings, SimClothCollection);
				break;
			
			case EChaosClothAssetTransferSkinWeightsMethod::ClosestPointOnSurface:
				bTransferResult = TransferClosestPointOnSurface(*TargetRefSkeleton, SourceDynamicMesh, ClothCollection, TransferSettings, SimClothCollection);
				break;

			default:
				checkNoEntry();
				break;
			}

			if (!bTransferResult)
			{
				Context.Warning(LOCTEXT("TransferWeightsDetails", "Failed to transfer skinning weights from the source."), this, Out);
				SetValue(Context, MoveTemp(*ClothCollection), &Collection);
				return;
			}

			// Optional check to make sure all vertices adhere to the bone influence limit
			constexpr bool bCheckMaxInfluenceComplience = false; 
			if (bCheckMaxInfluenceComplience)
			{
				for (int32 VID = 0; VID < ClothFacade.GetNumSimVertices3D(); ++VID)
				{
					const int32 NumInfluences = ClothFacade.GetSimBoneIndices()[VID].Num();
					if (!ensure(NumInfluences <= static_cast<int32>(FChaosClothAssetTransferSkinWeightsNode::MaxNumInfluences)))
					{
						UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Maximum number of influences is exceeded for sim vertex %i.", VID);
					}
				}

				for (int32 VID = 0; VID < ClothFacade.GetNumRenderVertices(); ++VID)
				{
					const int32 NumInfluences = ClothFacade.GetRenderBoneIndices()[VID].Num();
					if (!ensure(NumInfluences <= static_cast<int32>(FChaosClothAssetTransferSkinWeightsNode::MaxNumInfluences)))
					{
						UE_LOGF(LogChaosClothAssetDataflowNodes, Warning, "TransferSkinWeightsNode: Maximum number of influences is exceeded for render vertex %i.", VID);
					}
				}
			}

		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
