// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/TransformPositionsNode.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformPositionsNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetTransformPositionsNode"

FChaosClothAssetTransformPositionsNode::FChaosClothAssetTransformPositionsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetTransformPositionsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		// Always check for a valid cloth collection/facade to avoid processing non cloth collections
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid())
		{
			if (bTransform2DSimPositions)
			{
				auto Transform2D = [this](TArrayView<FVector2f>&& SimPositions2D)
				{
					for (FVector2f& SimPosition : SimPositions2D)
					{
						// Scale
						SimPosition = SimPosition * Sim2DScale;
						// Rotate
						float SinAngle, CosAngle;
						FMath::SinCos(&SinAngle, &CosAngle, FMath::DegreesToRadians(Sim2DRotation));
						SimPosition = FVector2f(
							SimPosition.X * CosAngle - SimPosition.Y * SinAngle,
							SimPosition.Y * CosAngle + SimPosition.X * SinAngle);  // Clockwise rotation
						// Translate
						SimPosition += FVector2f(Sim2DTranslation.X, Sim2DTranslation.Y);
					}
				};

				if (Sim2DPattern == INDEX_NONE)
				{
					Transform2D(ClothFacade.GetSimPosition2D());
				}
				else
				{
					const int32 NumPatterns = ClothFacade.GetNumSimPatterns();
					if (Sim2DPattern >= 0 && Sim2DPattern < NumPatterns)
					{
						FCollectionClothSimPatternFacade ClothPatternFacade = ClothFacade.GetSimPattern(Sim2DPattern);
						Transform2D(ClothPatternFacade.GetSimPosition2D());
					}
				}
			}
			if (bTransform3DSimPositions)
			{
				const FTransform3f Transform(FRotator3f(Sim3DRotation.X, Sim3DRotation.Y, Sim3DRotation.Z), Sim3DTranslation, Sim3DScale);

				TArrayView<FVector3f> SimPositions3D = ClothFacade.GetSimPosition3D();
				for (FVector3f& SimPosition : SimPositions3D)
				{
					SimPosition = Transform.TransformPosition(SimPosition);
				}
			}
			if (bTransformRenderPositions)
			{
				const FTransform3f Transform(FRotator3f(RenderRotation.X, RenderRotation.Y, RenderRotation.Z), RenderTranslation, RenderScale);
				auto Transform3D = [this, &Transform](TArrayView<FVector3f>&& RenderPositions)
				{
					for (FVector3f& RenderPosition : RenderPositions)
					{
						RenderPosition = Transform.TransformPosition(RenderPosition);
					}
				};

				if (RenderPattern == INDEX_NONE)
				{
					Transform3D(ClothFacade.GetRenderPosition());
				}
				else
				{
					const int32 NumPatterns = ClothFacade.GetNumRenderPatterns();
					if (RenderPattern >= 0 && RenderPattern < NumPatterns)
					{
						FCollectionClothRenderPatternFacade RenderPatternFacade = ClothFacade.GetRenderPattern(RenderPattern);
						Transform3D(RenderPatternFacade.GetRenderPosition());
					}
				}
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
