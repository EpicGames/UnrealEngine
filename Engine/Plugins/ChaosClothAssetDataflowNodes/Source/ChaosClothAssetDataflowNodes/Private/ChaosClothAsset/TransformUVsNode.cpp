// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/TransformUVsNode.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TransformUVsNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetTransformUVsNode"

FChaosClothAssetTransformUVsNode::FChaosClothAssetTransformUVsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
}

void FChaosClothAssetTransformUVsNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		using namespace UE::Chaos::ClothAsset;

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));

		// Always check for a valid cloth collection/facade to avoid processing non cloth collections
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid())
		{
			auto Transform = [this](TArrayView<TArray<FVector2f>>&& RenderUVsView)
				{
					for (TArray<FVector2f>& RenderUVs : RenderUVsView)
					{
						for (int32 Index = 0; Index < RenderUVs.Num(); ++Index)
						{
							if (UVChannel == INDEX_NONE || UVChannel == Index)
							{
								// Scale
								RenderUVs[Index] = RenderUVs[Index] * Scale;
								// Rotate
								float SinAngle, CosAngle;
								FMath::SinCos(&SinAngle, &CosAngle, FMath::DegreesToRadians(Rotation));
								RenderUVs[Index] = FVector2f(
									RenderUVs[Index].X * CosAngle - RenderUVs[Index].Y * SinAngle,
									RenderUVs[Index].Y * CosAngle + RenderUVs[Index].X * SinAngle);  // Clockwise rotation
								// Translate
								RenderUVs[Index] += FVector2f(Translation.X, -Translation.Y);  // UVs are imported as 1-Y
							}
						}
					}
				};

			if (Pattern == INDEX_NONE)
			{
				Transform(ClothFacade.GetRenderUVs());
			}
			else
			{
				const int32 NumPatterns = ClothFacade.GetNumRenderPatterns();
				if (Pattern >= 0 && Pattern < NumPatterns)
				{
					FCollectionClothRenderPatternFacade ClothPatternFacade = ClothFacade.GetRenderPattern(Pattern);
					Transform(ClothPatternFacade.GetRenderUVs());
				}
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

#undef LOCTEXT_NAMESPACE
