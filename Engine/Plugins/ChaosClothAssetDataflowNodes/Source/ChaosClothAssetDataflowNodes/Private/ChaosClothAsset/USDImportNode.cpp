// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/USDImportNode.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "StaticMeshAttributes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDImportNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetUSDImportNode"

FChaosClothAssetUSDImportNode::FChaosClothAssetUSDImportNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowTerminalNode(InParam, InGuid)
{
	using namespace UE::Chaos::ClothAsset;

	// Initialize to a valid collection cache
	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(CollectionCache));
	FCollectionClothFacade ClothFacade(ClothCollection);
	ClothFacade.DefineSchema();
	CollectionCache = MoveTemp(*ClothCollection);

	// Register connections
	RegisterOutputConnection(&Collection);
}

void FChaosClothAssetUSDImportNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>();

		// Import from cache
		FText ErrorText;
		if (!ImportFromCache(ClothCollection, ErrorText))
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Context.Error(
				FText::Format(LOCTEXT("FailedToExportUsdCacheDetails", "Error while importing USD cloth from cache '{0}':\n{1}"), FText::FromString(UsdFile.FilePath), ErrorText)
				, this, Out
			);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
}

void FChaosClothAssetUSDImportNode::Serialize(FArchive& Ar)
{
	using namespace UE::Chaos::ClothAsset;

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	::Chaos::FChaosArchive ChaosArchive(Ar);
	CollectionCache.Serialize(ChaosArchive);

	Ar << FileHash;

	if (Ar.IsLoading())
	{
		// Make sure to always have a valid cloth collection on reload, some new attributes could be missing from the cached collection
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(CollectionCache));
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (!ClothFacade.IsValid())
		{
			ClothFacade.DefineSchema();
		}

		// Also apply any required fixup (e.g. soft object path names)
		ClothFacade.PostSerialize(Ar);

		CollectionCache = MoveTemp(*ClothCollection);

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ChaosClothAssetUSDImportNodeAddAssetDependencies)
		{
			UpdateImportedAssets();
		}

		// Update the deprecated import path display
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ImportedFilePath = UsdFile.FilePath;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

void FChaosClothAssetUSDImportNode::UpdateImportedAssets()
{
	ImportedAssets.Reset();

	if (!PackagePath.IsEmpty())
	{
		TArray<FAssetData> AssetData;

		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		const UClass* const Class = UStaticMesh::StaticClass();
		constexpr bool bRecursive = true;
		constexpr bool bIncludeOnlyOnDiskAssets = false;
		AssetRegistryModule.Get().GetAssetsByPath(FName(*PackagePath), AssetData, bRecursive, bIncludeOnlyOnDiskAssets);

		ImportedAssets.Reserve(AssetData.Num());

		for (const FAssetData& AssetDatum : AssetData)
		{
			if (AssetDatum.IsUAsset() && AssetDatum.IsTopLevelAsset())  // IsUAsset returns false for redirects
			{
				ImportedAssets.Emplace(AssetDatum.GetAsset());  // GetAsset does not handle redirects

				UE_LOGF(LogChaosClothAssetDataflowNodes,
					Verbose,
					"Imported USD Object %ls of type %ls, path: %ls",
					*AssetDatum.AssetName.ToString(),
					*AssetDatum.AssetClassPath.ToString(),
					*AssetDatum.GetFullName());
			}
		}
	}
}

bool FChaosClothAssetUSDImportNode::ImportFromCache(const TSharedRef<FManagedArrayCollection>& OutClothCollection, FText& OutErrorText) const
{
	using namespace UE::Chaos::ClothAsset;

	// Initialize from collection cache
	// TODO: Until we have a schema so that we can use the asset cache and remove the collection cache
	*OutClothCollection = CollectionCache;

	for (const UObject* const Asset : ImportedAssets)
	{
		if (const UStaticMesh* const StaticMesh = Cast<UStaticMesh>(Asset))
		{
			if (StaticMesh->GetNumSourceModels() > 0)  // Only deals with LOD 0 for now
			{
				constexpr int32 LODIndex = 0;
				const FMeshDescription* const MeshDescription = StaticMesh->GetMeshDescription(LODIndex);
				const FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(LODIndex).BuildSettings;
				const TArray<FStaticMaterial>& StaticMaterials = StaticMesh->GetStaticMaterials();

				FSkeletalMeshLODModel SkeletalMeshModel;
				if (FClothDataflowTools::BuildSkeletalMeshModelFromMeshDescription(MeshDescription, BuildSettings, SkeletalMeshModel))
				{
					FStaticMeshConstAttributes MeshAttributes(*MeshDescription);
					TPolygonGroupAttributesConstRef<FName> MaterialSlotNames = MeshAttributes.GetPolygonGroupMaterialSlotNames();
					for (int32 SectionIndex = 0; SectionIndex < SkeletalMeshModel.Sections.Num(); ++SectionIndex)
					{
						// Section MaterialIndex refers to the polygon group index. Look up which material this corresponds with.
						const FName& MaterialSlotName = MaterialSlotNames[SkeletalMeshModel.Sections[SectionIndex].MaterialIndex];
						const int32 MaterialIndex = StaticMesh->GetMaterialIndexFromImportedMaterialSlotName(MaterialSlotName);
						const FString RenderMaterialPathName = StaticMaterials.IsValidIndex(MaterialIndex) && StaticMaterials[MaterialIndex].MaterialInterface ? 
							StaticMaterials[MaterialIndex].MaterialInterface->GetPathName() :
							FString();
						FClothDataflowTools::AddRenderPatternFromSkeletalMeshSection(OutClothCollection, SkeletalMeshModel, SectionIndex, RenderMaterialPathName);
					}
				}
			}
		}
	}

	// Bind to root bone
	constexpr bool bBindSimMesh = true;  // Some old imported collection caches did not have the sim mesh bound
	constexpr bool bBindRenderMesh = true;
	FClothGeometryTools::BindMeshToRootBone(OutClothCollection, bBindSimMesh, bBindRenderMesh);

	return true;
}

#undef LOCTEXT_NAMESPACE
