// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/USDImportNode_v2.h"
#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "ChaosClothAsset/RenderMeshImport.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Engine/StaticMesh.h"
#include "Materials/Material.h"
#include "Misc/ScopedSlowTask.h"
#include "StaticMeshAttributes.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/Package.h"
#include "AssetViewUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(USDImportNode_v2)

#define LOCTEXT_NAMESPACE "ChaosClothAssetUSDImportNode_v2"

namespace UE::Chaos::ClothAsset::Private
{
	// User attribute names
	static const FName OriginalIndicesName(TEXT("OriginalIndices"));

	// Cloth USD API names
	static const FName ClothRootAPI(TEXT("ClothRootAPI"));
	static const FName RenderPatternAPI(TEXT("RenderPatternAPI"));
	static const FName SimMeshDataAPI(TEXT("SimMeshDataAPI"));
	static const FName SimPatternAPI(TEXT("SimPatternAPI"));
	static const FName SewingAPI(TEXT("SewingAPI"));
	static const FName SpringAPI(TEXT("SpringAPI"));
	static const FName CloSolverPropertiesAPI(TEXT("CloSolverPropertiesAPI"));

	// Return the specified UObject's dependencies for top level UAssets that are not in the engine folders
	static TArray<UObject*> GetAssetDependencies(const UObject* Asset)
	{
		constexpr bool bInRequireDirectOuter = true;
		constexpr bool bShouldIgnoreArchetype = true;
		constexpr bool bInSerializeRecursively = false;  // Ignored if LimitOuter is nullptr
		constexpr bool bShouldIgnoreTransient = true;
		constexpr UObject* LimitOuter = nullptr;
		TArray<UObject*> References;
		FReferenceFinder ReferenceFinder(References, LimitOuter, bInRequireDirectOuter, bShouldIgnoreArchetype, bInSerializeRecursively, bShouldIgnoreTransient);
		ReferenceFinder.FindReferences(const_cast<UObject*>(Asset));

		TArray<UObject*> Dependencies;
		Dependencies.Reserve(References.Num());
		for (UObject* const Reference : References)
		{
			constexpr bool bEnginePluginIsAlsoEngine = true;  // Only includes non Engine or non Engine plugins assets (e.g. no USD materials)
			if (FAssetData::IsUAsset(Reference) &&
				FAssetData::IsTopLevelAsset(Reference) &&
				!AssetViewUtils::IsEngineFolder(Reference->GetPackage()->GetName(), bEnginePluginIsAlsoEngine))
			{
				Dependencies.Emplace(Reference);
			}
		}
		return Dependencies;
	}

	static TArray<TObjectPtr<UObject>> GetImportedAssetDependencies(const UObject* StaticMesh)
	{
		using namespace UE::Chaos::ClothAsset::Private;

		TSet<TObjectPtr<UObject>> ImportedAssets;
		if (StaticMesh)
		{
			TQueue<const UObject*> AssetsToVisit;
			AssetsToVisit.Enqueue(StaticMesh);

			const UObject* VisitedAsset;
			while (AssetsToVisit.Dequeue(VisitedAsset))
			{
				const FName VisitedAssetPackageName(VisitedAsset->GetPackage()->GetName());
				const TArray<UObject*> AssetDependencies = GetAssetDependencies(VisitedAsset);

				UE_CLOGF(AssetDependencies.Num(), LogChaosClothAssetDataflowNodes, Verbose, "Dependencies for Object %ls - %ls:", *VisitedAsset->GetName(), *VisitedAssetPackageName.ToString());
				for (UObject* const AssetDependency : AssetDependencies)
				{
					if (!ImportedAssets.Contains(AssetDependency))
					{
						// Add the dependency
						UE_LOGF(LogChaosClothAssetDataflowNodes, Verbose, "Found %ls", *AssetDependency->GetPackage()->GetName());
						ImportedAssets.Emplace(AssetDependency);

						// Visit this asset too
						AssetsToVisit.Enqueue(AssetDependency);
					}
				}
			}
		}
		return ImportedAssets.Array();
	}
}  // End namespace Private

FChaosClothAssetUsdClothData::FChaosClothAssetUsdClothData() = default;
FChaosClothAssetUsdClothData::~FChaosClothAssetUsdClothData() = default;

void FChaosClothAssetUsdClothData::Reset()
{
	SimPatterns.Reset();
	Sewings.Reset();
	RenderPatterns.Reset();
	RenderToSimPatterns.Reset();
	SimPatternFabricIndices.Reset();
	SimulationCollection.Reset();
}

bool FChaosClothAssetUsdClothData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);

	Ar << SimPatterns;
	Ar << Sewings;
	Ar << RenderPatterns;
	Ar << RenderToSimPatterns;

	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::AddSimulationPropertySupportToClothUSDImportNodeV2)
	{
		Ar << SimPatternFabricIndices;
		SimulationCollection.Serialize(Ar);
	}
	
	return true;
}

FChaosClothAssetUSDImportNode_v2::FChaosClothAssetUSDImportNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
	, ReloadSimStaticMesh(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(Collection));
				FText ErrorText;
				if (!ImportSimStaticMesh(ClothCollection, ErrorText))
				{
					Context.Error(
						FText::Format(LOCTEXT("FailedToImportSimMeshDetails", "Error while re-importing the simulation mesh from static mesh '{0}':\n{1}"), FText::FromString(ImportedSimStaticMesh->GetName()), ErrorText)
						, this
					);
				}
				Collection = MoveTemp(*ClothCollection);
			}))
	, ReloadRenderStaticMesh(
		FDataflowFunctionProperty::FDelegate::CreateLambda([this](UE::Dataflow::FContext& Context)
			{
				const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(Collection));
				FText ErrorText;
				if (!ImportRenderStaticMesh(ClothCollection, ErrorText))
				{
					Context.Error(
						FText::Format(LOCTEXT("FailedToImportRenderMeshDetails", "Error while re-importing the render mesh from static mesh '{0}':\n{1}"), FText::FromString(ImportedRenderStaticMesh->GetName()), ErrorText)
						, this
					);
				}
				Collection = MoveTemp(*ClothCollection);
			}))
{
	using namespace UE::Chaos::ClothAsset;

	// Initialize to a valid collection
	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(Collection));
	FCollectionClothFacade(ClothCollection).DefineSchema();
	Collection = MoveTemp(*ClothCollection);

	// Register connections
	RegisterOutputConnection(&Collection);
}

void FChaosClothAssetUSDImportNode_v2::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		SetValue(Context, Collection, &Collection);
	}
}

void FChaosClothAssetUSDImportNode_v2::Serialize(FArchive& Ar)
{
	using namespace UE::Chaos::ClothAsset;

	if (Ar.IsLoading() && !Ar.IsTransacting())
	{
		// Make sure to always have a valid cloth collection on reload, some new attributes could be missing from the cached collection
		// Must be executed before ImportRenderStaticMesh below, and after serializing the collection above, and even if the serialized version hasn't changed
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(Collection));
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (!ClothFacade.IsValid())
		{
			ClothFacade.DefineSchema();
		}

		// Also apply any required fixup (e.g. soft object path names)
		ClothFacade.PostSerialize(Ar);
		
		Collection = MoveTemp(*ClothCollection);

		// Regenerate correct dependencies if needed
		if (!ImportedAssets_DEPRECATED.IsEmpty())
		{
			ImportedAssets_DEPRECATED.Empty();
			ImportedSimAssets = Private::GetImportedAssetDependencies(ImportedSimStaticMesh);
			ImportedRenderAssets = Private::GetImportedAssetDependencies(ImportedRenderStaticMesh);
		}
	}
}

bool FChaosClothAssetUSDImportNode_v2::ImportSimStaticMesh(const TSharedRef<FManagedArrayCollection> ClothCollection, FText& OutErrorText)
{
	using namespace UE::Chaos::ClothAsset;
	using namespace ::Chaos::Softs;

	FCollectionClothFacade ClothFacade(ClothCollection);
	check(ClothFacade.IsValid());  // The Cloth Collection schema must be valid at this point

	// Define the selection schema if needed
	FCollectionClothSelectionFacade ClothSelectionFacade(ClothCollection);
	if (!ClothSelectionFacade.IsValid())
	{
		ClothSelectionFacade.DefineSchema();
	}

	// Empty the current sim mesh, fabrics, and any previously created selection set
	FClothGeometryTools::DeleteSimMesh(ClothCollection);
	FClothGeometryTools::DeleteSelections(ClothCollection, ClothCollectionGroup::SimFaces);

	// TODO: Move to FClothGeometryTools::DeleteFabrics(ClothCollection);
	ClothFacade.SetNumFabrics(0);

	ON_SCOPE_EXIT
		{
			// Bind to root bone on exit
			constexpr bool bBindSimMesh = true;
			constexpr bool bBindRenderMesh = false;
			FClothGeometryTools::BindMeshToRootBone(ClothCollection, bBindSimMesh, bBindRenderMesh);

			// Make sure to clean the dependencies whatever the import status is
			ImportedSimAssets = Private::GetImportedAssetDependencies(ImportedSimStaticMesh);
		};

	if (!ImportedSimStaticMesh)
	{
		return true;  // Nothing to import
	}

	// Init the static mesh attributes
	constexpr int32 LODIndex = 0;
	const FMeshDescription* const MeshDescription = ImportedSimStaticMesh->GetMeshDescription(LODIndex);
	check(MeshDescription);
	const FStaticMeshConstAttributes StaticMeshAttributes(*MeshDescription);

	if (!StaticMeshAttributes.GetVertexInstanceUVs().GetNumChannels())
	{
		OutErrorText = LOCTEXT("CantFindUVs", "Missing UV layer to initialize sim mesh data.");
		return false;
	}

	TArray<FVector2f> RestPositions2D;
	TArray<FVector3f> DrapedPositions3D;
	TArray<FIntVector3> TriangleToVertexIndex;

	// Retrieve 3D drapped positions
	DrapedPositions3D = StaticMeshAttributes.GetVertexPositions().GetRawArray();

	// Retrieve triangle indices and 2D rest positions
	RestPositions2D.SetNumZeroed(DrapedPositions3D.Num());

	const TConstArrayView<FVertexID> VertexInstanceVertexIndices = StaticMeshAttributes.GetVertexInstanceVertexIndices().GetRawArray();
	const TConstArrayView<FVertexInstanceID> TriangleVertexInstanceIndices = StaticMeshAttributes.GetTriangleVertexInstanceIndices().GetRawArray();
	const TConstArrayView<FVector2f> VertexInstanceUVs = StaticMeshAttributes.GetVertexInstanceUVs().GetRawArray();

	check(TriangleVertexInstanceIndices.Num() % 3 == 0);
	TriangleToVertexIndex.SetNumUninitialized(TriangleVertexInstanceIndices.Num() / 3);

	auto SetRestPositions2D = [&RestPositions2D, &VertexInstanceUVs](FVertexID VertexID, FVertexInstanceID VertexInstanceID) -> bool
		{
			if (RestPositions2D[VertexID] == FVector2f::Zero())
			{
				RestPositions2D[VertexID] = VertexInstanceUVs[VertexInstanceID];
			}
			else if (!RestPositions2D[VertexID].Equals(VertexInstanceUVs[VertexInstanceID]))
			{
				return false;
			}
			return true;
		};

	for (int32 TriangleIndex = 0; TriangleIndex < TriangleToVertexIndex.Num(); ++TriangleIndex)
	{
		const FVertexInstanceID VertexInstanceID0 = TriangleVertexInstanceIndices[TriangleIndex * 3];
		const FVertexInstanceID VertexInstanceID1 = TriangleVertexInstanceIndices[TriangleIndex * 3 + 1];
		const FVertexInstanceID VertexInstanceID2 = TriangleVertexInstanceIndices[TriangleIndex * 3 + 2];

		const FVertexID VertexID0 = VertexInstanceVertexIndices[VertexInstanceID0];
		const FVertexID VertexID1 = VertexInstanceVertexIndices[VertexInstanceID1];
		const FVertexID VertexID2 = VertexInstanceVertexIndices[VertexInstanceID2];

		TriangleToVertexIndex[TriangleIndex] = FIntVector3(VertexID0, VertexID1, VertexID2);

		if (!SetRestPositions2D(VertexID0, VertexInstanceID0) ||
			!SetRestPositions2D(VertexID1, VertexInstanceID1) ||
			!SetRestPositions2D(VertexID2, VertexInstanceID2))
		{
			OutErrorText = LOCTEXT("UsdSimMeshWelded", "The sim mesh has already been welded. This importer needs an unwelded sim mesh.");
			// TODO: unweld vertices, generate seams(?), and reindex all constraints
			return false;
		}
	}

	// Rescale the 2D mesh with the UV scale, and flip the UV's Y coordinates
	for (FVector2f& Pos : RestPositions2D)
	{
		Pos.Y = 1.f - Pos.Y;
		Pos *= ImportedUVScale;
	}

	// Save pattern to the collection cache
	check(RestPositions2D.Num() == DrapedPositions3D.Num());  // Should have already exited with the UsdSimMeshWelded error in this case
	if (TriangleToVertexIndex.Num() && RestPositions2D.Num())
	{
		// Cleanup sim mesh
		FClothDataflowTools::FSimMeshCleanup SimMeshCleanup(TriangleToVertexIndex, RestPositions2D, DrapedPositions3D);

		bool bHasRepairedTriangles = SimMeshCleanup.RemoveDegenerateTriangles();
		bHasRepairedTriangles = SimMeshCleanup.RemoveDuplicateTriangles() || bHasRepairedTriangles;

		const TArray<int32> OriginalToNewTriangles = FClothDataflowTools::GetOriginalToNewIndices<TSet<int32>>(SimMeshCleanup.OriginalTriangles, TriangleToVertexIndex.Num());

		// Add support for original indices
		ClothFacade.AddUserDefinedAttribute<TArray<int32>>(Private::OriginalIndicesName, ClothCollectionGroup::SimFaces);
		ClothFacade.AddUserDefinedAttribute<TArray<int32>>(Private::OriginalIndicesName, ClothCollectionGroup::SimVertices2D);

		// Add the patterns from the clean mesh
		for (const TPair<FName, TSet<int32>>& PatternNameFaces : UsdClothData.SimPatterns)
		{
			// Filter the pattern selection set using the remaining triangles from the cleaned triangle list
			TSet<int32> PatternSet;
			PatternSet.Reserve(PatternNameFaces.Value.Num());
			for (const int32 Face : PatternNameFaces.Value)
			{
				if (OriginalToNewTriangles.IsValidIndex(Face) && OriginalToNewTriangles[Face] != INDEX_NONE)
				{
					PatternSet.Emplace(OriginalToNewTriangles[Face]);
				}
			}

			// Add the new pattern
			if (PatternSet.Num())
			{
				TArray<FIntVector3> PatternTriangleToVertexIndex;
				TArray<TArray<int32>> PatternOriginalTriangles;
				PatternTriangleToVertexIndex.Reserve(PatternSet.Num());
				PatternOriginalTriangles.Reserve(PatternSet.Num());
				{
					for (const int32 Index : PatternSet)
					{
						PatternTriangleToVertexIndex.Emplace(SimMeshCleanup.TriangleToVertexIndex[Index]);
						PatternOriginalTriangles.Emplace(SimMeshCleanup.OriginalTriangles[Index].Array());
					}
				}

				TArray<FVector2f> PatternRestPositions2D;
				TArray<FVector3f> PatternDrapedPositions3D;
				TArray<TArray<int32>> PatternOriginalVertices;
				TArray<int32> PatternVertexReindex;
				const int32 MaxNumVertices = SimMeshCleanup.RestPositions2D.Num();
				PatternRestPositions2D.Reserve(MaxNumVertices);
				PatternDrapedPositions3D.Reserve(MaxNumVertices);
				PatternOriginalVertices.Reserve(MaxNumVertices);
				PatternVertexReindex.Init(INDEX_NONE, MaxNumVertices);

				int32 NewIndex = -1;
				for (FIntVector3& Triangle : PatternTriangleToVertexIndex)
				{
					for (int32 Vertex = 0; Vertex < 3; ++Vertex)
					{
						// Add the new vertex
						int32& Index = Triangle[Vertex];
						if (PatternVertexReindex[Index] == INDEX_NONE)
						{
							PatternVertexReindex[Index] = ++NewIndex;
							PatternRestPositions2D.Emplace(SimMeshCleanup.RestPositions2D[Index]);
							PatternDrapedPositions3D.Emplace(SimMeshCleanup.DrapedPositions3D[Index]);
							PatternOriginalVertices.Emplace(SimMeshCleanup.OriginalVertices[Index].Array());
						}
						// Reindex the triangle vertex with the new index
						Index = PatternVertexReindex[Index];
					}
				}

				// Find this pattern's fabric if any
				const int32 FabricIndex = UsdClothData.SimPatternFabricIndices.Contains(PatternNameFaces.Key) ?
					UsdClothData.SimPatternFabricIndices[PatternNameFaces.Key] :
					INDEX_NONE;

				// Add this pattern to the cloth collection
				const int32 SimPatternIndex = ClothFacade.AddSimPattern();
				FCollectionClothSimPatternFacade SimPattern = ClothFacade.GetSimPattern(SimPatternIndex);
				SimPattern.Initialize(PatternRestPositions2D, PatternDrapedPositions3D, PatternTriangleToVertexIndex, FabricIndex);

				// Keep track of the original triangle indices
				const TArrayView<TArray<int32>> OriginalTriangles =
					ClothFacade.GetUserDefinedAttribute<TArray<int32>>(Private::OriginalIndicesName, ClothCollectionGroup::SimFaces);  // Don't move outside the loop, the array might get re-allocated
				const int32 SimFacesOffset = SimPattern.GetSimFacesOffset();
				for (int32 Index = 0; Index < PatternOriginalTriangles.Num(); ++Index)
				{
					OriginalTriangles[SimFacesOffset + Index] = PatternOriginalTriangles[Index];
				}

				// Keep track of the original vertex indices
				const TArrayView<TArray<int32>> OriginalVertices =
					ClothFacade.GetUserDefinedAttribute<TArray<int32>>(Private::OriginalIndicesName, ClothCollectionGroup::SimVertices2D);  // Don't move outside the loop, the array might get re-allocated
				const int32 SimVertices2DOffset = SimPattern.GetSimVertices2DOffset();
				for (int32 Index = 0; Index < PatternOriginalVertices.Num(); ++Index)
				{
					OriginalVertices[SimVertices2DOffset + Index] = PatternOriginalVertices[Index];
				}

				// Add the pattern triangle list as a selection set
				TSet<int32>& SelectionSet = ClothSelectionFacade.FindOrAddSelectionSet(PatternNameFaces.Key, ClothCollectionGroup::SimFaces);
				SelectionSet.Empty(PatternSet.Num());
				for (int32 Index = SimFacesOffset; Index < SimFacesOffset + PatternTriangleToVertexIndex.Num(); ++Index)
				{
					SelectionSet.Emplace(Index);
				}
			}
		}

		// Check the resulting cleaned mesh
		const int32 NumSimVertices2D = ClothFacade.GetNumSimVertices2D();
		const int32 NumSimFaces = ClothFacade.GetNumSimFaces();
		if (!NumSimVertices2D || !NumSimFaces)
		{
			return true;  // Empty mesh
		}

		const TConstArrayView<TArray<int32>> OriginalTriangles =
			ClothFacade.GetUserDefinedAttribute<TArray<int32>>(Private::OriginalIndicesName, ClothCollectionGroup::SimFaces);
		const TArray<int32> OriginalToNewFaceIndices = FClothDataflowTools::GetOriginalToNewIndices(OriginalTriangles, TriangleToVertexIndex.Num());

		const TConstArrayView<TArray<int32>> OriginalVertices =
			ClothFacade.GetUserDefinedAttribute<TArray<int32>>(Private::OriginalIndicesName, ClothCollectionGroup::SimVertices2D);
		const TArray<int32> OriginalToNewVertexIndices = FClothDataflowTools::GetOriginalToNewIndices(OriginalVertices, RestPositions2D.Num());

		// Add the sewings
		for (const TPair<FName, TSet<FIntVector2>>& SewingNameIndices : UsdClothData.Sewings)
		{
			TSet<FIntVector2> Indices;
			for (const FIntVector2& Stitch : SewingNameIndices.Value)
			{
				if (!OriginalToNewVertexIndices.IsValidIndex(Stitch[0]) || !OriginalToNewVertexIndices.IsValidIndex(Stitch[1]))
				{
					OutErrorText = LOCTEXT("BadSewingIndex", "An out of range sewing index has been found.");
					return false;
				}
				const int32 StitchIndex0 = OriginalToNewVertexIndices[Stitch[0]];
				const int32 StitchIndex1 = OriginalToNewVertexIndices[Stitch[1]];
				if (StitchIndex0 != INDEX_NONE && StitchIndex1 != INDEX_NONE)
				{
					Indices.Emplace(StitchIndex0 < StitchIndex1 ? FIntVector2(StitchIndex0, StitchIndex1) : FIntVector2(StitchIndex1, StitchIndex0));
				}
			}

			FCollectionClothSeamFacade ClothSeamFacade = ClothFacade.AddGetSeam();
			ClothSeamFacade.Initialize(Indices.Array());
		}

		// Add the springs
		const TSharedRef<FManagedArrayCollection> UsdClothDataCollection = MakeShared<FManagedArrayCollection>(UsdClothData.SimulationCollection);
		const FEmbeddedSpringFacade UsdClothDataSpringFacade(*UsdClothDataCollection, ClothCollectionGroup::SimVertices3D);
		if (UsdClothDataSpringFacade.IsValid())
		{
			// Initialize the spring facade
			FEmbeddedSpringFacade SpringFacade(*ClothCollection, ClothCollectionGroup::SimVertices3D);
			checkf(SpringFacade.IsValid(), TEXT("FEmbeddedSpringFacade constructor should have defined the schema."));
			FEmbeddedSpringConstraintFacade SpringConstraintFacade = SpringFacade.AddGetSpringConstraint();
			SpringConstraintFacade.Initialize(
				TConstArrayView<FIntVector2>(),
				TConstArrayView<float>(),
				TConstArrayView<float>(),
				TConstArrayView<float>(),
				TConstArrayView<float>(),
				TEXT("VertexSpringConstraint"));

			for (int32 ConstraintIndex = 0; ConstraintIndex < UsdClothDataSpringFacade.GetNumSpringConstraints(); ++ConstraintIndex)
			{
				const FEmbeddedSpringConstraintFacade UsdClothDataSpringConstraintFacade = UsdClothDataSpringFacade.GetSpringConstraintConst(ConstraintIndex);
				if (UsdClothDataSpringConstraintFacade.GetConstraintEndPointNumIndices() == FUintVector2(1, 1) &&
					UsdClothDataSpringConstraintFacade.GetConstraintName() == TEXT("VertexSpringConstraint"))
				{
					const TConstArrayView<int32> SimVertex3DLookup = ClothFacade.GetSimVertex3DLookup();

					TArray<TArray<int32>> SourceIndices(UsdClothDataSpringConstraintFacade.GetSourceIndexConst());
					TArray<TArray<int32>> TargetIndices(UsdClothDataSpringConstraintFacade.GetTargetIndexConst());
					for (TArray<int32>& SourceIndex : SourceIndices)
					{
						SourceIndex[0] = OriginalToNewVertexIndices.IsValidIndex(SourceIndex[0]) && OriginalToNewVertexIndices[SourceIndex[0]] != INDEX_NONE ?
							SimVertex3DLookup[OriginalToNewVertexIndices[SourceIndex[0]]] : INDEX_NONE;
					}
					for (TArray<int32>& TargetIndex : TargetIndices)
					{
						TargetIndex[0] = OriginalToNewVertexIndices.IsValidIndex(TargetIndex[0]) && OriginalToNewVertexIndices[TargetIndex[0]] != INDEX_NONE ?
							SimVertex3DLookup[OriginalToNewVertexIndices[TargetIndex[0]]] : INDEX_NONE;
					}
					SpringConstraintFacade.Append(
						SourceIndices,
						UsdClothDataSpringConstraintFacade.GetSourceWeightsConst(),
						TargetIndices,
						UsdClothDataSpringConstraintFacade.GetTargetWeightsConst(),
						UsdClothDataSpringConstraintFacade.GetSpringLengthConst());
				}
			}
			// Copy the properties
			const FCollectionPropertyConstFacade UsdClothDataPropertyFacade(UsdClothDataCollection);
			if (UsdClothDataPropertyFacade.IsValid())
			{
				FCollectionPropertyMutableFacade PropertyFacade(ClothCollection);
				PropertyFacade.Copy(UsdClothData.SimulationCollection);
			}
		}

		// Add the solver properties
		const FCollectionClothConstFacade SimulationClothFacade(UsdClothDataCollection);
		if (SimulationClothFacade.IsValid(EClothCollectionExtendedSchemas::Solvers))
		{
			ClothFacade.DefineSchema(EClothCollectionExtendedSchemas::Solvers);
			ClothFacade.SetSolverAirDamping(SimulationClothFacade.GetSolverAirDamping());
			ClothFacade.SetSolverGravity(SimulationClothFacade.GetSolverGravity());
			ClothFacade.SetSolverSubSteps(SimulationClothFacade.GetSolverSubSteps());
			ClothFacade.SetSolverTimeStep(SimulationClothFacade.GetSolverTimeStep());
		}

		// Add the fabric properties
		if (SimulationClothFacade.IsValid(EClothCollectionExtendedSchemas::Fabrics))
		{
			ClothFacade.DefineSchema(EClothCollectionExtendedSchemas::Fabrics);

			for (int32 FabricIndex = 0; FabricIndex < SimulationClothFacade.GetNumFabrics(); ++FabricIndex)
			{
				verify(ClothFacade.AddFabric() == FabricIndex);
				FCollectionClothFabricFacade ClothFabricFacade = ClothFacade.GetFabric(FabricIndex);
				const FCollectionClothFabricConstFacade SimulationClothFabricFacade = SimulationClothFacade.GetFabric(FabricIndex);

				ClothFabricFacade.Initialize(
					SimulationClothFabricFacade.GetBendingStiffness(),
					SimulationClothFabricFacade.GetBucklingRatio(),
					SimulationClothFabricFacade.GetBucklingStiffness(),
					SimulationClothFabricFacade.GetStretchStiffness(),
					SimulationClothFabricFacade.GetDensity(),
					SimulationClothFabricFacade.GetFriction(),
					SimulationClothFabricFacade.GetDamping(),
					SimulationClothFabricFacade.GetPressure(),
					SimulationClothFabricFacade.GetLayer(),
					SimulationClothFabricFacade.GetCollisionThickness());
			}
		}
	}
	return true;
}

bool FChaosClothAssetUSDImportNode_v2::ImportRenderStaticMesh(const TSharedRef<FManagedArrayCollection> ClothCollection, FText& OutErrorText)
{
	using namespace UE::Chaos::ClothAsset;

	FCollectionClothFacade ClothFacade(ClothCollection);
	check(ClothFacade.IsValid());  // The Cloth Collection schema must be valid at this point

	// Define the selection schema if needed
	FCollectionClothSelectionFacade ClothSelectionFacade(ClothCollection);
	if (!ClothSelectionFacade.IsValid())
	{
		ClothSelectionFacade.DefineSchema();
	}

	// Empty the current render mesh and previously create selections
	FClothGeometryTools::DeleteRenderMesh(ClothCollection);
	FClothGeometryTools::DeleteSelections(ClothCollection, ClothCollectionGroup::RenderFaces);

	// Make sure to clean the dependencies whatever the import status is
	ON_SCOPE_EXIT
		{
			// Bind to root bone on exit
			constexpr bool bBindSimMesh = false;
			constexpr bool bBindRenderMesh = true;
			FClothGeometryTools::BindMeshToRootBone(ClothCollection, bBindSimMesh, bBindRenderMesh);

			// Make sure to clean the dependencies whatever the import status is
			ImportedRenderAssets = Private::GetImportedAssetDependencies(ImportedRenderStaticMesh);
		};

	// Import the LOD 0
	if (ImportedRenderStaticMesh && ImportedRenderStaticMesh->GetNumSourceModels())
	{
		constexpr int32 LODIndex = 0;
		if (const FMeshDescription* const MeshDescription = ImportedRenderStaticMesh->GetMeshDescription(LODIndex))
		{
			const FMeshBuildSettings& BuildSettings = ImportedRenderStaticMesh->GetSourceModel(LODIndex).BuildSettings;
			FRenderMeshImport RenderMeshImport(*MeshDescription, BuildSettings);

			const TArray<FStaticMaterial>& StaticMaterials = ImportedRenderStaticMesh->GetStaticMaterials();
			RenderMeshImport.AddRenderSections(ClothCollection, StaticMaterials, Private::OriginalIndicesName, Private::OriginalIndicesName);

			// Create pattern selection sets
			const TConstArrayView<TArray<int32>> OriginalTriangles =
				ClothFacade.GetUserDefinedAttribute<TArray<int32>>(Private::OriginalIndicesName, ClothCollectionGroup::RenderFaces);

			if (OriginalTriangles.Num())
			{
				const TConstArrayView<FIntVector3> TriangleToVertexIndex = ClothFacade.GetRenderIndices();
				const TArray<int32> OriginalToNewTriangles = FClothDataflowTools::GetOriginalToNewIndices<TArray<int32>>(OriginalTriangles, TriangleToVertexIndex.Num());

				for (const TPair<FName, TSet<int32>>& PatternNameFaces : UsdClothData.RenderPatterns)
				{
					// Add the pattern triangle list as a selection set
					TSet<int32>& SelectionSet = ClothSelectionFacade.FindOrAddSelectionSet(PatternNameFaces.Key, ClothCollectionGroup::RenderFaces);
					SelectionSet.Empty(PatternNameFaces.Value.Num());
					for (const int32 Index : PatternNameFaces.Value)
					{
						if (OriginalToNewTriangles.IsValidIndex(Index) && OriginalToNewTriangles[Index] != INDEX_NONE)
						{
							SelectionSet.Emplace(OriginalToNewTriangles[Index]);
						}
					}
				}
			}
			// TODO: Proxy deformer
		}
		else
		{
			OutErrorText = LOCTEXT("MissingMeshDescription", "An imported render static mesh has no mesh description!");
			return false;
		}
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
