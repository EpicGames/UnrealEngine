// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/WeightMapNode.h"
#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/ClothDataflowViewModes.h"
#include "ChaosClothAsset/ClothGeometryTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ClothDataflowViewModes.h"
#include "Dataflow/DataflowRenderingViewMode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "Dataflow/DataflowInputOutput.h"
#include "Dataflow/DataflowObject.h"
#include "InteractiveToolChange.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WeightMapNode)

#define LOCTEXT_NAMESPACE "ChaosClothAssetWeightMapNode"

namespace UE::Chaos::ClothAsset::Private
{
	// These are defined in AddWeightMapNode.cpp

	void TransferWeightMap(
		const TConstArrayView<FVector2f>& InSourcePositions,
		const TConstArrayView<FIntVector3>& SourceIndices,
		const TConstArrayView<int32> SourceWeightsLookup,
		const TConstArrayView<float>& InSourceWeights,
		const TConstArrayView<FVector2f>& InTargetPositions,
		const TConstArrayView<FIntVector3>& TargetIndices,
		const TConstArrayView<int32> TargetWeightsLookup,
		TArray<float>& OutTargetWeights);

	void SetVertexWeights(const TConstArrayView<float> InputMap, const TArray<float>& FinalValues, EChaosClothAssetWeightMapOverrideType OverrideType, TArray<float>& SourceVertexWeights);

	void CalculateFinalVertexWeightValues(const TConstArrayView<float> InputMap, TArrayView<float> FinalOutputMap, EChaosClothAssetWeightMapOverrideType OverrideType, const TArray<float>& SourceVertexWeights);
}

FChaosClothAssetWeightMapNode::FChaosClothAssetWeightMapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowVertexAttributeEditableNode(InParam, InGuid)
	, Transfer(FDataflowFunctionProperty::FDelegate::CreateRaw(this, &FChaosClothAssetWeightMapNode::OnTransfer))
{
	RegisterInputConnection(&Collection);
	RegisterInputConnection(&InputName.StringValue, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableIStringValue, StringValue))
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterInputConnection(&TransferCollection)
		.SetCanHidePin(true)
		.SetPinIsHidden(true);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterOutputConnection(&OutputName.StringValue, (FString*)nullptr, GET_MEMBER_NAME_CHECKED(FChaosClothAssetConnectableOStringValue, StringValue));
}

void FChaosClothAssetWeightMapNode::OnTransfer(UE::Dataflow::FContext& Context)
{
	using namespace UE::Chaos::ClothAsset;

	// Transfer weight map if the transfer collection input has changed and is valid
	FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
	FCollectionClothConstFacade ClothFacade(ClothCollection);
	if (!ClothFacade.IsValid())
	{
		Context.Error(LOCTEXT("OnTransferError_InvalidCollection", "The input Collection is not a valid cloth collection"), this);
		return;
	}

	{
		FManagedArrayCollection InTransferCollection = GetValue<FManagedArrayCollection>(Context, &TransferCollection);

		// UseRenderMesh targeting the sim mesh can read directly from the Collection's own render mesh,
		// so a connected TransferCollection is optional in that case. All other paths require it.
		const bool bTransferCollectionRequired =
			MeshTarget != EChaosClothAssetWeightMapMeshTarget::Simulation ||
			TransferType != EChaosClothAssetWeightMapTransferType::UseRenderMesh;

		// Build the transfer facade only when a TransferCollection is actually connected.
		TSharedPtr<const FManagedArrayCollection> TransferClothCollectionPtr;
		TUniquePtr<FCollectionClothConstFacade> OwnedTransferFacade;
		if (InTransferCollection.IsEmpty())
		{
			if (bTransferCollectionRequired)
			{
				Context.Error(LOCTEXT("OnTransferError_TransferCollectionEmpty", "Transfer collection is empty, no source data to transfer from"), this);
				return;
			}
			// No TransferCollection provided and none required — continue with a null facade
			// (sim-target UseRenderMesh will fall back to the input Collection's own render mesh).
		}
		else
		{
			TransferClothCollectionPtr = MakeShared<const FManagedArrayCollection>(MoveTemp(InTransferCollection));
			OwnedTransferFacade = MakeUnique<FCollectionClothConstFacade>(TransferClothCollectionPtr.ToSharedRef());
		}

		const FName InInputName = GetInputName(Context);

		if (MeshTarget == EChaosClothAssetWeightMapMeshTarget::Simulation)
		{
			if (TransferType == EChaosClothAssetWeightMapTransferType::UseRenderMesh)
			{
				// Transfer render mesh weight map → sim mesh via 3D spatial (BVH) mapping.
				// Source: TransferCollection render mesh when connected; otherwise the Collection's own render mesh
				// (a SkeletalMeshImport with "Import Sim Mesh" produces a collection with both render and sim data).
				// HasWeightMap() only covers SimVertices3D, so render attributes are read via GetUserDefinedAttribute.
				const FCollectionClothConstFacade& SourceFacade = OwnedTransferFacade ? *OwnedTransferFacade : ClothFacade;

				const TConstArrayView<float> TransferRenderWeights =
					SourceFacade.GetUserDefinedAttribute<float>(InInputName, ClothCollectionGroup::RenderVertices);

				if (TransferRenderWeights.Num() > 0 && TransferRenderWeights.Num() == SourceFacade.GetNumRenderVertices())
				{
					TArray<float> RemappedWeights;
					RemappedWeights.SetNumZeroed(ClothFacade.GetNumSimVertices3D());

					FClothGeometryTools::TransferWeightMap(
						SourceFacade.GetRenderPosition(),
						SourceFacade.GetRenderIndices(),
						TransferRenderWeights,
						ClothFacade.GetSimPosition3D(),
						ClothFacade.GetSimNormal(),
						ClothFacade.GetSimIndices3D(),
						TArrayView<float>(RemappedWeights));

					TArray<float> WeightsToStore;
					Private::SetVertexWeights(ClothFacade.GetWeightMap(InInputName), RemappedWeights, MapOverrideType, WeightsToStore);
					StoreWeightsInSnapshot(&ClothFacade, WeightsToStore, AddSnapshot());
				}
				else
				{
					Context.Error(
						FText::Format(LOCTEXT("OnTransferError_InvalidRenderWeightmapForSim", "Invalid render weightmap [{0}]: no matching attribute found on the render mesh"),
							FText::FromName(InInputName)
						),
						this
					);
				}
			}
			else if (OwnedTransferFacade && OwnedTransferFacade->HasWeightMap(InInputName))
			{
				// Remap the weights
				FCollectionClothConstFacade& TransferClothFacade = *OwnedTransferFacade;
				TArray<float> RemappedWeights;
				RemappedWeights.SetNumZeroed(ClothFacade.GetNumSimVertices3D());

				switch (TransferType)
				{
				case EChaosClothAssetWeightMapTransferType::Use2DSimMesh:
					Private::TransferWeightMap(
						TransferClothFacade.GetSimPosition2D(),
						TransferClothFacade.GetSimIndices2D(),
						TransferClothFacade.GetSimVertex3DLookup(),
						TransferClothFacade.GetWeightMap(InInputName),
						ClothFacade.GetSimPosition2D(),
						ClothFacade.GetSimIndices2D(),
						ClothFacade.GetSimVertex3DLookup(),
						RemappedWeights);
					break;
				case EChaosClothAssetWeightMapTransferType::Use3DSimMesh:
					FClothGeometryTools::TransferWeightMap(
						TransferClothFacade.GetSimPosition3D(),
						TransferClothFacade.GetSimIndices3D(),
						TransferClothFacade.GetWeightMap(InInputName),
						ClothFacade.GetSimPosition3D(),
						ClothFacade.GetSimNormal(),
						ClothFacade.GetSimIndices3D(),
						TArrayView<float>(RemappedWeights));
					break;
				default: unimplemented();
				}

				TArray<float> WeightsToStore;
				Private::SetVertexWeights(ClothFacade.GetWeightMap(InInputName), RemappedWeights, MapOverrideType, WeightsToStore);
				StoreWeightsInSnapshot(&ClothFacade, WeightsToStore, AddSnapshot());
			}
			else
			{
				Context.Error(
					FText::Format(LOCTEXT("OnTransferError_InvalidSimWeightmap", "Invalid transfer collection sim weightmap [{0}], no source data to transfer from"),
						FText::FromName(InInputName)
					),
					this
				);
			}
		}
		else
		{
			check(MeshTarget == EChaosClothAssetWeightMapMeshTarget::Render);
			check(OwnedTransferFacade); // TransferCollection is required for render mesh target (enforced above)
			FCollectionClothConstFacade& TransferClothFacade = *OwnedTransferFacade;

			// Try to get a render weight map
			TConstArrayView<float> TransferWeightMap = TransferClothFacade.GetUserDefinedAttribute<float>(InInputName, ClothCollectionGroup::RenderVertices);
			if (TransferWeightMap.Num() > 0 && TransferWeightMap.Num() == TransferClothFacade.GetNumRenderVertices())
			{
				// Remap the weights
				TArray<float> RemappedWeights;
				RemappedWeights.SetNumZeroed(ClothFacade.GetNumRenderVertices());

				FClothGeometryTools::TransferWeightMap(
					TransferClothFacade.GetRenderPosition(),
					TransferClothFacade.GetRenderIndices(),
					TransferWeightMap,
					ClothFacade.GetRenderPosition(),
					ClothFacade.GetRenderNormal(),
					ClothFacade.GetRenderIndices(),
					TArrayView<float>(RemappedWeights));

				TArray<float> WeightsToStore;
				Private::SetVertexWeights(ClothFacade.GetUserDefinedAttribute<float>(InInputName, ClothCollectionGroup::RenderVertices), RemappedWeights, MapOverrideType, WeightsToStore);
				StoreWeightsInSnapshot(&ClothFacade, WeightsToStore, AddSnapshot());
			}
			else
			{
				Context.Error(
					FText::Format(LOCTEXT("InvalidRenderWeightmap", "Invalid transfer collection render weightmap [{0}], no source data to transfer from"),
						FText::FromName(InInputName)
					),
					this
				);
			}
		}
	}
}

void FChaosClothAssetWeightMapNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	auto CheckSourceVertexWeights = [this, &Context, Out](TArrayView<float>& ClothWeights, const TArray<float>& SourceVertexWeights, bool bIsSim)
	{
		if (SourceVertexWeights.Num() > 0 && SourceVertexWeights.Num() != ClothWeights.Num())
		{
			Context.Warning(
				FText::Format(LOCTEXT("VertexCountMismatchDetails", "{0} vertex weights in the node: {1}\n{0} vertices in the cloth: {2}"),
					bIsSim ? FText::FromString("Sim") : FText::FromString("Render"),
					SourceVertexWeights.Num(),
					ClothWeights.Num())
				, this, Out
			);
		}
	};

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		// Evaluate InputName
		const FName InInputName = GetInputName(Context);

		// Evaluate in collection
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothFacade ClothFacade(ClothCollection);
		if (ClothFacade.IsValid())  // Can only act on the collection if it is a valid cloth collection
		{
			const FName InName(OutputName.StringValue.IsEmpty() ? InInputName : FName(OutputName.StringValue));

			// Copy simulation weights into cloth collection

			if (MeshTarget == EChaosClothAssetWeightMapMeshTarget::Simulation)
			{
				ClothFacade.AddWeightMap(InName);		// Does nothing if weight map already exists
				TArrayView<float> ClothSimWeights = ClothFacade.GetWeightMap(InName);

				if (ClothSimWeights.Num() != ClothFacade.GetNumSimVertices3D())
				{
					check(ClothSimWeights.Num() == 0);
					Context.Warning(
						FText::Format(LOCTEXT("InvalidSimWeightMapNameDetails", "Could not create a sim weight map with name \"{0}\" (reserved name? wrong type?)."),
							FText::FromName(InName))
						, this, Out
					);
				}
				else
				{
					TArray<float> WeightMapValues;
					GetWeightsFromActiveSnapshot(&ClothFacade, WeightMapValues);

					constexpr bool bIsSim = true;
					CheckSourceVertexWeights(ClothSimWeights, WeightMapValues, bIsSim);
					CalculateFinalVertexWeightValues(&ClothFacade, ClothFacade.GetWeightMap(InInputName), ClothSimWeights);
				}
			}
			else 
			{
				check(MeshTarget == EChaosClothAssetWeightMapMeshTarget::Render);

				ClothFacade.AddUserDefinedAttribute<float>(InName, ClothCollectionGroup::RenderVertices);
				TArrayView<float> ClothRenderWeights = ClothFacade.GetUserDefinedAttribute<float>(InName, ClothCollectionGroup::RenderVertices);

				if (ClothRenderWeights.Num() != ClothFacade.GetNumRenderVertices())
				{
					check(ClothRenderWeights.Num() == 0);
					Context.Warning(
						FText::Format(LOCTEXT("InvalidRenderWeightMapNameDetails", "Could not create a render weight map with name \"{0}\" (reserved name? wrong type?)."),
							FText::FromName(InName))
						, this, Out
					);
				}
				else
				{
					TArray<float> WeightMapValues;
					GetWeightsFromActiveSnapshot(&ClothFacade, WeightMapValues);

					constexpr bool bIsSim = false;
					CheckSourceVertexWeights(ClothRenderWeights, WeightMapValues, bIsSim);
					CalculateFinalVertexWeightValues(&ClothFacade, ClothFacade.GetUserDefinedAttribute<float>(InInputName, ClothCollectionGroup::RenderVertices), ClothRenderWeights);
				}
			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
	else if (Out->IsA<FString>(&OutputName.StringValue))
	{
		FString InputNameString = GetValue<FString>(Context, &InputName.StringValue);
		UE::Chaos::ClothAsset::FWeightMapTools::MakeWeightMapName(InputNameString);
		SetValue(Context, OutputName.StringValue.IsEmpty() ? InputNameString : OutputName.StringValue, &OutputName.StringValue);
	}
}

void FChaosClothAssetWeightMapNode::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!Name.IsEmpty() && OutputName.StringValue.IsEmpty())  // TODO: Discard for v2
		{
			OutputName.StringValue = MoveTemp(Name);
			Name.Empty();
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

FDataflowOutput* FChaosClothAssetWeightMapNode::RedirectSerializedOutput(const FName& MissingOutputName)
{
	if (MissingOutputName == TEXT("Name"))
	{
		return FindOutput(FName(TEXT("OutputName.StringValue")));
	}
	return nullptr;
}

FName FChaosClothAssetWeightMapNode::GetInputName(UE::Dataflow::FContext& Context) const
{
	FString InputNameString = GetValue<FString>(Context, &InputName.StringValue);
	UE::Chaos::ClothAsset::FWeightMapTools::MakeWeightMapName(InputNameString);
	const FName InInputName(*InputNameString);
	return InInputName != NAME_None ? InInputName : FName(OutputName.StringValue);
}

void FChaosClothAssetWeightMapNode::GetSupportedViewModes(UE::Dataflow::FContext& Context, TArray<FName>& OutViewModeNames) const
{
	switch (MeshTarget)
	{
	case EChaosClothAssetWeightMapMeshTarget::Simulation:
		OutViewModeNames.Add(UE::Chaos::ClothAsset::FCloth2DSimViewMode::Name);
		OutViewModeNames.Add(UE::Chaos::ClothAsset::FCloth3DSimViewMode::Name);
		break;
	case EChaosClothAssetWeightMapMeshTarget::Render:
		OutViewModeNames.Add(UE::Chaos::ClothAsset::FClothRenderViewMode::Name);
		break;
	} 
}

void FChaosClothAssetWeightMapNode::GetVertexAttributeValues(UE::Dataflow::FContext& Context, TArray<float>& OutValues) const
{
	using namespace UE::Chaos::ClothAsset;

	const FName InInputName = GetInputName(Context);

	FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
	FCollectionClothConstFacade ClothFacade(ClothCollection);
	if (ClothFacade.IsValid())  // Can only act on the collection if it is a valid cloth collection
	{
		TConstArrayView<float> InputAttributeValues;
		int32 NumValues = 0;
		if (MeshTarget == EChaosClothAssetWeightMapMeshTarget::Simulation)
		{
			InputAttributeValues = ClothFacade.GetWeightMap(InInputName);
			NumValues = ClothFacade.GetNumSimVertices3D();
		}
		else
		{
			InputAttributeValues = ClothFacade.GetUserDefinedAttribute<float>(InInputName, ClothCollectionGroup::RenderVertices);
			NumValues = ClothFacade.GetNumRenderVertices();
		}
		TArray<float> FallbackInputValues;
		if (InputAttributeValues.IsEmpty())
		{
			FallbackInputValues.Init(0, NumValues);
			InputAttributeValues = FallbackInputValues;
		}

		// This keeps backward compatibility to load what's on the node 
		TArray<float> WeightMapValues;
		GetWeightsFromActiveSnapshot(&ClothFacade, WeightMapValues);

		OutValues.Init(0, NumValues);
		UE::Chaos::ClothAsset::Private::CalculateFinalVertexWeightValues(InputAttributeValues, OutValues, MapOverrideType, WeightMapValues);
	}
}

void FChaosClothAssetWeightMapNode::SetVertexAttributeValues(UE::Dataflow::FContext& Context, const TArray<float>& InValues, const TArray<int32>& InWeightIndices)
{
	using namespace UE::Chaos::ClothAsset;

	const FName InInputName = GetInputName(Context);

	FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
	const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
	FCollectionClothConstFacade ClothFacade(ClothCollection);
	if (ClothFacade.IsValid())  // Can only act on the collection if it is a valid cloth collection
	{
		TConstArrayView<float> InputAttributeValues;
		int32 NumFinalWeights;
		if (MeshTarget == EChaosClothAssetWeightMapMeshTarget::Simulation)
		{
			if ((InValues.Num() != ClothFacade.GetNumSimVertices3D()) &&
				(InValues.Num() != ClothFacade.GetNumSimVertices2D() || InWeightIndices.Num() != ClothFacade.GetNumSimVertices2D()))
			{
				Context.Error(LOCTEXT("MeshTargetChangedToRender", "Can't match the painted vertex back to the Cloth Collection, most likely because the Mesh Target has been changed to Render while painting a Simulation weight map."));
				return;
			}
			InputAttributeValues = ClothFacade.GetWeightMap(InInputName);
			NumFinalWeights = ClothFacade.GetNumSimVertices3D();
		}
		else
		{
			if (InValues.Num() != ClothFacade.GetNumRenderVertices())
			{
				Context.Error(LOCTEXT("MeshTargetChangedToSim", "Can't match the painted vertex back to the Cloth Collection, most likely because the Mesh Target has been changed to Simulation while painting a Render weight map."));
				return;
			}
			InputAttributeValues = ClothFacade.GetUserDefinedAttribute<float>(InInputName, ClothCollectionGroup::RenderVertices);
			NumFinalWeights = ClothFacade.GetNumRenderVertices();
		}

		TArray<float> ValuesToStore;
		if (InWeightIndices.Num() == InValues.Num())
		{
			TArray<float> ValuesToApply;
			ValuesToApply.Init(0.f, NumFinalWeights);
			for (int32 Index = 0; Index < InWeightIndices.Num(); ++Index)
			{
				const int32 WeightIndex = InWeightIndices[Index];
				if (ValuesToApply.IsValidIndex(WeightIndex))
				{
					ValuesToApply[InWeightIndices[Index]] = InValues[Index];
				}
			}
			UE::Chaos::ClothAsset::Private::SetVertexWeights(InputAttributeValues, ValuesToApply, MapOverrideType, ValuesToStore);
		}
		else
		{
			UE::Chaos::ClothAsset::Private::SetVertexWeights(InputAttributeValues, InValues, MapOverrideType, ValuesToStore);
		}

		StoreWeightsInSnapshot(&ClothFacade, ValuesToStore, AddSnapshot());
	}
}

void FChaosClothAssetWeightMapNode::GetExtraVertexMapping(UE::Dataflow::FContext& Context, FName SelectedViewMode, TArray<int32>& OutMappingToWeight, TArray<TArray<int32>>& OutMappingFromWeight) const
{
	OutMappingToWeight.Reset();
	OutMappingFromWeight.Reset();

	using namespace UE::Chaos::ClothAsset;
	if (MeshTarget == EChaosClothAssetWeightMapMeshTarget::Simulation)
	{
		if (SelectedViewMode == FCloth2DSimViewMode::Name || SelectedViewMode == UE::Dataflow::FDataflowConstruction2DViewMode::Name)
		{
			FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
			const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
			FCollectionClothConstFacade ClothFacade(ClothCollection);
			if (ClothFacade.IsValid())  // Can only act on the collection if it is a valid cloth collection
			{
				OutMappingToWeight = ClothFacade.GetSimVertex3DLookup();
				OutMappingFromWeight = ClothFacade.GetSimVertex2DLookup();
			}
		}
	}
}

void FChaosClothAssetWeightMapNode::SetVertexWeights(const TConstArrayView<float> InputMap, const TArray<float>& FinalValues)
{
	TArray<float> WeightsToStore;
	UE::Chaos::ClothAsset::Private::SetVertexWeights(InputMap, FinalValues, MapOverrideType, WeightsToStore);

	StoreWeightsInSnapshot(/*ClothFacade*/nullptr, WeightsToStore, AddSnapshot());
}

void FChaosClothAssetWeightMapNode::CalculateFinalVertexWeightValues(UE::Chaos::ClothAsset::FCollectionClothConstFacade* ClothFacade, const TConstArrayView<float> InputMap, TArrayView<float> FinalOutputMap) const
{
	TArray<float> WeightMapValues;
	GetWeightsFromActiveSnapshot(ClothFacade, WeightMapValues);

	UE::Chaos::ClothAsset::Private::CalculateFinalVertexWeightValues(InputMap, FinalOutputMap, MapOverrideType, WeightMapValues);
}

namespace ChaosClothAssetWeightMapNode::Private
{
	static FName VertexGroupName = TEXT("Vertices");
	static FName VertexPositionsAttributeName = TEXT("Positions");
	static FName VertexWeightsAttributeName = TEXT("Weights");

	static FName FaceGroupName = TEXT("Faces");
	static FName FaceIndicesAttributeName = TEXT("Indices");

	template<typename T>
	void CopyData(TConstArrayView<T> From, TManagedArray<T>& To)
	{
		if (ensure(From.Num() == To.Num()))
		{
			for (int32 Index = 0; Index < From.Num(); ++Index)
			{
				To[Index] = From[Index];
			}
		}
	};
}

void FChaosClothAssetWeightMapNode::GetWeightsFromActiveSnapshot(UE::Chaos::ClothAsset::FCollectionClothConstFacade* ClothFacade, TArray<float>& OutValues) const
{
	if (const FDataflowToolNodeSnapshot* ActiveSnapshot = GetActiveSnapshot())
	{
		GetWeightsFromSnapshot(ClothFacade, *ActiveSnapshot, OutValues);
	}
	else
	{
		// This keeps backward compatibility to load what's on the node before we add snapshots
		OutValues = GetLegacyVertexWeights();
	}
}

void FChaosClothAssetWeightMapNode::GetWeightsFromSnapshot(UE::Chaos::ClothAsset::FCollectionClothConstFacade* ClothFacade, const FDataflowToolNodeSnapshot& Snapshot, TArray<float>& OutValues) const
{
	using namespace UE::Chaos::ClothAsset;

	OutValues.Reset();

	TConstArrayView<FVector3f> ClothVertices;
	TConstArrayView<FIntVector> ClothFaceIndices;
	if (ClothFacade && ClothFacade->IsValid())
	{
		if (MeshTarget == EChaosClothAssetWeightMapMeshTarget::Simulation)
		{
			ClothVertices = ClothFacade->GetSimPosition3D();
			ClothFaceIndices = ClothFacade->GetSimIndices3D();
		}
		else if (MeshTarget == EChaosClothAssetWeightMapMeshTarget::Render)
		{
			ClothVertices = ClothFacade->GetRenderPosition();
			ClothFaceIndices = ClothFacade->GetRenderIndices();
		}
	}

	const TManagedArray<float>* VtxWeights = Snapshot.Data.FindAttribute<float>(ChaosClothAssetWeightMapNode::Private::VertexWeightsAttributeName, ChaosClothAssetWeightMapNode::Private::VertexGroupName);
	if (VtxWeights)
	{
		const TManagedArray<FVector3f>* VtxPositions = Snapshot.Data.FindAttribute<FVector3f>(ChaosClothAssetWeightMapNode::Private::VertexPositionsAttributeName, ChaosClothAssetWeightMapNode::Private::VertexGroupName);
		bool bSameVertices = (VtxPositions && VtxPositions->Num() == ClothVertices.Num());
		if (bSameVertices)
		{
			// more expensive check
			for (int32 VtxIndex = 0; VtxIndex < ClothVertices.Num(); ++VtxIndex)
			{
				if (!FVector3f::PointsAreSame(ClothVertices[VtxIndex], (*VtxPositions)[VtxIndex]))
				{
					bSameVertices = false;
					break;
				}
			}
		}

		const TManagedArray<FIntVector>* FaceIndices = Snapshot.Data.FindAttribute<FIntVector>(ChaosClothAssetWeightMapNode::Private::FaceIndicesAttributeName, ChaosClothAssetWeightMapNode::Private::FaceGroupName);
		bool bSameFaceIndices = (FaceIndices && FaceIndices->Num() == ClothFaceIndices.Num());
		if (bSameVertices && bSameFaceIndices)
		{
			// more expensive check
			for (int32 FaceIndex = 0; FaceIndex < ClothFaceIndices.Num(); ++FaceIndex)
			{
				if (ClothFaceIndices[FaceIndex] != (*FaceIndices)[FaceIndex])
				{
					bSameFaceIndices = false;
					break;
				}
			}
		}

		bool bMatchOriginalTopology = (bSameVertices && bSameFaceIndices);
		const bool bIsValidSourceMesh = (VtxPositions && FaceIndices);
		const bool bIsValidTargetMesh = (ClothVertices.Num() > 0 && ClothFaceIndices.Num() > 0);

		if (!bMatchOriginalTopology)
		{
			if (bIsValidSourceMesh && bIsValidTargetMesh && ClothFacade)
			{
				// We need to transfer the transfer the data over to the new topology 
				if (MeshTarget == EChaosClothAssetWeightMapMeshTarget::Simulation)
				{
					// Remap the weights
					OutValues.SetNumZeroed(ClothFacade->GetNumSimVertices3D());

					// in this automatic mode we only transfer on 3D model because the patterns may have changed and be in different place
					// we also only store the original 3D Mesh 
					FClothGeometryTools::TransferWeightMap(
						VtxPositions->GetConstArray(),
						FaceIndices->GetConstArray(),
						VtxWeights->GetConstArray(),
						ClothFacade->GetSimPosition3D(),
						ClothFacade->GetSimNormal(),
						ClothFacade->GetSimIndices3D(),
						MakeArrayView(OutValues));
				}
				else if (MeshTarget == EChaosClothAssetWeightMapMeshTarget::Render)
				{
					OutValues.SetNumZeroed(ClothFacade->GetNumRenderVertices());

					FClothGeometryTools::TransferWeightMap(
						VtxPositions->GetConstArray(),
						FaceIndices->GetConstArray(),
						VtxWeights->GetConstArray(),
						ClothFacade->GetRenderPosition(),
						ClothFacade->GetRenderNormal(),
						ClothFacade->GetRenderIndices(),
						MakeArrayView(OutValues));
				}
			}
		}
		else 
		{
			// we simply read the weights as they are 
			OutValues.Append(VtxWeights->GetConstArray());
		}
	}
}

void FChaosClothAssetWeightMapNode::StoreWeightsInSnapshot(UE::Chaos::ClothAsset::FCollectionClothConstFacade* ClothFacade, const TArray<float>& InValues, FDataflowToolNodeSnapshot& OutSnapshot) const
{
	using namespace UE::Chaos::ClothAsset;

	OutSnapshot.Data.Reset();

	if (InValues.Num())
	{
		TManagedArray<float>& VtxWeights = OutSnapshot.Data.AddAttribute<float>(ChaosClothAssetWeightMapNode::Private::VertexWeightsAttributeName, ChaosClothAssetWeightMapNode::Private::VertexGroupName);

		// for backward compatibility with cloth tools, we need to support case where the Cloth facade is not available 
		// all new Dataflow workflow should have a valid cloth collection though
		if (ClothFacade && ClothFacade->IsValid())
		{
			TManagedArray<FVector3f>& VtxPositions = OutSnapshot.Data.AddAttribute<FVector3f>(ChaosClothAssetWeightMapNode::Private::VertexPositionsAttributeName, ChaosClothAssetWeightMapNode::Private::VertexGroupName);
			TManagedArray<FIntVector>& FaceIndices = OutSnapshot.Data.AddAttribute<FIntVector>(ChaosClothAssetWeightMapNode::Private::FaceIndicesAttributeName, ChaosClothAssetWeightMapNode::Private::FaceGroupName);

			if (MeshTarget == EChaosClothAssetWeightMapMeshTarget::Simulation)
			{
				OutSnapshot.Data.Resize(ClothFacade->GetNumSimVertices3D(), ChaosClothAssetWeightMapNode::Private::VertexGroupName);
				OutSnapshot.Data.Resize(ClothFacade->GetNumSimFaces(), ChaosClothAssetWeightMapNode::Private::FaceGroupName);

				ChaosClothAssetWeightMapNode::Private::CopyData(ClothFacade->GetSimPosition3D(), VtxPositions);
				ChaosClothAssetWeightMapNode::Private::CopyData(MakeArrayView(InValues), VtxWeights);
				ChaosClothAssetWeightMapNode::Private::CopyData(ClothFacade->GetSimIndices3D(), FaceIndices);
			}
			else if (MeshTarget == EChaosClothAssetWeightMapMeshTarget::Render)
			{
				OutSnapshot.Data.Resize(ClothFacade->GetNumRenderVertices(), ChaosClothAssetWeightMapNode::Private::VertexGroupName);
				OutSnapshot.Data.Resize(ClothFacade->GetNumRenderFaces(), ChaosClothAssetWeightMapNode::Private::FaceGroupName);

				ChaosClothAssetWeightMapNode::Private::CopyData(ClothFacade->GetRenderPosition(), VtxPositions);
				ChaosClothAssetWeightMapNode::Private::CopyData(MakeArrayView(InValues), VtxWeights);
				ChaosClothAssetWeightMapNode::Private::CopyData(ClothFacade->GetRenderIndices(), FaceIndices);
			}
		}
		else
		{
			// backward compatibility path , we only store the weights
			OutSnapshot.Data.Resize(InValues.Num(), ChaosClothAssetWeightMapNode::Private::VertexGroupName);
			ChaosClothAssetWeightMapNode::Private::CopyData(MakeArrayView(InValues), VtxWeights);
		}
	}
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

// Object encapsulating a change to the WeightMap node's values. Used for Undo/Redo.
class FChaosClothAssetWeightMapNode::FWeightMapNodeChange final 
	: public FDataflowToolNode::FSnapshotToolChange
{
public:
	FWeightMapNodeChange(FChaosClothAssetWeightMapNode& Node) 
		: FSnapshotToolChange(Node)
		, SavedMapOverrideType(Node.MapOverrideType)
		, SavedWeightMapName(Node.OutputName.StringValue)
	{}

private:
	EChaosClothAssetWeightMapOverrideType SavedMapOverrideType;
	FString SavedWeightMapName;

	virtual FString ToString() const final
	{
		return TEXT("FChaosClothAssetWeightMapNode::FWeightMapNodeChange");
	}

	virtual void SwapApplyRevert(UObject* Object, FDataflowToolNode& Node) override final
	{
		if (FChaosClothAssetWeightMapNode* TypedNode = Node.AsType<FChaosClothAssetWeightMapNode>())
		{
			Swap(TypedNode->MapOverrideType, SavedMapOverrideType);
			Swap(TypedNode->OutputName.StringValue, SavedWeightMapName);
		}
		FDataflowToolNode::FSnapshotToolChange::SwapApplyRevert(Object, Node);
	}
};

TUniquePtr<FToolCommandChange> FChaosClothAssetWeightMapNode::MakeWeightMapNodeChange(FChaosClothAssetWeightMapNode& Node)
{
	return MakeUnique<FChaosClothAssetWeightMapNode::FWeightMapNodeChange>(Node);
}

#undef LOCTEXT_NAMESPACE
