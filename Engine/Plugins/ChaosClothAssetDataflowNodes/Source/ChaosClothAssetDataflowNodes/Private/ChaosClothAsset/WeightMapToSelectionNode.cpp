// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/WeightMapToSelectionNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "ChaosClothAsset/ClothDataflowTools.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/CollectionClothSelectionFacade.h"
#include "Dataflow/DataflowInputOutput.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WeightMapToSelectionNode)
#define LOCTEXT_NAMESPACE "FChaosClothAssetWeightMapToSelectionNode"

FChaosClothAssetWeightMapToSelectionNode::FChaosClothAssetWeightMapToSelectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid)
	: FDataflowNode(InParam, InGuid)
{
	RegisterInputConnection(&Collection);
	RegisterOutputConnection(&Collection, &Collection);
	RegisterInputConnection(&WeightMapName);
	RegisterOutputConnection(&SelectionName);
}

void FChaosClothAssetWeightMapToSelectionNode::Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const
{
	using namespace UE::Chaos::ClothAsset;

	if (Out->IsA<FManagedArrayCollection>(&Collection))
	{
		FManagedArrayCollection InCollection = GetValue<FManagedArrayCollection>(Context, &Collection);
		const TSharedRef<FManagedArrayCollection> ClothCollection = MakeShared<FManagedArrayCollection>(MoveTemp(InCollection));
		FCollectionClothConstFacade ClothFacade(ClothCollection);
		const FName InWeightMapName(*GetValue<FString>(Context, &WeightMapName));

		if (ClothFacade.IsValid())
		{
			if (InWeightMapName != NAME_None && ClothFacade.HasWeightMap(InWeightMapName))
			{
				TConstArrayView<float> WeightMap = ClothFacade.GetWeightMap(InWeightMapName);

				FName SelectionGroup;
				switch (SelectionType)
				{
				default:
				case EChaosClothAssetWeightMapConvertableSelectionType::SimVertices3D:
					SelectionGroup = ClothCollectionGroup::SimVertices3D;
					break;
				case EChaosClothAssetWeightMapConvertableSelectionType::SimVertices2D:
					SelectionGroup = ClothCollectionGroup::SimVertices2D;
					break;
				case EChaosClothAssetWeightMapConvertableSelectionType::SimFaces:
					SelectionGroup = ClothCollectionGroup::SimFaces;
					break;
				}

				FCollectionClothSelectionFacade SelectionFacade(ClothCollection);
				SelectionFacade.DefineSchema();

				const FName InSelectionName(SelectionName.IsEmpty() ? InWeightMapName : FName(SelectionName));
				TSet<int32>& SelectionSet = SelectionFacade.FindOrAddSelectionSet(InSelectionName, SelectionGroup);
				SelectionSet.Reset();

				if (SelectionGroup == ClothCollectionGroup::SimVertices3D)
				{
					for (int32 VertexIndex = 0; VertexIndex < WeightMap.Num(); ++VertexIndex)
					{
						if (WeightMap[VertexIndex] >= SelectionThreshold)
						{
							SelectionSet.Add(VertexIndex);
						}
					}
				}
				else if (SelectionGroup == ClothCollectionGroup::SimVertices2D)
				{
					const TConstArrayView<TArray<int32>> Vertex3DTo2D = ClothFacade.GetSimVertex2DLookup();

					for (int32 Vertex3DIndex = 0; Vertex3DIndex < WeightMap.Num(); ++Vertex3DIndex)
					{
						// Select all 2D vertices that correspond with 3D vertices above the threshold

						if (WeightMap[Vertex3DIndex] >= SelectionThreshold)
						{
							for (const int32 Vertex2DIndex : Vertex3DTo2D[Vertex3DIndex])
							{
								SelectionSet.Add(Vertex2DIndex);
							}
						}
					}
				}
				else
				{
					check(SelectionGroup == ClothCollectionGroup::SimFaces);

					const TConstArrayView<FIntVector3> SimIndices3D = ClothFacade.GetSimIndices3D();
					// Select a face if all 3 vertices are above the threshold
					for (int32 FaceIndex = 0; FaceIndex < SimIndices3D.Num(); ++FaceIndex)
					{
						const FIntVector3& Index = SimIndices3D[FaceIndex];
						if (WeightMap[Index[0]] >= SelectionThreshold &&
							WeightMap[Index[1]] >= SelectionThreshold &&
							WeightMap[Index[2]] >= SelectionThreshold)
						{
							SelectionSet.Add(FaceIndex);
						}
					}
				}
			}
			else
			{
				Context.Warning(
					FText::Format(LOCTEXT("WeightMapeNotFoundDetails", "Weight map \"{0}\" was not found in the collection."),
						FText::FromName(InWeightMapName))
					, this, Out
				);

			}
		}

		SetValue(Context, MoveTemp(*ClothCollection), &Collection);
	}
	else if (Out->IsA<FString>(&SelectionName))
	{
		SetValue(Context, SelectionName.IsEmpty() ? GetValue<FString>(Context, &WeightMapName) : SelectionName, &SelectionName);
	}
}
#undef LOCTEXT_NAMESPACE
