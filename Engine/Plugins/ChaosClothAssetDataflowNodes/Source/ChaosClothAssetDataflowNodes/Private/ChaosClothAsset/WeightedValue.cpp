// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/WeightedValue.h"
#include "ChaosClothAsset/ClothDataflowTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WeightedValue)

namespace UE::Chaos::ClothAsset
{
	void FWeightMapTools::MakeWeightMapName(FString& InOutString)
	{
		FClothDataflowTools::MakeCollectionName(InOutString);
	}
}  // End namespace UE::Chaos::ClothAsset
