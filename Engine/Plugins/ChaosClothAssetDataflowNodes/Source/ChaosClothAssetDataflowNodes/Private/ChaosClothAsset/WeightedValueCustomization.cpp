// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/WeightedValueCustomization.h"
#include "ChaosClothAsset/ClothAssetEditorStyle.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "ChaosClothAssetWeightedValueCustomization"

namespace UE::Chaos::ClothAsset
{
	namespace Private
	{
		static const FString ImportFabricBounds = TEXT("ImportFabricBounds");
		
		static bool ImportFabricBoundsProperty(const TSharedPtr<IPropertyHandle>& Property)
		{
			const FStringView PropertyPath = Property ? Property->GetPropertyPath() : FStringView();
			return PropertyPath.EndsWith(ImportFabricBounds, ESearchCase::CaseSensitive);
		}
	}

	TSharedRef<IPropertyTypeCustomization> FWeightedValueCustomization::MakeInstance()
	{
		return MakeShareable(new FWeightedValueCustomization);
	}

	FWeightedValueCustomization::FWeightedValueCustomization() = default;
	
	FWeightedValueCustomization::~FWeightedValueCustomization() = default;


	void FWeightedValueCustomization::MakeHeaderRow(TSharedRef<class IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row)
	{
		const TWeakPtr<IPropertyHandle> StructWeakHandlePtr = StructPropertyHandle;

		TSharedPtr<SHorizontalBox> ValueHorizontalBox;
		TSharedPtr<SHorizontalBox> NameHorizontalBox;

		Row.NameContent()
			[
				SAssignNew(NameHorizontalBox, SHorizontalBox)
				.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, StructWeakHandlePtr)
			]
		.ValueContent()
			// Make enough space for each child handle
			.MinDesiredWidth(125.f * SortedChildHandles.Num())
			.MaxDesiredWidth(125.f * SortedChildHandles.Num())
			[
				SAssignNew(ValueHorizontalBox, SHorizontalBox)
				.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, StructWeakHandlePtr)
			];
		
		for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];
			
			if (CouldUseFabricsProperty(ChildHandle))
			{
				bool bValue = false;
				ChildHandle->GetValue(bValue);
				if(!bValue)
				{
					break;
				}
			}
			if (Private::ImportFabricBoundsProperty(ChildHandle))
			{
				AddToggledCheckBox(ChildHandle, NameHorizontalBox, FAppStyle::Get().GetBrush("Icons.Import"));
			}
			else if (BuildFabricMapsProperty(ChildHandle))
			{
				AddToggledCheckBox(ChildHandle, NameHorizontalBox, UE::Chaos::ClothAsset::FClothAssetEditorStyle::Get().GetBrush("ClassIcon.ChaosClothPreset"));
			}
		}
		
		NameHorizontalBox->AddSlot().VAlign(VAlign_Center)
				.Padding(FMargin(4.f, 2.f, 4.0f, 2.f))
				.HAlign(HAlign_Right)
				.AutoWidth()
				[
					StructPropertyHandle->CreatePropertyNameWidget()
				];

		for (int32 ChildIndex = 0; ChildIndex < SortedChildHandles.Num(); ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = SortedChildHandles[ChildIndex];

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (IsOverrideProperty(ChildHandle))
			{
				continue;  // Skip overrides
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			// Propagate metadata to child properties so that it's reflected in the nested, individual spin boxes
			ChildHandle->SetInstanceMetaData(TEXT("UIMin"), StructPropertyHandle->GetMetaData(TEXT("UIMin")));
			ChildHandle->SetInstanceMetaData(TEXT("UIMax"), StructPropertyHandle->GetMetaData(TEXT("UIMax")));
			ChildHandle->SetInstanceMetaData(TEXT("SliderExponent"), StructPropertyHandle->GetMetaData(TEXT("SliderExponent")));
			ChildHandle->SetInstanceMetaData(TEXT("Delta"), StructPropertyHandle->GetMetaData(TEXT("Delta")));
			ChildHandle->SetInstanceMetaData(TEXT("LinearDeltaSensitivity"), StructPropertyHandle->GetMetaData(TEXT("LinearDeltaSensitivity")));
			ChildHandle->SetInstanceMetaData(TEXT("ShiftMultiplier"), StructPropertyHandle->GetMetaData(TEXT("ShiftMultiplier")));
			ChildHandle->SetInstanceMetaData(TEXT("CtrlMultiplier"), StructPropertyHandle->GetMetaData(TEXT("CtrlMultiplier")));
			ChildHandle->SetInstanceMetaData(TEXT("SupportDynamicSliderMaxValue"), StructPropertyHandle->GetMetaData(TEXT("SupportDynamicSliderMaxValue")));
			ChildHandle->SetInstanceMetaData(TEXT("SupportDynamicSliderMinValue"), StructPropertyHandle->GetMetaData(TEXT("SupportDynamicSliderMinValue")));
			ChildHandle->SetInstanceMetaData(TEXT("ClampMin"), StructPropertyHandle->GetMetaData(TEXT("ClampMin")));
			ChildHandle->SetInstanceMetaData(TEXT("ClampMax"), StructPropertyHandle->GetMetaData(TEXT("ClampMax")));

			const bool bLastChild = SortedChildHandles.Num() - 1 == ChildIndex;

			TSharedRef<SWidget> ChildWidget = MakeChildWidget(StructPropertyHandle, ChildHandle);
			if(ChildWidget != SNullWidget::NullWidget)
			{
				if (ChildHandle->GetPropertyClass() == FBoolProperty::StaticClass())
				{
					ValueHorizontalBox->AddSlot()
						.Padding(FMargin(0.f, 2.f, bLastChild ? 0.f : 3.f, 2.f))
						.AutoWidth()  // keep the check box slots small
						[
							ChildWidget
						];
				}
				else
				{
					if (ChildHandle->GetPropertyClass() == FFloatProperty::StaticClass())
					{
						NumericEntryBoxWidgetList.Add(ChildWidget);
					}

					ValueHorizontalBox->AddSlot()
						.Padding(FMargin(0.f, 2.f, bLastChild ? 0.f : 3.f, 2.f))
						[
							ChildWidget
						];
				}
			}
		}
	}

	TSharedRef<SWidget> FWeightedValueCustomization::MakeChildWidget(
		TSharedRef<IPropertyHandle>& StructurePropertyHandle,
		TSharedRef<IPropertyHandle>& PropertyHandle)
	{
		const FFieldClass* PropertyClass = PropertyHandle->GetPropertyClass();

		if (PropertyClass == FFloatProperty::StaticClass())
		{
			return MakeFloatWidget(StructurePropertyHandle, PropertyHandle);
		}
		if (PropertyClass == FBoolProperty::StaticClass())
		{
			TWeakPtr<IPropertyHandle> WeakHandlePtr = PropertyHandle;
			if (!Private::ImportFabricBoundsProperty(PropertyHandle) && !BuildFabricMapsProperty(PropertyHandle) && !CouldUseFabricsProperty(PropertyHandle))
			{
				return
					SNew(SCheckBox)
					.ToolTipText(PropertyHandle->GetToolTipText())
					.Type(ESlateCheckBoxType::CheckBox)
					.IsChecked_Lambda([WeakHandlePtr]()->ECheckBoxState
						{
							bool bValue = false;
							if (const TSharedPtr<IPropertyHandle> BoolPropertyHandle = WeakHandlePtr.Pin())
							{
								BoolPropertyHandle->GetValue(bValue);
							}
							return bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						})
					.OnCheckStateChanged_Lambda([WeakHandlePtr](ECheckBoxState CheckBoxState)
						{
							if (const TSharedPtr<IPropertyHandle> BoolPropertyHandle = WeakHandlePtr.Pin())
							{
								BoolPropertyHandle->SetValue(CheckBoxState == ECheckBoxState::Checked, EPropertyValueSetFlags::DefaultFlags);
							}
						});
			}
		}
		return FConnectableValueCustomization::MakeChildWidget(StructurePropertyHandle, PropertyHandle);
	}

	TSharedRef<SWidget> FWeightedValueCustomization::MakeFloatWidget(
		TSharedRef<IPropertyHandle>& StructurePropertyHandle,
		TSharedRef<IPropertyHandle>& PropertyHandle)
	{
		TOptional<float> MinValue, MaxValue, SliderMinValue, SliderMaxValue;
		float SliderExponent, Delta;
		float ShiftMultiplier = 10.f;
		float CtrlMultiplier = 0.1f;
		bool SupportDynamicSliderMaxValue = false;
		bool SupportDynamicSliderMinValue = false;

		ExtractFloatMetadata(StructurePropertyHandle, MinValue, MaxValue, SliderMinValue, SliderMaxValue, SliderExponent, Delta, ShiftMultiplier, CtrlMultiplier, SupportDynamicSliderMaxValue, SupportDynamicSliderMinValue);

		TWeakPtr<IPropertyHandle> WeakHandlePtr = PropertyHandle;

		return SNew(SNumericEntryBox<float>)
			.IsEnabled(this, &FMathStructCustomization::IsValueEnabled, WeakHandlePtr)
			.EditableTextBoxStyle(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
			.Value_Lambda([WeakHandlePtr]()
				{
					float Value = 0.;
					if (const TSharedPtr<IPropertyHandle> FloatPropertyHandle = WeakHandlePtr.Pin())
					{
						if (FloatPropertyHandle->GetValue(Value) == FPropertyAccess::Success)
						{
							return TOptional<float>(Value);
						}
					}
					return TOptional<float>();  // Value couldn't be accessed, return an unset value
				})
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
			.OnValueCommitted_Lambda([WeakHandlePtr](float Value, ETextCommit::Type)
				{
					WeakHandlePtr.Pin()->SetValue(Value, EPropertyValueSetFlags::DefaultFlags);
				})
			.OnValueChanged_Lambda([this, WeakHandlePtr](float Value)
				{
					if (bIsUsingSlider)
					{
						if (const TSharedPtr<IPropertyHandle> FloatPropertyHandle = WeakHandlePtr.Pin())
						{
							FloatPropertyHandle->SetValue(Value, EPropertyValueSetFlags::InteractiveChange | EPropertyValueSetFlags::NotTransactable);
						}
					}
				})
			.OnBeginSliderMovement_Lambda([this]()
				{
					bIsUsingSlider = true;
					GEditor->BeginTransaction(LOCTEXT("SetVectorProperty", "Set Vector Property"));
				})
			.OnEndSliderMovement_Lambda([this](float Value)
				{
					bIsUsingSlider = false;
					GEditor->EndTransaction();
				})
			.LabelVAlign(VAlign_Center)
			// Only allow spin on handles with one object.  Otherwise it is not clear what value to spin
			.AllowSpin(PropertyHandle->GetNumOuterObjects() < 2)
			.ShiftMultiplier(ShiftMultiplier)
			.CtrlMultiplier(CtrlMultiplier)
			.SupportDynamicSliderMaxValue(SupportDynamicSliderMaxValue)
			.SupportDynamicSliderMinValue(SupportDynamicSliderMinValue)
			.OnDynamicSliderMaxValueChanged(this, &FWeightedValueCustomization::OnDynamicSliderMaxValueChanged)
			.OnDynamicSliderMinValueChanged(this, &FWeightedValueCustomization::OnDynamicSliderMinValueChanged)
			.MinValue(MinValue)
			.MaxValue(MaxValue)
			.MinSliderValue(SliderMinValue)
			.MaxSliderValue(SliderMaxValue)
			.SliderExponent(SliderExponent)
			.Delta(Delta)
			.Label()
			[
				SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(FText::FromString(PropertyHandle->GetMetaData(TEXT("ChaosClothAssetShortName"))))  // Case specific metadata, uses ChaosCloth prefix to avoid conflict with future engine extensions
			];
	}

	// The following code is just a plain copy of FMathStructCustomization which
	// would need changes to be able to serve as a base class for this customization.
	void FWeightedValueCustomization::ExtractFloatMetadata(TSharedRef<IPropertyHandle>& PropertyHandle, TOptional<float>& MinValue, TOptional<float>& MaxValue, TOptional<float>& SliderMinValue, TOptional<float>& SliderMaxValue, float& SliderExponent, float& Delta, float& ShiftMultiplier, float& CtrlMultiplier, bool& SupportDynamicSliderMaxValue, bool& SupportDynamicSliderMinValue)
	{
		FProperty* Property = PropertyHandle->GetProperty();

		const FString& MetaUIMinString = Property->GetMetaData(TEXT("UIMin"));
		const FString& MetaUIMaxString = Property->GetMetaData(TEXT("UIMax"));
		const FString& SliderExponentString = Property->GetMetaData(TEXT("SliderExponent"));
		const FString& DeltaString = Property->GetMetaData(TEXT("Delta"));
		const FString& ShiftMultiplierString = Property->GetMetaData(TEXT("ShiftMultiplier"));
		const FString& CtrlMultiplierString = Property->GetMetaData(TEXT("CtrlMultiplier"));
		const FString& SupportDynamicSliderMaxValueString = Property->GetMetaData(TEXT("SupportDynamicSliderMaxValue"));
		const FString& SupportDynamicSliderMinValueString = Property->GetMetaData(TEXT("SupportDynamicSliderMinValue"));
		const FString& ClampMinString = Property->GetMetaData(TEXT("ClampMin"));
		const FString& ClampMaxString = Property->GetMetaData(TEXT("ClampMax"));

		// If no UIMin/Max was specified then use the clamp string
		const FString& UIMinString = MetaUIMinString.Len() ? MetaUIMinString : ClampMinString;
		const FString& UIMaxString = MetaUIMaxString.Len() ? MetaUIMaxString : ClampMaxString;

		float ClampMin = TNumericLimits<float>::Lowest();
		float ClampMax = TNumericLimits<float>::Max();

		if (!ClampMinString.IsEmpty())
		{
			TTypeFromString<float>::FromString(ClampMin, *ClampMinString);
		}

		if (!ClampMaxString.IsEmpty())
		{
			TTypeFromString<float>::FromString(ClampMax, *ClampMaxString);
		}

		float UIMin = TNumericLimits<float>::Lowest();
		float UIMax = TNumericLimits<float>::Max();
		TTypeFromString<float>::FromString(UIMin, *UIMinString);
		TTypeFromString<float>::FromString(UIMax, *UIMaxString);

		SliderExponent = float(1);

		if (SliderExponentString.Len())
		{
			TTypeFromString<float>::FromString(SliderExponent, *SliderExponentString);
		}

		Delta = float(0);

		if (DeltaString.Len())
		{
			TTypeFromString<float>::FromString(Delta, *DeltaString);
		}

		ShiftMultiplier = 10.f;
		if (ShiftMultiplierString.Len())
		{
			TTypeFromString<float>::FromString(ShiftMultiplier, *ShiftMultiplierString);
		}

		CtrlMultiplier = 0.1f;
		if (CtrlMultiplierString.Len())
		{
			TTypeFromString<float>::FromString(CtrlMultiplier, *CtrlMultiplierString);
		}

		const float ActualUIMin = FMath::Max(UIMin, ClampMin);
		const float ActualUIMax = FMath::Min(UIMax, ClampMax);

		MinValue = ClampMinString.Len() ? ClampMin : TOptional<float>();
		MaxValue = ClampMaxString.Len() ? ClampMax : TOptional<float>();
		SliderMinValue = (UIMinString.Len()) ? ActualUIMin : TOptional<float>();
		SliderMaxValue = (UIMaxString.Len()) ? ActualUIMax : TOptional<float>();

		SupportDynamicSliderMaxValue = SupportDynamicSliderMaxValueString.Len() > 0 && SupportDynamicSliderMaxValueString.ToBool();
		SupportDynamicSliderMinValue = SupportDynamicSliderMinValueString.Len() > 0 && SupportDynamicSliderMinValueString.ToBool();
	}

	void FWeightedValueCustomization::OnDynamicSliderMaxValueChanged(float NewMaxSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfHigher)
	{
		for (TWeakPtr<SWidget>& Widget : NumericEntryBoxWidgetList)
		{
			TSharedPtr<SNumericEntryBox<float>> NumericBox = StaticCastSharedPtr<SNumericEntryBox<float>>(Widget.Pin());

			if (NumericBox.IsValid())
			{
				TSharedPtr<SSpinBox<float>> SpinBox = StaticCastSharedPtr<SSpinBox<float>>(NumericBox->GetSpinBox());

				if (SpinBox.IsValid())
				{
					if (SpinBox != InValueChangedSourceWidget)
					{
						if ((NewMaxSliderValue > SpinBox->GetMaxSliderValue() && UpdateOnlyIfHigher) || !UpdateOnlyIfHigher)
						{
							// Make sure the max slider value is not a getter otherwise we will break the link!
							verifySlow(!SpinBox->IsMaxSliderValueBound());
							SpinBox->SetMaxSliderValue(NewMaxSliderValue);
						}
					}
				}
			}
		}

		if (IsOriginator)
		{
			OnNumericEntryBoxDynamicSliderMaxValueChanged.Broadcast((float)NewMaxSliderValue, InValueChangedSourceWidget, false, UpdateOnlyIfHigher);
		}
	}

	void FWeightedValueCustomization::OnDynamicSliderMinValueChanged(float NewMinSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfLower)
	{
		for (TWeakPtr<SWidget>& Widget : NumericEntryBoxWidgetList)
		{
			TSharedPtr<SNumericEntryBox<float>> NumericBox = StaticCastSharedPtr<SNumericEntryBox<float>>(Widget.Pin());

			if (NumericBox.IsValid())
			{
				TSharedPtr<SSpinBox<float>> SpinBox = StaticCastSharedPtr<SSpinBox<float>>(NumericBox->GetSpinBox());

				if (SpinBox.IsValid())
				{
					if (SpinBox != InValueChangedSourceWidget)
					{
						if ((NewMinSliderValue < SpinBox->GetMinSliderValue() && UpdateOnlyIfLower) || !UpdateOnlyIfLower)
						{
							// Make sure the min slider value is not a getter otherwise we will break the link!
							verifySlow(!SpinBox->IsMinSliderValueBound());
							SpinBox->SetMinSliderValue(NewMinSliderValue);
						}
					}
				}
			}
		}

		if (IsOriginator)
		{
			OnNumericEntryBoxDynamicSliderMinValueChanged.Broadcast((float)NewMinSliderValue, InValueChangedSourceWidget, false, UpdateOnlyIfLower);
		}
	}
}  // End namespace UE::Chaos::ClothAsset

#undef LOCTEXT_NAMESPACE
