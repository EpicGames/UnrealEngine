// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosClothAsset/ConnectableValueCustomization.h"

template <typename OptionalType> struct TOptional;

namespace UE::Chaos::ClothAsset
{
	/**
	 * Customization for all property that also uses weight maps.
	 * Works like a FMathStructCustomization with the difference that
	 * the structure member short names are displayed in the header's row.
	 */
	class FWeightedValueCustomization
		: public FConnectableValueCustomization
	{
	public:
		static TSharedRef<IPropertyTypeCustomization> MakeInstance();

		FWeightedValueCustomization();
		virtual ~FWeightedValueCustomization() override;

	protected:
		//~ Begin IPropertyTypeCustomization implementation
		virtual void CustomizeChildren(
			TSharedRef<IPropertyHandle> PropertyHandle,
			IDetailChildrenBuilder& ChildBuilder,
			IPropertyTypeCustomizationUtils& CustomizationUtils)
		{
			// Skip the FConnectableValueCustomization implementation that doesn't display children
			FImportedValueCustomization::CustomizeChildren(PropertyHandle, ChildBuilder, CustomizationUtils);
		}
		//~ End IPropertyTypeCustomization implementation

		//~ Begin FMathStructCustomization implementation
		virtual void MakeHeaderRow(TSharedRef<class IPropertyHandle>& StructPropertyHandle, FDetailWidgetRow& Row) override;
		virtual TSharedRef<SWidget> MakeChildWidget(TSharedRef<IPropertyHandle>& StructurePropertyHandle, TSharedRef<IPropertyHandle>& PropertyHandle) override;
		//~ End FMathStructCustomization implementation

	private:
		static void ExtractFloatMetadata(
			TSharedRef<IPropertyHandle>& PropertyHandle,
			TOptional<float>& MinValue,
			TOptional<float>& MaxValue,
			TOptional<float>& SliderMinValue,
			TOptional<float>& SliderMaxValue,
			float& SliderExponent,
			float& Delta,
			float& ShiftMultiplier,
			float& CtrlMultiplier,
			bool& SupportDynamicSliderMaxValue,
			bool& SupportDynamicSliderMinValue);
		TSharedRef<SWidget> MakeFloatWidget(TSharedRef<IPropertyHandle>& StructurePropertyHandle, TSharedRef<IPropertyHandle>& PropertyHandle);
		void OnDynamicSliderMaxValueChanged(float NewMaxSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfHigher);
		void OnDynamicSliderMinValueChanged(float NewMinSliderValue, TWeakPtr<SWidget> InValueChangedSourceWidget, bool IsOriginator, bool UpdateOnlyIfLower);
	};
}
