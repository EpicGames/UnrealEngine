// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "AddStitchNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetAddStitchNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetAddStitchNode, "AddStitch", "Cloth", "Cloth Simulation Add Stitch")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Set of vertices to stitch together. Can be 2D or 3D vertices. A seam will be created by making a chain of stitches (all vertices will merge to a single 3D vertex).*/
	UPROPERTY(EditAnywhere, Category = "Add Stitch")
	FChaosClothAssetConnectableIStringValue MergeToSingleVertexSelection;

	FChaosClothAssetAddStitchNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
