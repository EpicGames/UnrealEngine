// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowTerminalNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "AddWeightMapNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

UENUM()
enum class EChaosClothAssetWeightMapTransferType : uint8
{
	/** Transfer weight maps from the 2D simulation mesh (pattern against pattern). */
	Use2DSimMesh UMETA(DisplayName = "Use 2D Sim Mesh"),
	/** Transfer weight maps from the 3D simulation mesh (rest mesh against rest mesh). */
	Use3DSimMesh UMETA(DisplayName = "Use 3D Sim Mesh"),
	/** Transfer weight maps from the 3D render mesh into the sim mesh using spatial (BVH) mapping. Use this when the source collection contains an imported render mesh with painted weight maps. */
	UseRenderMesh UMETA(DisplayName = "Use Render Mesh"),
};

/** Which mesh to update with the corresponding weight map */
UENUM()
enum class EChaosClothAssetWeightMapMeshType : uint8
{
	Simulation,
	Render,
	Both
};

/** How the map stored on the AddWeightMapNode should be applied to an existing map. If no map exists, it is treated as zero.*/
UENUM()
enum class EChaosClothAssetWeightMapOverrideType : uint8
{
	/** The full map is stored and reapplied.*/
	ReplaceAll,
	/** Only changed values are stored and reapplied. */
	ReplaceChanged,
	/** Add values. */
	Add
};


/** 
* Painted weight map attributes node.
* Deprecated, use FChaosClothAssetWeightMapNode instead.
*/
PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT(Meta = (DataflowCloth, Deprecated="5.5"))
struct UE_DEPRECATED(5.5, "Use FChaosClothAssetWeightMapNode instead.") FChaosClothAssetAddWeightMapNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetAddWeightMapNode, "AddWeightMap", "Cloth", "Cloth Add Weight Map")

public:
	static constexpr float ReplaceChangedPassthroughValue = UE_BIG_NUMBER;

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/**
	 * The collection used to transfer weight map from.
	 * Connecting a collection containing a weight map with Input Name (or Name if Input Name is empty) 
	 * will transfer the weights to the input collection vertices.
	 * Note this operation only happens once when the TransferCollection is first connected, or updated.
	 * Changing the InputName or the TransferType will also redo the transfer operation.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Add Weight Map", Meta = (DataflowInput))
	FManagedArrayCollection TransferCollection;

	/** The name to be set as a weight map attribute. */
	UPROPERTY(VisibleAnywhere, Category = "Add Weight Map", Meta = (DataflowOutput))
	FString Name;

	/** The name to populate this map from and override based on Map Override Type. Name will be used if Input Name is empty.*/
	UPROPERTY(VisibleAnywhere, Category = "Add Weight Map")
	FChaosClothAssetConnectableIStringValue InputName = {TEXT("")};

	/** How to apply this node's weight values onto existing maps. Changing this value will change the output map. 
	 *  To change how the node's stored weights are calculated, change the equivalent value on the Weight Map Paint Tool context.*/
	UPROPERTY(VisibleAnywhere, Category = "Add Weight Map")
	EChaosClothAssetWeightMapOverrideType MapOverrideType = EChaosClothAssetWeightMapOverrideType::ReplaceAll;

	/**
	 * The type of transfer used to transfer the weight map when a TransferCollection is connected.
	 * This property is disabled when no TransferCollection input has been conencted.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Add Weight Map", Meta = (EditCondition = "TransferCollectionHash != 0"))
	EChaosClothAssetWeightMapTransferType TransferType = EChaosClothAssetWeightMapTransferType::Use2DSimMesh;

	UE_DEPRECATED(5.4, "This property will be made private.")
	UPROPERTY()
	TArray<float> VertexWeights;

	UPROPERTY(VisibleAnywhere, Category = "Add Weight Map")
	EChaosClothAssetWeightMapMeshType MeshTarget = EChaosClothAssetWeightMapMeshType::Simulation;

	FChaosClothAssetAddWeightMapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	friend class UClothEditorWeightMapPaintTool;

	class FWeightMapNodeChange;
	static TUniquePtr<class FToolCommandChange> CHAOSCLOTHASSETDATAFLOWNODES_API MakeWeightMapNodeChange(const FChaosClothAssetAddWeightMapNode& Node);

	const TArray<float>& GetVertexWeights() const { return VertexWeights; }
	TArray<float>& GetVertexWeights() { return VertexWeights; }

	const TArray<float>& GetRenderVertexWeights() const { return RenderVertexWeights; }
	TArray<float>& GetRenderVertexWeights() { return RenderVertexWeights; }

	UPROPERTY()
	TArray<float> RenderVertexWeights;

	// These methods are exported for UClothEditorWeightMapPaintTool which lives in a different module.
	FName CHAOSCLOTHASSETDATAFLOWNODES_API GetInputName(UE::Dataflow::FContext& Context) const;

	void CHAOSCLOTHASSETDATAFLOWNODES_API SetVertexWeights(const TConstArrayView<float> InputMap, const TArray<float>& FinalValues);
	void CHAOSCLOTHASSETDATAFLOWNODES_API SetRenderVertexWeights(const TConstArrayView<float> InputMap, const TArray<float>& FinalValues);

	// Input and FinalOutputMap can be the same array, but should not otherwise be interleaved.
	void CHAOSCLOTHASSETDATAFLOWNODES_API CalculateFinalVertexWeightValues(const TConstArrayView<float> InputMap, TArrayView<float> FinalOutputMap) const;
	void CHAOSCLOTHASSETDATAFLOWNODES_API CalculateFinalRenderVertexWeightValues(const TConstArrayView<float> InputMap, TArrayView<float> FinalOutputMap) const;

	//~ Begin FDataflowNode interface
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface

	UPROPERTY()
	uint32 TransferCollectionHash = 0;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
