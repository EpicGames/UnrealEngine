// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ApplyProxyDeformerNode.generated.h"

/** Update the Render Mesh by applying any existing proxy deformer data. 
  * This node can be used to deform the render mesh after the sim mesh has changed after the proxy deformer data was calculated.
  * It has no effect if there is no existing proxy deformer data, or if the sim or render mesh have not deformed since that data was calculated.*/
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetApplyProxyDeformerNode final : public FDataflowNode 
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetApplyProxyDeformerNode, "ApplyProxyDeformer", "Cloth", "Cloth Apply Proxy Deformer")

public:
	FChaosClothAssetApplyProxyDeformerNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	/** Input/output collection */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Ignore Skinning Blend Weights (apply proxy deformer to all points)*/
	UPROPERTY(EditAnywhere, Category = "Apply Proxy Deformer")
	bool bIgnoreSkinningBlendWeights = false;
};
