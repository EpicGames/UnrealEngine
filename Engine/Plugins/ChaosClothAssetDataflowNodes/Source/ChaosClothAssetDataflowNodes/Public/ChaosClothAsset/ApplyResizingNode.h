// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "MeshResizing/RBFInterpolation.h"
#include "ApplyResizingNode.generated.h"

class USkeletalMesh;

/** Apply resizing for a given Target Mesh.*/
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetApplyResizingNode final : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetApplyResizingNode, "ApplyResizing", "Cloth", "Apply Cloth Outfit Resizing")

public:

	FChaosClothAssetApplyResizingNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const override;

	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;

	/** The target mesh that corresponds with the SourceMesh used to generate the InterpolationData. Must have matching vertices with SourceMesh */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<const USkeletalMesh> TargetSkeletalMesh;

	/** Source and Target mesh LOD.*/
	UPROPERTY(EditAnywhere, Category = "Resizing", meta = (DataflowInput, ClampMin = 0))
	int32 SkeletalMeshLODIndex = 0;

	/** The pre-calculated base RBF interpolation data.*/
	UPROPERTY(meta = (DataflowInput))
	FMeshResizingRBFInterpolationData InterpolationData;

	/** Force apply to the render mesh. Otherwise, the sim mesh will be resized if it exists, only resizing the render mesh if no sim mesh data exists. */
	UPROPERTY(EditAnywhere, Category = "Resizing", meta = (DataflowInput))
	bool bForceApplyToRenderMesh = false;

	/** The source mesh. Used for CustomResizingRegions. Must have matching vertices with TargetMesh */
	UPROPERTY(meta = (DataflowInput))
	TObjectPtr<const USkeletalMesh> SourceSkeletalMesh;

	/** Skip applying Custom Region Resizing data. */
	UPROPERTY(EditAnywhere, Category = "Resizing", meta = (DataflowInput))
	bool bSkipCustomRegionResizing = false;

	/** Save pre-resized sim 3d vertices for scaling 2D rest length in XPBD anisotropic stretch constraints.
	* IMPORTANT: Using Save Pre-Resized Sim Position 3D requires the following settings in SimulationStretchConfig node:
	* Stretch Use 3d Rest Lengths: false
	* Solver Type: XPBD
	* Distribution Type: Anisotropic */
	UPROPERTY(EditAnywhere, Category = "Resizing", meta = (DataflowInput, DisplayName = "Save Pre-Resized Sim Position 3D"))
	bool bSavePreResizedSimPosition3D = false;
};