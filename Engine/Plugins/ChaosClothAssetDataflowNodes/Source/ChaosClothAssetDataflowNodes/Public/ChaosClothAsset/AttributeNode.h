// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "AttributeNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/**
 * The managed array collection group used for the attribute creation.
 * This separate structure is required to allow for customization of the UI.
 */
USTRUCT()
struct FChaosClothAssetNodeAttributeGroup
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Attribute Group")
	FString Name;
};

UENUM()
enum class EChaosClothAssetNodeAttributeType : uint8
{
	Integer,
	Float,
	Vector
};

/** Create a new attribute for the specified group. */
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetAttributeNode_v2 final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetAttributeNode_v2, "Attribute", "Cloth", "Cloth Attribute")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The name of the attribute to create. */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	FChaosClothAssetConnectableIOStringValue Name;

	/** The attribute group. */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	FChaosClothAssetNodeAttributeGroup Group;

	/** The attribute type. */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	EChaosClothAssetNodeAttributeType Type = EChaosClothAssetNodeAttributeType::Integer;

	/** Default integer value. */
	UPROPERTY(EditAnywhere, Category = "Attribute", Meta = (EditCondition = "Type == EChaosClothAssetNodeAttributeType::Integer", EditConditionHides))
	int32 IntValue = 0;

	/** Default float value. */
	UPROPERTY(EditAnywhere, Category = "Attribute", Meta = (EditCondition = "Type == EChaosClothAssetNodeAttributeType::Float", EditConditionHides))
	float FloatValue = 0.f;

	/** Default vector value. */
	UPROPERTY(EditAnywhere, Category = "Attribute", Meta = (EditCondition = "Type == EChaosClothAssetNodeAttributeType::Vector", EditConditionHides))
	FVector3f VectorValue = FVector3f::ZeroVector;

	FChaosClothAssetAttributeNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};

/** Create a new attribute for the specified group. */
USTRUCT(Meta = (DataflowCloth, Experimental, Deprecated = "5.5"))
struct UE_DEPRECATED(5.5, "Use the newer version of this node instead.") FChaosClothAssetAttributeNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetAttributeNode, "Attribute", "Cloth", "Cloth Attribute")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The name of the attribute to create. */
	UPROPERTY(EditAnywhere, Category = "Attribute", Meta = (DataflowOutput))
	FString Name;

	/** The attribute group. */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	FChaosClothAssetNodeAttributeGroup Group;

	/** The attribute type. */
	UPROPERTY(EditAnywhere, Category = "Attribute")
	EChaosClothAssetNodeAttributeType Type = EChaosClothAssetNodeAttributeType::Integer;

	/** Default integer value. */
	UPROPERTY(EditAnywhere, Category = "Attribute", Meta = (EditCondition = "Type == EChaosClothAssetNodeAttributeType::Integer", EditConditionHides))
	int32 IntValue = 0;

	/** Default float value. */
	UPROPERTY(EditAnywhere, Category = "Attribute", Meta = (EditCondition = "Type == EChaosClothAssetNodeAttributeType::Float", EditConditionHides))
	float FloatValue = 0.f;

	/** Default vector value. */
	UPROPERTY(EditAnywhere, Category = "Attribute", Meta = (EditCondition = "Type == EChaosClothAssetNodeAttributeType::Vector", EditConditionHides))
	FVector3f VectorValue = FVector3f::ZeroVector;

	FChaosClothAssetAttributeNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	/** Return a cached array of all the groups used by the input collection during at the time of the latest evaluation. */
	UE_DEPRECATED(5.5, "This function is deprecated and will now return an empty array.")
	const TArray<FName>& GetCachedCollectionGroupNames() const { return CachedCollectionGroupNames; }

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	UE_DEPRECATED(5.5, "This function is deprecated and will not be called on selection/deselection.")
	virtual void OnSelected(UE::Dataflow::FContext& Context) {}
	UE_DEPRECATED(5.5, "This function is deprecated and will not be called on selection/deselection.")
	virtual void OnDeselected() {}
	//~ End FDataflowNode interface

	TArray<FName> CachedCollectionGroupNames;
};
