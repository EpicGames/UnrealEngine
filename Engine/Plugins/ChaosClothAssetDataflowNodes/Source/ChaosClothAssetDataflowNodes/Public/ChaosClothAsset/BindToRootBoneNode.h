// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "BindToRootBoneNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** Bind an entire mesh to the single root bone of the current skeleton set on the cloth collection. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetBindToRootBoneNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetBindToRootBoneNode, "BindToRootBone", "Cloth", "Cloth Bind Skinning Weights To Root Bone")

public:
	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Whether to bind the simulation mesh. */
	UPROPERTY(EditAnywhere, Category = "Bind To Root Bone")
	bool bBindSimMesh = true;

	/** Whether to bind the render mesh. */
	UPROPERTY(EditAnywhere, Category = "Bind To Root Bone")
	bool bBindRenderMesh = true;

	FChaosClothAssetBindToRootBoneNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
