// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "BlendVerticesNode.generated.h"

/** Blend vertex values from another cloth collection. The topology of the Collection will remain the same.*/
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetBlendVerticesNode final : public FDataflowNode 
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetBlendVerticesNode, "BlendVertices", "Cloth", "Cloth Simulation Blend Vertices")

public:

	/** Input/output collection */
	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Collection to blend in. */
	UPROPERTY(Meta = (Dataflowinput))
	FManagedArrayCollection BlendCollection;

	/** Require same vertex counts between Collection and BlendCollection in order to blend a vertex type. Otherwise the shared subset will be blended. */
	UPROPERTY(EditAnywhere, Category = "Blend Vertices")
	bool bRequireSameVertexCounts = true;

	/** Blending Weight. 0 = Keep existing values in Collection. 1 = Set values from BlendCollection. */
	UPROPERTY(EditAnywhere, Category = "Blend Vertices", Meta = (ClampMin = 0, ClampMax = 1))
	float BlendingWeight = 1.f;

	/** Blend Sim Mesh.*/
	UPROPERTY(EditAnywhere, Category = "Blend Vertices - Sim Mesh", Meta = (EditCondition = "BlendingWeight > 0"))
	bool bBlendSimMesh = true;

	/** Blend 2D Sim Positions.*/
	UPROPERTY(EditAnywhere, Category = "Blend Vertices - Sim Mesh", Meta = (EditCondition = "bBlendSimMesh && BlendingWeight > 0"))
	bool bBlend2DSimPositions = true;

	/** Blend 3D Sim Positions.*/
	UPROPERTY(EditAnywhere, Category = "Blend Vertices - Sim Mesh", Meta = (EditCondition = "bBlendSimMesh && BlendingWeight > 0"))
	bool bBlend3DSimPositions = true;

	/** Blend 3D Sim Normals.*/
	UPROPERTY(EditAnywhere, Category = "Blend Vertices - Sim Mesh", Meta = (EditCondition = "bBlendSimMesh && BlendingWeight > 0"))
	bool bBlendSimNormals = true;

	/** Blend Render Mesh.*/
	UPROPERTY(EditAnywhere, Category = "Blend Vertices - Render Mesh", Meta = (EditCondition = "BlendingWeight > 0.f"))
	bool bBlendRenderMesh = true;

	/** Blend Render Positions.*/
	UPROPERTY(EditAnywhere, Category = "Blend Vertices - Render Mesh", Meta = (EditCondition = "bBlendRenderMesh && BlendingWeight > 0"))
	bool bBlendRenderPositions = true;

	/** Blend Render Normals and Tangents.*/
	UPROPERTY(EditAnywhere, Category = "Blend Vertices - Render Mesh", Meta = (EditCondition = "bBlendRenderMesh && BlendingWeight > 0"))
	bool bBlendRenderNormalsAndTangents = true;

	/** Blend Render UVs. Only existing UV sets on Collection will be updated.*/
	UPROPERTY(EditAnywhere, Category = "Blend Vertices - Render Mesh", Meta = (EditCondition = "bBlendRenderMesh && BlendingWeight > 0"))
	bool bBlendRenderUVs = true;

	/** Blend Render Colors.*/
	UPROPERTY(EditAnywhere, Category = "Blend Vertices - Render Mesh", Meta = (EditCondition = "bBlendRenderMesh && BlendingWeight > 0"))
	bool bBlendRenderColors = true;

	FChaosClothAssetBlendVerticesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
