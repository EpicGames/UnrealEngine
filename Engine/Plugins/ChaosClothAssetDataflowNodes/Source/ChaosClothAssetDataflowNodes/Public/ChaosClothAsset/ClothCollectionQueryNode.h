// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ClothCollectionQueryNode.generated.h"


class UDynamicMesh;
class UMaterialInterface;

/** Query a Managed Array Collection about its Cloth Collection properties.
 */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetCollectionQueryNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetCollectionQueryNode, "ClothCollectionQuery", "Cloth", "Cloth Collection Query")

public:

	FChaosClothAssetCollectionQueryNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:

	/** Input/output collection (Output is always a passthrough)*/
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Is this collection a valid cloth collection */
	UPROPERTY(Meta = (DataflowOutput))
	bool bIsClothCollection = false;

	/** Does this collection have a cloth sim mesh */
	UPROPERTY(Meta = (DataflowOutput))
	bool bHasClothSimMesh = false;

	/** Does this collection have a cloth render mesh */
	UPROPERTY(Meta = (DataflowOutput))
	bool bHasClothRenderMesh = false;

	/** Does this collection have proxy deformer data */
	UPROPERTY(Meta = (DataflowOutput))
	bool bHasClothProxyDeformer = false;

	/** Name of boolean property to query */
	UPROPERTY(EditAnywhere, Category = "Cloth Collection Query")
	FString BooleanPropertyName = TEXT("PropertyName");

	/** Result of querying BooleanPropertyName. Default value if property doesn't exist matches node value.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Collection Query", Meta =(DataflowOutput))
	bool bBooleanPropertyValue = false;
};
