// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "ChaosClothAsset/WeightMapNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ClothCollectionToDynamicMeshNode.generated.h"


class UDynamicMesh;
class UMaterialInterface;

/** Convert a Cloth Collection mesh to a dynamic mesh.
 */
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetCollectionToDynamicMeshNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetCollectionToDynamicMeshNode, "ClothCollectionToDynamicMesh", "Cloth", "Cloth Collection Dynamic Mesh")
	DATAFLOW_NODE_RENDER_TYPE_START()
	DATAFLOW_NODE_RENDER_TYPE_ADD("SurfaceRender", FName("FDynamicMesh3"), {"SimDynamicMesh"})
	DATAFLOW_NODE_RENDER_TYPE_ADD("SurfaceRender", FName("FDynamicMesh3"), {"RenderDynamicMesh", "RenderMaterials"})
	DATAFLOW_NODE_RENDER_TYPE_END()

public:

	FChaosClothAssetCollectionToDynamicMeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:

	UPROPERTY(Meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** Output sim collection dynamic mesh. */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> SimDynamicMesh;

	/** Output render collection dynamic mesh. */
	UPROPERTY(meta = (DataflowOutput))
	TObjectPtr<UDynamicMesh> RenderDynamicMesh;

	/** Render materials.*/
	UPROPERTY(meta = (DataflowOutput))
	TArray<TObjectPtr<UMaterialInterface>> RenderMaterials;

};


/** Update cloth collection attributes from a DynamicMesh
 */
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetUpdateClothFromDynamicMeshNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetUpdateClothFromDynamicMeshNode, "UpdateClothFromDynamicMesh", "Cloth", "Update Cloth Collection Dynamic Mesh Cloth")

public:

	FChaosClothAssetUpdateClothFromDynamicMeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(Meta = (DataflowInput))
	TObjectPtr<const UDynamicMesh> DynamicMesh;

	UPROPERTY(Meta = (DataflowInput))
	TArray<TObjectPtr<const UMaterialInterface>> Materials;

	/** Copy DynamicMesh Vertex Positions to Render Positions */
	UPROPERTY(EditAnywhere, Category = "DynamicMesh To Cloth Collection")
	bool bCopyToRenderPositions = false;

	/** Copy DynamicMesh Vertex Normals and Tangents to Render Normals and Tangents */
	UPROPERTY(EditAnywhere, Category = "DynamicMesh To Cloth Collection")
	bool bCopyToRendeNormalsAndTangents = false;

	/** Copy DynamicMesh UVs to Render UVs */
	UPROPERTY(EditAnywhere, Category = "DynamicMesh To Cloth Collection")
	bool bCopyUVsToRenderUVs = false;

	/** Copy input materials to Render Materials (order and number must match otherwise only the minimum common number of materials are updated) */
	UPROPERTY(EditAnywhere, Category = "DynamicMesh To Cloth Collection")
	bool bCopyToRenderMaterials = false;

	/** Copy DynamicMesh Vertex Positions to Sim3D Positions */
	UPROPERTY(EditAnywhere, Category = "DynamicMesh To Cloth Collection")
	bool bCopyToSim3DPositions = false;

	/** Copy DynamicMesh Vertex Normals to Sim Normals */
	UPROPERTY(EditAnywhere, Category = "DynamicMesh To Cloth Collection")
	bool bCopyToSimNormals = false;

	/** Copy DynamicMesh UVs to Sim2D Positions */
	UPROPERTY(EditAnywhere, Category = "DynamicMesh To Cloth Collection")
	bool bCopyUVsToSim2DPositions = false;

	/** Which UV Channel to use at Sim2D Positions or Render UVs. Use -1 to copy all Render UVs. */
	UPROPERTY(EditAnywhere, Category = "DynamicMesh To Cloth Collection", meta = (EditCondition = "bCopyUVsToSim2DPositions || bCopyUVsToRenderUVs", EditConditionHides))
	int32 UVChannelIndex = 0;
};

/** Extract a weight map from a Cloth Collection.
 */
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetExtractWeightMapNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetExtractWeightMapNode, "ExtractClothWeightMap", "Cloth", "Extract Cloth Weight Map")

public:

	FChaosClothAssetExtractWeightMapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:

	UPROPERTY(Meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** Name of the weight map to be extracted */
	UPROPERTY(EditAnywhere, Category = "Extract Weight Map", Meta = (DataflowInput))
	FChaosClothAssetConnectableIStringValue WeightMap = { TEXT("WeightMap") };

	/** Type of the weight map to be extracted */
	UPROPERTY(EditAnywhere, Category = "Extract Weight Map")
	EChaosClothAssetWeightMapMeshTarget MeshTarget = EChaosClothAssetWeightMapMeshTarget::Simulation;

	/** Reorder extracted weights to match the order of a DynamicMesh that was created via ClothCollectionToDynamicMesh */
	UPROPERTY(EditAnywhere, Category = "Extract Weight Map")
	bool bReorderForDynamicMesh = true;

	/** Dynamic mesh used to reorder weights. */
	UPROPERTY(meta = (DataflowInput, EditCondition = "bReorderForDynamicMesh"))
	TObjectPtr<UDynamicMesh> DynamicMesh;

	UPROPERTY(meta = (DataflowOutput))
	TArray<float> ExtractedWeightMap;

};


/** Extract a selection set from a Cloth Collection.
 */
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetExtractSelectionSetNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetExtractSelectionSetNode, "ExtractClothSelectionSet", "Cloth", "Extract Cloth Selection Set")

public:

	FChaosClothAssetExtractSelectionSetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:

	UPROPERTY(Meta = (DataflowInput))
	FManagedArrayCollection Collection;

	/** Name of the selection set to be extracted. Currently only SimVertices3D and RenderVertices sets are supported. */
	UPROPERTY(EditAnywhere, Category = "Extract Selection Set", Meta = (DataflowInput))
	FChaosClothAssetConnectableIStringValue Selection = { TEXT("Selection") };

	/** Reorder extracted indices to match the order of a DynamicMesh that was created via ClothCollectionToDynamicMesh */
	UPROPERTY(EditAnywhere, Category = "Extract Selection Set")
	bool bReorderForDynamicMesh = true;

	/** Dynamic mesh used to reorder indices. */
	UPROPERTY(meta = (DataflowInput, EditCondition = "bReorderForDynamicMesh"))
	TObjectPtr<UDynamicMesh> DynamicMesh;

	/** Extracted Selection Set as a Set */
	UPROPERTY(meta = (DataflowOutput))
	TSet<int32> ExtractedSelectionSet;

	/** Extracted Selection Set as an array */
	UPROPERTY(meta = (DataflowOutput))
	TArray<int32> ExtractedSelectionArray;

};