// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"
#include "Containers/UnrealString.h"
#include "ConnectableValue.generated.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT()
struct FChaosClothAssetConnectableStringValue
{
	GENERATED_BODY()
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** The value for this property. */
	UPROPERTY(EditAnywhere, Category = "Value", Meta = (SkipInDisplayNameChain))
	FString StringValue;

	UE_DEPRECATED(5.5, "Override properties are no longer used.")
	UPROPERTY(Transient)
	mutable FString StringValue_Override;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT()
struct FChaosClothAssetConnectableIStringValue
{
	GENERATED_BODY()
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** The value for this property. */
	UPROPERTY(EditAnywhere, Category = "Value", Meta = (DataflowInput, SkipInDisplayNameChain))
	FString StringValue;

	/**
	  * Whether the property could import fabrics datas or not
	  */
	UPROPERTY(VisibleAnywhere, Transient, Category = "Value", Meta = (EditCondition = "false", EditConditionHides))
	bool bCouldUseFabrics = false;

	UE_DEPRECATED(5.5, "Override properties are no longer used.")
	UPROPERTY(Transient)
	mutable FString StringValue_Override;
	
	/**
	 * Whether the property can override the weight map based on the imported fabrics
	 */
	UPROPERTY(EditAnywhere, Category = "Value", Meta = (EditCondition = "bCouldUseFabrics", EditConditionHides))
	bool bBuildFabricMaps = false;
};

USTRUCT()
struct FChaosClothAssetConnectableOStringValue
{
	GENERATED_BODY()

	/** The value for this property. */
	UPROPERTY(EditAnywhere, Category = "Value", Meta = (DataflowOutput, SkipInDisplayNameChain))
	FString StringValue;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT()
struct FChaosClothAssetConnectableIOStringValue
{
	GENERATED_BODY()
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** The value for this property. */
	UPROPERTY(EditAnywhere, Category = "Value", Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "StringValue", SkipInDisplayNameChain))
	FString StringValue;

	UE_DEPRECATED(5.5, "Override properties are no longer used.")
	UPROPERTY(Transient)
	mutable FString StringValue_Override;
};
