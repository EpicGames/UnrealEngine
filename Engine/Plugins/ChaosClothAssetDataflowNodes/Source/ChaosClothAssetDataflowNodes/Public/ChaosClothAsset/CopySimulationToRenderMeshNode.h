// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "CopySimulationToRenderMeshNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class UMaterialInterface;

/** Copy the simulation mesh to the render mesh to be able to render the simulation mesh, or when not using a different mesh for rendering. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetCopySimulationToRenderMeshNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetCopySimulationToRenderMeshNode, "CopySimulationToRenderMesh", "Cloth", "Cloth Simulation Render Mesh")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** New material for the render mesh. */
	UPROPERTY(EditAnywhere, Category = "Copy Simulation To Render Mesh")
	TObjectPtr<const UMaterialInterface> Material;

	/** Generate a single render pattern rather than a render pattern per sim pattern. */
	UPROPERTY(EditAnywhere, Category = "Copy Simulation To Render Mesh")
	bool bGenerateSingleRenderPattern = true;

	FChaosClothAssetCopySimulationToRenderMeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
