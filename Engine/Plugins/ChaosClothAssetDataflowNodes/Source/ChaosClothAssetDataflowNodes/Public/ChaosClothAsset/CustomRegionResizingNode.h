// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "MeshResizing/RBFInterpolation.h"
#include "MeshResizing/CustomRegionResizing.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "CustomRegionResizingNode.generated.h"


/** Data to generate a Custom Resizing Region.*/
USTRUCT()
struct FChaosClothAssetCustomRegionResizingInput
{
	GENERATED_USTRUCT_BODY()

	/** Source Set. Must be convertible to a vertex set. Render sets will only be used if this ClothAsset does not have a sim mesh.*/
	UPROPERTY(EditAnywhere, Category = "Custom Region Resizing")
	FChaosClothAssetConnectableIStringValue InputSet = {TEXT("InputSet")};

	/** Custom Resizing type. */
	UPROPERTY(EditAnywhere, Category = "Custom Region Resizing")
	EMeshResizingCustomRegionType ResizingType = EMeshResizingCustomRegionType::TrilinearInterpolation;
};

/** Node for adding custom region resizing data used by the ChaosOutfitAsset's Resizeable Outfit.*/
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetCustomRegionResizingNode final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetCustomRegionResizingNode, "CustomRegionResizing", "Cloth", "Cloth Outfit Custom Region Resizing")

public:

	FChaosClothAssetCustomRegionResizingNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override 
	{ 
		return true; 
	}
	virtual bool CanRemovePin() const override 
	{ 
		return InputGroupData.Num() > NumInitialOptionalInputs;
	}
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface


	UE::Dataflow::TConnectionReference<FString> GetConnectionReference(int32 Index) const;

	static constexpr int32 NumRequiredInputs = 1; // non-construction set inputs
	static constexpr int32 NumInitialOptionalInputs = 1;

	struct FRegionData
	{
		FString InputSet;
		EMeshResizingCustomRegionType ResizingType;
	};
	TArray<FRegionData> GetRegionData(UE::Dataflow::FContext& Context) const;

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Input custom region data. This data will be stored in the resulting Cloth Asset, and available for use when resizing.*/
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Custom Region Resizing", Meta = (SkipInDisplayNameChain))
	TArray<FChaosClothAssetCustomRegionResizingInput> InputGroupData;

	/** The name of the sim mesh weight map generated by this node detailing the vertices should use Custom Regions as opposed to 
	* traditional resizing interpolation.
	* Value ranges between 0 (use traditional resizing interpolation) and 1 (use resizing custom group interpolation).
	* These regions can be blended if desired.
	* The name of this sim mesh weight map cannot be changed and is only provided for further tweaking.*/
	UPROPERTY(VisibleAnywhere, Category = "Custom Region Resizing", Meta = (DataflowOutput))
	FString SimCustomResizingBlendName;

	/** The name of the render mesh weight map generated by this node detailing the vertices should use Custom Regions as opposed to
	* traditional resizing interpolation.
	* Value ranges between 0 (use traditional resizing interpolation) and 1 (use resizing custom group interpolation).
	* These regions can be blended if desired.
	* The name of this render mesh weight map cannot be changed and is only provided for further tweaking.*/
	UPROPERTY(VisibleAnywhere, Category = "Custom Region Resizing", Meta = (DataflowOutput))
	FString RenderCustomResizingBlend;
};