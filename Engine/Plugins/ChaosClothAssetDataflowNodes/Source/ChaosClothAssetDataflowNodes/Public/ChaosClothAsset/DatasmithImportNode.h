// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "DatasmithImportNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** Deprecated. */
UCLASS()
class UChaosClothAssetDatasmithClothAssetFactory final : public UObject
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UChaosClothAssetDatasmithClothAssetFactory() = default;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual ~UChaosClothAssetDatasmithClothAssetFactory() = default;

	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UObject* CreateClothAsset(UObject* /*Outer*/, const FName& /*Name*/, EObjectFlags /*ObjectFlags*/) const { return nullptr; }
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UObject* DuplicateClothAsset(UObject* /*ClothAsset*/, UObject* Outer, const FName& /*Name*/) const { return nullptr; }
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	void InitializeClothAsset(UObject* /*ClothAsset*/, const class FDatasmithCloth& /*DatasmithCloth*/) const {}
};


/** Deprecated. */
UCLASS()
class UChaosClothAssetDatasmithClothComponentFactory final : public UObject
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	UChaosClothAssetDatasmithClothComponentFactory() = default;
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	virtual ~UChaosClothAssetDatasmithClothComponentFactory() = default;

	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	USceneComponent* CreateClothComponent(UObject* /*Outer*/) const { return nullptr; }
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	void InitializeClothComponent(class USceneComponent* /*ClothComponent*/, UObject* /*ClothAsset*/, class USceneComponent* /*RootComponent*/) const {}
};

/** Deprecated. The experimental Datasmith cloth importer is no longer supported. Use the USDImport node instead. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.4"))
struct FChaosClothAssetDatasmithImportNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetDatasmithImportNode, "DatasmithImport", "Cloth", "Cloth Datasmith Import")

public:
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/** Path of the file to import using any available Datasmith cloth translator. */
	UPROPERTY(VisibleAnywhere, Category = "Datasmith Import", Meta = (EditCondition = "false"))
	FFilePath ImportFile;

	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	static void RegisterModularFeature() {}
	UE_DEPRECATED(5.5, "The experimental Cloth importer is no longer supported.")
	static void UnregisterModularFeature() {}

	FChaosClothAssetDatasmithImportNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual void Serialize(FArchive& Archive) override;
	//~ End FDataflowNode interface

	FManagedArrayCollection ImportCache;
};
