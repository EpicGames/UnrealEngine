// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ChaosClothAsset/SelectionNode.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "DeleteElementNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

UENUM()
enum class UE_DEPRECATED(5.4, "Use FChaosClothAssetNodeSelectionGroup instead") EChaosClothAssetElementType : uint8
{
	None,
	SimMesh,
	RenderMesh,
	SimPattern,
	RenderPattern,
	SimVertex2D,
	SimVertex3D,
	RenderVertex,
	SimFace,
	RenderFace,
	Seam,
	/** Deprecated marker */
	Deprecated UMETA(Hidden)
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // For EChaosClothAssetElementType
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetDeleteElementNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetDeleteElementNode, "DeleteElement", "Cloth", "Cloth Simulation Delete Element")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Element type to delete.*/
	UE_DEPRECATED(5.4, "Use Group instead")
	UPROPERTY()
	EChaosClothAssetElementType ElementType_DEPRECATED = EChaosClothAssetElementType::Deprecated;

	/** Delete the sim mesh. */
	UPROPERTY(EditAnywhere, Category = "Delete Element")
	bool bDeleteSimMesh = false;

	/** Delete the render mesh. */
	UPROPERTY(EditAnywhere, Category = "Delete Element")
	bool bDeleteRenderMesh = false;

	/** Delete specific elements.*/
	UPROPERTY(EditAnywhere, Category = "Delete Element")
	FChaosClothAssetNodeSelectionGroup Group;

	/** List of Elements to delete from Group. All Elements will be used if left empty. */
	UPROPERTY(EditAnywhere, Category = "Delete Element")
	TArray<int32> Elements;

	/** Set of Elements to delete. This selection set will be deleted from the downstream collection since it will now be empty.*/
	UPROPERTY(EditAnywhere, Category = "Delete Element")
	FChaosClothAssetConnectableIStringValue SelectionName = {""};

	FChaosClothAssetDeleteElementNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	/** Return a cached array of all the groups used by the input collection during at the time of the latest evaluation. */
	UE_DEPRECATED(5.5, "This function is deprecated and will now return an empty array.")
	const TArray<FName>& GetCachedCollectionGroupNames() const { return CachedCollectionGroupNames; }

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	UE_DEPRECATED(5.5, "This function is deprecated and will not be called on selection/deselection.")
	virtual void OnSelected(UE::Dataflow::FContext& Context) {}
	UE_DEPRECATED(5.5, "This function is deprecated and will not be called on selection/deselection.")
	virtual void OnDeselected() {}
	virtual void Serialize(FArchive& Ar);

	TArray<FName> CachedCollectionGroupNames;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS  // For EChaosClothAssetElementType
