// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "EnableUVResizingNode.generated.h"

/** Node for enabling UV Resizing used by the ChaosOutfitAsset's Resizeable Outfit.*/
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetEnableUVResizingNode final : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetEnableUVResizingNode, "EnableUVResizing", "Cloth", "Cloth Enable UV Resizing")

public:

	FChaosClothAssetEnableUVResizingNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
