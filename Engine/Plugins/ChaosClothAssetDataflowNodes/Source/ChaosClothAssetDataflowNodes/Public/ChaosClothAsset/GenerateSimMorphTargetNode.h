// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GenerateSimMorphTargetNode.generated.h"

/** Generate a Sim Morph target from a cloth collection sim mesh (with matching topology).*/
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetGenerateSimMorphTargetNode final : public FDataflowNode 
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetGenerateSimMorphTargetNode, "GenerateSimMorphTarget", "Cloth", "Cloth Generate Sim Morph Target")

public:

	/** Input/output collection */
	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Collection to generate the morph target from. */
	UPROPERTY(Meta = (Dataflowinput))
	FManagedArrayCollection MorphTargetCollection;

	/** Morph target name */
	UPROPERTY(EditAnywhere, Category = "Sim Morph Target", meta = (DataflowOutput))
	FString MorphTargetName;

	/** Whether or not to generate normal deltas */
	UPROPERTY(EditAnywhere, Category = "Sim Morph Target")
	bool bGenerateNormalDeltas = false;

	FChaosClothAssetGenerateSimMorphTargetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
