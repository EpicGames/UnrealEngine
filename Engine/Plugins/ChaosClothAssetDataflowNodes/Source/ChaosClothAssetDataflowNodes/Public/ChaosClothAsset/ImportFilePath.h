// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "ImportFilePath.generated.h"

namespace UE::Dataflow
{
	class FContext;
}

USTRUCT()
struct FChaosClothAssetImportFilePath
{
	GENERATED_USTRUCT_BODY()

public:
	DECLARE_DELEGATE_OneParam(FDelegate, UE::Dataflow::FContext&);

	UPROPERTY(EditAnywhere, Category = "Import File Path")
	FString FilePath;

	FChaosClothAssetImportFilePath() = default;

	explicit FChaosClothAssetImportFilePath(FDelegate&& InDelegate) : Delegate(MoveTemp(InDelegate)) {}

	void Execute(UE::Dataflow::FContext& Context) const { Delegate.ExecuteIfBound(Context); }

private:
	FDelegate Delegate;
};
