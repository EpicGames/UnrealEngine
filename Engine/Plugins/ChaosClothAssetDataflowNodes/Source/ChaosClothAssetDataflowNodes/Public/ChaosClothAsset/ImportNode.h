// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowFunctionProperty.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ImportNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class UChaosClothAsset;

/** Refresh structure for push button customization. */
USTRUCT()
struct UE_DEPRECATED(5.5, "Use UE::Dataflow::FunctionProperty instead.") FChaosClothAssetImportNodeRefreshAsset
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Import Node Refresh Asset")
	bool bRefreshAsset = false;
};

/** Import an existing Cloth Asset into the graph. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // For Reimport
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetImportNode : public FDataflowNode
{
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetImportNode, "ClothAssetImport", "Cloth", "Cloth Asset Import")

public:
	UPROPERTY(Meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/** The Cloth Asset to import into a collection. */
	UPROPERTY(EditAnywhere, Category = "Cloth Asset Import", Meta = (DataflowInput))
	TObjectPtr<const UChaosClothAsset> ClothAsset;

	/** Reimport the imported cloth asset. */
	UPROPERTY(EditAnywhere, Category = "Cloth Asset Import", Meta = (ButtonImage = "Persona.ReimportAsset", DisplayName = "Reimport Asset"))
	FDataflowFunctionProperty Reimport;

	/** The LOD to import into the collection. Only one LOD can be imported at a time. */
	UPROPERTY(EditAnywhere, Category = "Cloth Asset Import", Meta = (DisplayName = "Import LOD", ClampMin = "0"))
	int32 ImportLod = 0;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.5, "Use FDataflowFunctionProperty Reimport instead.")
	UPROPERTY()
	mutable FChaosClothAssetImportNodeRefreshAsset ReimportAsset;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FChaosClothAssetImportNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
