// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ImportSimulationCacheNode.generated.h"

class UChaosCacheCollection;

/** Set vertex values from a simulation cache. The topology of the Collection will remain the same.*/
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetImportSimulationCacheNode final : public FDataflowNode 
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetImportSimulationCacheNode, "ImportSimulationCache", "Cloth", "Cloth Simulation Import Cache")

public:

	FChaosClothAssetImportSimulationCacheNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	/** Input/output collection */
	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Cache to import in. */
	UPROPERTY(EditAnywhere, Category = "Import Simulation Cache", Meta =(Dataflowinput))
	TObjectPtr<const UChaosCacheCollection> ImportedCache;

	/** Cache index to read */
	UPROPERTY(EditAnywhere, Category = "Import Simulation Cache")
	int32 CacheIndex = 0;

	/** Cache time to read */
	UPROPERTY(EditAnywhere, Category = "Import Simulation Cache")
	float CacheTime = 0.f;

	/** Transform cache data. */
	UPROPERTY(EditAnywhere, Category = "Import Simulation Cache")
	FTransform Transform;

	/** Particle cache offset. */
	UPROPERTY(EditAnywhere, Category = "Import Simulation Cache")
	int32 ParticleOffset = 0;

	/** Update simulation mesh from cache. */
	UPROPERTY(EditAnywhere, Category = "Import Simulation Cache")
	bool bUpdateSimulationMesh = true;

	/** Recalculate simulation normals based on imported positions. */
	UPROPERTY(EditAnywhere, Category = "Import Simulation Cache", Meta = (EditCondition = "bUpdateSimulationMesh"))
	bool bRecalculateNormals = true;

	/** Update render mesh from cache via proxy deformer data. */
	UPROPERTY(EditAnywhere, Category = "Import Simulation Cache")
	bool bUpdateRenderMesh = false;

};
