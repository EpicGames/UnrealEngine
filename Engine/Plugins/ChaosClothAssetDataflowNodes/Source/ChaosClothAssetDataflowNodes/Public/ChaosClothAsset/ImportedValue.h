// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"
#include "ImportedValue.generated.h"

USTRUCT()
struct FChaosClothAssetImportedVectorValue
{
	GENERATED_BODY()

	using ImportedType = FVector3f;

	/**
	 * Property vector Value
	*/
	UPROPERTY(EditAnywhere, Category = "Imported Value", Meta = (EditCondition = "!bUseImportedValue"))
	mutable FVector3f ImportedValue = FVector3f::Zero();
	
	/**
	 * Whether the property can use the values imported from USD
	 */
	UPROPERTY(EditAnywhere, Category = "Imported Value")
	bool bUseImportedValue = false;
};

USTRUCT()
struct FChaosClothAssetImportedIntValue
{
	GENERATED_BODY()

	using ImportedType = int32;

	/**
	 * Property integer Value
	*/
	UPROPERTY(EditAnywhere, Category = "Imported Value", Meta = (EditCondition = "!bUseImportedValue"))
	mutable int32 ImportedValue = 0;
	
	/**
	 * Whether the property can use the values imported from USD
	 */
	UPROPERTY(EditAnywhere, Category = "Imported Value")
	bool bUseImportedValue = false;
};

USTRUCT()
struct FChaosClothAssetImportedFloatValue
{
	GENERATED_BODY()

	using ImportedType = float;

	/**
	 * Property float value
	*/
	UPROPERTY(EditAnywhere, Category = "Imported Value", Meta = (EditCondition = "!bUseImportedValue"))
	mutable float ImportedValue = 0.0f;
	
	/**
	 * Whether the property can use the values imported from USD
	 */
	UPROPERTY(EditAnywhere, Category = "Imported Value")
	bool bUseImportedValue = false;
};

