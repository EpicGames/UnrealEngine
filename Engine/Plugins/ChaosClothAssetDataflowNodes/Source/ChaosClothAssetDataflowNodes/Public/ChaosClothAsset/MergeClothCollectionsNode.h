// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "MergeClothCollectionsNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** Merge multiple cloth collections into a single cloth collection of multiple patterns. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetMergeClothCollectionsNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetMergeClothCollectionsNode_v2, "MergeClothCollections", "Cloth", "Cloth Merge Collection")

public:
	UPROPERTY()
	TArray<FManagedArrayCollection> Collections;

	UPROPERTY(Meta = (DataflowOutput, DataflowPassthrough = "Collections[0]"))
	FManagedArrayCollection Collection;

	FChaosClothAssetMergeClothCollectionsNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return true; }
	virtual bool CanRemovePin() const override { return Collections.Num() > NumInitialOptionalInputs; }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface


	static constexpr int32 NumRequiredInputs = 0;
	static constexpr int32 NumInitialOptionalInputs = 2;
	UE::Dataflow::TConnectionReference<FManagedArrayCollection> GetConnectionReference(int32 Index) const;
};



/** Merge multiple cloth collections into a single cloth collection of multiple patterns. */
USTRUCT(Meta = (DataflowCloth, Deprecated="5.5"))
struct UE_DEPRECATED(5.5, "Use the newer version of this node instead.") FChaosClothAssetMergeClothCollectionsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetMergeClothCollectionsNode, "MergeClothCollections", "Cloth", "Cloth Merge Collection")

public:
	static constexpr int32 MaxInputs = 6;  // Hardcoded number of inputs since it is currently not possible to use arrays for optional inputs

	/** Input 0, right click on the node and add pins to add more merge inputs. */
	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;
	/** Input 1, right click on the node and add pins to add more merge inputs. */
	UPROPERTY(Meta = (DisplayName = "Collection 1"))
	FManagedArrayCollection Collection1;
	/** Input 2, right click on the node and add pins to add more merge inputs. */
	UPROPERTY(Meta = (DisplayName = "Collection 2"))
	FManagedArrayCollection Collection2;
	/** Input 3, right click on the node and add pins to add more merge inputs. */
	UPROPERTY(Meta = (DisplayName = "Collection 3"))
	FManagedArrayCollection Collection3;
	/** Input 4, right click on the node and add pins to add more merge inputs. */
	UPROPERTY(Meta = (DisplayName = "Collection 4"))
	FManagedArrayCollection Collection4;
	/** Input 5, right click on the node and add pins to add more merge inputs. */
	UPROPERTY(Meta = (DisplayName = "Collection 5"))
	FManagedArrayCollection Collection5;
	/** The number of inputs currently exposed to the node UI. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	int32 NumInputs = NumInitialOptionalInputs;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FChaosClothAssetMergeClothCollectionsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return NumInputs < MaxInputs; }
	virtual bool CanRemovePin() const override { return NumInputs > NumInitialOptionalInputs; }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface

	TArray<const FManagedArrayCollection*> GetCollections() const;
	const FManagedArrayCollection* GetCollection(int32 Index) const;

	static constexpr int32 NumRequiredInputs = 0;
	static constexpr int32 NumInitialOptionalInputs = 1;
};
