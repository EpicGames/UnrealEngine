// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include  "SelectionNode.h"
#include "ProceduralSelectionNode.generated.h"

/** Type of procedural selection */
UENUM()
enum class EChaosClothAssetProceduralSelectionType : uint8
{
	SelectAll /*Select all elements within the selected Group*/,
	Conversion /*Convert an existing selection set to Group (if supported)*/
};

/** Procedurally generate a Cloth Selection set. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetProceduralSelectionNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetProceduralSelectionNode, "ProceduralSelection", "Cloth", "Cloth Procedural Selection")

public:

	FChaosClothAssetProceduralSelectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The name to be use as a selection. */
	UPROPERTY(EditAnywhere, Category = "Procedural Selection", Meta = (DataflowOutput))
	FString OutputName;

	/** The type of element the selection refers to */
	UPROPERTY(EditAnywhere, Category = "Procedural Selection")
	FChaosClothAssetNodeSelectionGroup Group;

	/** The procedural selection method */
	UPROPERTY(EditAnywhere, Category = "Procedural Selection")
	EChaosClothAssetProceduralSelectionType SelectionType = EChaosClothAssetProceduralSelectionType::SelectAll;

	/** Selection set to convert from when using Conversion selection type. */
	UPROPERTY(EditAnywhere, Category = "Procedural Selection", Meta = (Dataflowinput, EditCondition = "SelectionType == EChaosClothAssetProceduralSelectionType::Conversion"))
	FChaosClothAssetConnectableIStringValue ConversionInputName;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
