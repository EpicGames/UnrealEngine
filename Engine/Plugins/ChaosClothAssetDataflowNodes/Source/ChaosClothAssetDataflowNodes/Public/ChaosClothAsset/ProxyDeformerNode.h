// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ProxyDeformerNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** Selection Filter Set*/
USTRUCT()
struct FChaosClothAssetSelectionFilterSet
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableIStringValue RenderSelection = { TEXT("SelectionRenderFilterSet") };

	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableIStringValue SimSelection = { TEXT("SelectionSimFilterSet") };
};

/** Add the proxy deformer information to this cloth collection's render data. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.5"))
struct UE_DEPRECATED(5.5, "Use the newer version of this node instead.") FChaosClothAssetProxyDeformerNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetProxyDeformerNode, "ProxyDeformer", "Cloth", "Cloth Simulation Proxy Deformer")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/**
	 * The name of a selection containing all the dynamic points. Must be of group type SimVertices2D, SimVertices3D, or SimFaces.
	 * Using an empty (or invalid) selection will make the proxy deformer consider all simulation points as dynamic points,
	 * and will fully contribute to the render mesh animations (as opposed to using the render mesh skinning for the non dynamic points).
	 * This selection is usually built from the same weight map set to the MaxDistance config using a WeightMapToSelection node and a very low threshold.
	 */
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableIStringValue SimVertexSelection;

	/** Selection filter set used to restrict a primary selection of render vertices to a secondary selection of simulation mesh triangles. Right click and do AddPin to add more selection sets .*/
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableIStringValue SelectionFilterSet0 = { TEXT("SelectionFilterSet") };  // Must be an IStringValue for the first element of the array

	/** Selection filter set used to restrict a primary selection of render vertices to a secondary selection of simulation mesh triangles. Right click and do AddPin to add more selection sets .*/
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableStringValue SelectionFilterSet1 = { TEXT("SelectionFilterSet") };

	/** Selection filter set used to restrict a primary selection of render vertices to a secondary selection of simulation mesh triangles. Right click and do AddPin to add more selection sets .*/
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableStringValue SelectionFilterSet2 = { TEXT("SelectionFilterSet") };

	/** Selection filter set used to restrict a primary selection of render vertices to a secondary selection of simulation mesh triangles. Right click and do AddPin to add more selection sets .*/
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableStringValue SelectionFilterSet3 = { TEXT("SelectionFilterSet") };

	/** Selection filter set used to restrict a primary selection of render vertices to a secondary selection of simulation mesh triangles. Right click and do AddPin to add more selection sets .*/
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableStringValue SelectionFilterSet4 = { TEXT("SelectionFilterSet") };

	/** Selection filter set used to restrict a primary selection of render vertices to a secondary selection of simulation mesh triangles. Right click and do AddPin to add more selection sets .*/
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableStringValue SelectionFilterSet5 = { TEXT("SelectionFilterSet") };

	/** Selection filter set used to restrict a primary selection of render vertices to a secondary selection of simulation mesh triangles. Right click and do AddPin to add more selection sets .*/
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableStringValue SelectionFilterSet6 = { TEXT("SelectionFilterSet") };

	/** Selection filter set used to restrict a primary selection of render vertices to a secondary selection of simulation mesh triangles. Right click and do AddPin to add more selection sets .*/
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableStringValue SelectionFilterSet7 = { TEXT("SelectionFilterSet") };

	/** Selection filter set used to restrict a primary selection of render vertices to a secondary selection of simulation mesh triangles. Right click and do AddPin to add more selection sets .*/
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableStringValue SelectionFilterSet8 = { TEXT("SelectionFilterSet") };

	/** Selection filter set used to restrict a primary selection of render vertices to a secondary selection of simulation mesh triangles. Right click and do AddPin to add more selection sets .*/
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	FChaosClothAssetConnectableStringValue SelectionFilterSet9 = { TEXT("SelectionFilterSet") };

	/** Whether using multiple simulation mesh triangles to influence the position of the deformed render vertex. */
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	bool bUseMultipleInfluences = false;

	/** The radius around the render vertices to look for all simulation mesh triangles influencing it (AKA SkinningKernelRadius). */
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer", Meta = (EditCondition = "bUseMultipleInfluences"))
	float InfluenceRadius = 5.f;

	/**
	 * Whether to create a smoothed _SkinningBlendWeight render weight map to ease the transition between the deformed part and the skinned part of the render mesh.
	 * When no transition is created there will be a visible step in the rendered triangles around the edge of the kinematic/dynamic transition of the proxy simulation mesh.
	 * The _SkinningBlendWeight render weight map is created regardless of the transition being created smooth or not, and can be later adjusted using the weight map tool.
	 */
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	bool bUseSmoothTransition = true;

	/**
	 * The name of the render mesh weight map generated by this node detailing the contribution of the proxy deformer.
	 * Value ranges between 0 (fully deformed) and 1 (fully skinned).
	 * The name of this render mesh weight map cannot be changed and is only provided for further tweaking.
	 */
	UPROPERTY(VisibleAnywhere, Category = "ProxyDeformer", Meta = (DataflowOutput))
	FString SkinningBlendName;

	FChaosClothAssetProxyDeformerNode(const UE::Dataflow::FNodeParameters & InParam, FGuid InGuid = FGuid::NewGuid());

private:
	/** Hardcoded number of FilterSets since it is currently not possible to use arrays for optional inputs. */
	static constexpr int32 MaxNumFilterSets = 10;

	/** The number of filter sets currently exposed to the node UI. */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UPROPERTY()
	int32 NumFilterSets = NumInitialOptionalInputs;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

		//~ Begin FDataflowNode interface
		virtual void Evaluate(UE::Dataflow::FContext & Context, const FDataflowOutput * Out) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return NumFilterSets < MaxNumFilterSets; }
	virtual bool CanRemovePin() const override { return NumFilterSets > NumInitialOptionalInputs; }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin & Pin) override;
	virtual void PostSerialize(const FArchive & Ar) override;
	//~ End FDataflowNode interface

	TArray<FName> GetSelectionFilterNames(UE::Dataflow::FContext & Context) const;
	TArray<const FChaosClothAssetConnectableStringValue*> Get1To9SelectionFilterSets() const;

	static constexpr int32 NumRequiredInputs = 2; // non-filter set inputs
	static constexpr int32 NumInitialOptionalInputs = 1; // filter set inputs that are created in the constructor.
};

/**
 * Adds the proxy deformer information to this cloth collection's render data.
 * This version of the node does not try to preserve existing Render Tangents by default to maintain existing behavior.
 * 
 * This node is only required to selectively assign specific areas of the render mesh to the sim mesh in the proxy deformer data.
 * When this node isn't provided to the construction graph, default proxy deformer data are automatically generated.
 * This node can be used to either override or remove the default proxy deformer data.
 * 
 * To enable deformer areas, selection filter sets must be provided.
 * With no selection filter sets, the Cloth Asset is by default fully skinned.
 * When more selection filter sets are needed, right-click on the node and do AddOptionPin to add more sets.
 * Each selection filter set is composed by two selections:
 * * A render mesh selection (natively a RenderVertices selection, but RenderFaces selections can also be used).
 * * A simulation mesh selection (natively a SimFaces selection, but SimVertices3D and SimVertices2D selections can also be used).
 * The render mesh selection of vertices will then only be deformed by the associated simulation mesh selection of faces.
 * The Proxy Deformer node evaluations are usually slower than the equivalent default proxy deformer due to the added processing cost
 * of the selection filter sets. When in use, it is therefore better to place it earlier in the graph.
 * 
 * A default SkinningBlend weight map is also generated by the node that masks the areas for which selection filters aren't provided.
 * The map can then be painted, or replaced by a smoothed transition map generated by the SkinningBlend node.
 * The SkinningBlend node can leverage information about the kinematic areas added by the MaxDistanceConfig node.
 * Unlike the default SkinningBlend weight map provided by the Proxy Deformer node, the SkinningBlend node generates areas of smooth
 * transition where the render mesh vertices can be simultaneously skinned and deformed by the cloth.
 * 
 * To sum up:
 * * No ProxyDeformer node -> The Cloth Asset is fully deformed.
 * * ProxyDeformer node without selection filters -> The Cloth Asset is fully skinned.
 * * ProxyDeformer node with selection filters -> the Cloth Asset is partially deformed/skinned.
 * * The ProxyDeformer can be slow to evaluate, always place it as early as possible in the graph to avoid costly re-evaluations.
 * * A SkinningBlend node placed after the MaxDistanceConfig adds smooth transitions between the skinned and proxy deformed areas.
 */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.6"))
struct FChaosClothAssetProxyDeformerNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetProxyDeformerNode_v2, "ProxyDeformer", "Cloth", "Cloth Simulation Proxy Deformer")

public:

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Selection filter set used to restrict a selection of render vertices to a selection of simulation mesh triangles. Right click and do AddOptionPin to add more selection sets .*/
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "ProxyDeformer", Meta = (SkipInDisplayNameChain))
	TArray<FChaosClothAssetSelectionFilterSet> SelectionFilterSets;

	/** Whether using multiple simulation mesh triangles to influence the position of the deformed render vertex. */
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	bool bUseMultipleInfluences = false;

	/** The radius around the render vertices to look for all simulation mesh triangles influencing it (AKA SkinningKernelRadius). */
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer", Meta = (EditCondition = "bUseMultipleInfluences"))
	float InfluenceRadius = 5.f;

	/** Whether or not to include the RenderTangents when generating the proxy deformer data. */
	UPROPERTY(EditAnywhere, Category = "ProxyDeformer")
	bool bPreserveRenderTangents = false;

	FChaosClothAssetProxyDeformerNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid(), bool bInPreserveRenderTangents = false);

protected:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return true; }
	virtual bool CanRemovePin() const override { return SelectionFilterSets.Num() > NumInitialSelectionFilterSets; }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface

private:

	TArray<TPair<FName, FName>> GetSelectionFilterNames(UE::Dataflow::FContext& Context) const;
	UE::Dataflow::TConnectionReference<FString> GetRenderConnectionReference(int32 Index) const;
	UE::Dataflow::TConnectionReference<FString> GetSimConnectionReference(int32 Index) const;

	static constexpr int32 NumRequiredInputs = 1; // non-filter set inputs
	static constexpr int32 NumInitialSelectionFilterSets = 1;
};


/**
 * Adds the proxy deformer information to this cloth collection's render data.
 * This version will include the render tangents when generating the proxy deformer data by default (matches the default behavior if no proxy deformer node is added).
 *
 * This node is only required to selectively assign specific areas of the render mesh to the sim mesh in the proxy deformer data.
 * When this node isn't provided to the construction graph, default proxy deformer data are automatically generated.
 * This node can be used to either override or remove the default proxy deformer data.
 *
 * To enable deformer areas, selection filter sets must be provided.
 * With no selection filter sets, the Cloth Asset is by default fully skinned.
 * When more selection filter sets are needed, right-click on the node and do AddOptionPin to add more sets.
 * Each selection filter set is composed by two selections:
 * * A render mesh selection (natively a RenderVertices selection, but RenderFaces selections can also be used).
 * * A simulation mesh selection (natively a SimFaces selection, but SimVertices3D and SimVertices2D selections can also be used).
 * The render mesh selection of vertices will then only be deformed by the associated simulation mesh selection of faces.
 * The Proxy Deformer node evaluations are usually slower than the equivalent default proxy deformer due to the added processing cost
 * of the selection filter sets. When in use, it is therefore better to place it earlier in the graph.
 *
 * A default SkinningBlend weight map is also generated by the node that masks the areas for which selection filters aren't provided.
 * The map can then be painted, or replaced by a smoothed transition map generated by the SkinningBlend node.
 * The SkinningBlend node can leverage information about the kinematic areas added by the MaxDistanceConfig node.
 * Unlike the default SkinningBlend weight map provided by the Proxy Deformer node, the SkinningBlend node generates areas of smooth
 * transition where the render mesh vertices can be simultaneously skinned and deformed by the cloth.
 *
 * To sum up:
 * * No ProxyDeformer node -> The Cloth Asset is fully deformed.
 * * ProxyDeformer node without selection filters -> The Cloth Asset is fully skinned.
 * * ProxyDeformer node with selection filters -> the Cloth Asset is partially deformed/skinned.
 * * The ProxyDeformer can be slow to evaluate, always place it as early as possible in the graph to avoid costly re-evaluations.
 * * A SkinningBlend node placed after the MaxDistanceConfig adds smooth transitions between the skinned and proxy deformed areas.
 */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetProxyDeformerNode_v3 : public FChaosClothAssetProxyDeformerNode_v2
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetProxyDeformerNode_v3, "ProxyDeformer", "Cloth", "Cloth Simulation Proxy Deformer")

public:
	FChaosClothAssetProxyDeformerNode_v3(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
};
