// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "RecalculateNormalsNode.generated.h"

/** Recalculate the geometry's normals. Experimental, only does the render geometry. */
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetRecalculateNormalsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetRecalculateNormalsNode, "RecalculateNormals", "Cloth", "Cloth Recalculate Simulation Render Mesh Normals")

public:
	FChaosClothAssetRecalculateNormalsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode Interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode Interface

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;
};
