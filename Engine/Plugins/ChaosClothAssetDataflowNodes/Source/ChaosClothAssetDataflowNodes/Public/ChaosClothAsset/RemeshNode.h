// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "RemeshNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

UENUM(BlueprintType)
enum class EChaosClothAssetRemeshMethod : uint8
{
	Remesh,
	Simplify
};

UENUM()
enum class EChaosClothAssetRemeshSelectionSource : uint8
{
	SelectionInput  UMETA(DisplayName = "From Selection Input"),
	DensityMapInput UMETA(DisplayName = "From DensityMap Input (With Threshold)"),
};

/** Remesh the cloth surface(s) to get the specified mesh resolution(s).
 *  NOTE: Accessory Meshes, Weight Maps, Skinning Data, Self Collision Spheres, and Long Range Attachment Constraints will be reconstructed on the output mesh, however all other Selections will be removed
 */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetRemeshNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetRemeshNode_v2, "Remesh", "Cloth", "Cloth Remesh")

public:

	FChaosClothAssetRemeshNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** Cloth collection to remesh */
	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** True if the sim mesh needs to be remeshed */
	UPROPERTY(EditAnywhere, Category = "Sim Mesh")
	bool bRemeshSim = true;

	/** Defines the source for selecting which part of the sim mesh is being remeshed */
	UPROPERTY(EditAnywhere, Category = "Sim Mesh", Meta = (EditCondition = "bRemeshSim"))
	EChaosClothAssetRemeshSelectionSource SelectionSourceSim = EChaosClothAssetRemeshSelectionSource::DensityMapInput;

	/** 
	* Optional sim mesh triangle selection to use for remeshing 
	* the selection defines what triangles will can remeshed, the one not in the selection will remain unchanged
	*/
	UPROPERTY(EditAnywhere, Category = "Sim Mesh", Meta = (EditCondition = "bRemeshSim && SelectionSourceSim == EChaosClothAssetRemeshSelectionSource::SelectionInput"))
	FChaosClothAssetConnectableIStringValue TriangleSelectionSim;

	/**
	 * Range of target mesh resolutions, as a percentage of input triangle mesh resolution. A value of 50 on all vertices should roughly halve the total number of triangles.
	 * If a valid vertex weight map is specified, it will use vertex weights to interpolate between the Lo and Hi values. Otherwise it will use the Lo value on all vertices.
	 */
	UPROPERTY(EditAnywhere, Category = "Sim Mesh", Meta = (EditCondition = "bRemeshSim"))
	FChaosClothAssetWeightedValueNonAnimatable DensityMapSim = { 100.f, 200.f, TEXT("DensityMapSim") };

	/** 
	* Threshold used to define how to convert the sim mesh density into a selection map 
	* Triangles with all their vertices above or equal to the threshold will remain untouched
	*/
	UPROPERTY(EditAnywhere, Category = "Sim Mesh", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bRemeshSim && SelectionSourceSim == EChaosClothAssetRemeshSelectionSource::DensityMapInput"))
	float DensityMapThresholdSim = 1.0f;

	/** Number of iterations to use for remeshing the sim mesh */
	UPROPERTY(EditAnywhere, Category = "Sim Mesh", meta = (UIMin = "0", UIMax = "20", ClampMin = "0", ClampMax = "200", EditCondition = "bRemeshSim"))
	int32 IterationsSim = 10;

	/** Smoothing factor for the sim remeshing operation */
	UPROPERTY(EditAnywhere, Category = "Sim Mesh", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bRemeshSim"))
	double SmoothingSim = 0.25;

	/** True if the render mesh needs to be remeshed */
	UPROPERTY(EditAnywhere, Category = "Render Mesh")
	bool bRemeshRender = false;

	/** Method to use when remeshing the render mesh */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bRemeshRender"))
	EChaosClothAssetRemeshMethod RemeshMethodRender = EChaosClothAssetRemeshMethod::Remesh;

	/** Defines the source for selecting which part of the Render mesh is being remeshed */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", Meta = (EditCondition = "bRemeshRender"))
	EChaosClothAssetRemeshSelectionSource SelectionSourceRender = EChaosClothAssetRemeshSelectionSource::DensityMapInput;

	/**
	* Optional render mesh triangle selection to use for remeshing
	* The selection defines what traingles will can remeshed, the one not in teh selection will remain unchanged
	*/
	UPROPERTY(EditAnywhere, Category = "Render Mesh", Meta = (EditCondition = "bRemeshRender && && SelectionSourceRender == EChaosClothAssetRemeshSelectionSource::SelectionInput"))
	FChaosClothAssetConnectableIStringValue TriangleSelectionRender;

	/**
	 * Range of target mesh resolutions when using the Remesh method, as a percentage of input triangle mesh resolution. A value of 50 on all vertices should roughly halve the total number of triangles.
	 * If a valid vertex weight map is specified, it will use vertex weights to interpolate between the Lo and Hi values. Otherwise it will use the Lo value on all vertices.
	 */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", Meta = (EditCondition = "bRemeshRender && RemeshMethodRender == EChaosClothAssetRemeshMethod::Remesh"))
	FChaosClothAssetWeightedValueNonAnimatable DensityMapRender = { 100.f, 200.f, TEXT("DensityMapRender") };

	/**
	* Threshold used to define how to convert the render mesh density into a selection map
	* Triangles with all their vertices above or equal to the threshold will remain untouched
	*/
	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bRemeshRender && SelectionSourceRender == EChaosClothAssetRemeshSelectionSource::DensityMapInput"))
	float DensityMapThresholdRender = 1.0f;

	/**
	 * Target mesh resolution when using the Simplify method, as a percentage of input triangle mesh resolution. A value of 50 should roughly halve the total number of triangles.
	 */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (UIMin = "1", UIMax = "200", ClampMin = "1", EditCondition = "bRemeshRender && RemeshMethodRender == EChaosClothAssetRemeshMethod::Simplify"))
	int32 TargetPercentRender = 100;

	/** number of iterations to use for remeshing the render mesh */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (UIMin = "0", UIMax = "20", ClampMin = "0", ClampMax = "100", EditCondition = "bRemeshRender && RemeshMethodRender == EChaosClothAssetRemeshMethod::Remesh"))
	int32 IterationsRender = 10;

	/** smoothing factor for the render remeshing operation */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bRemeshRender && RemeshMethodRender == EChaosClothAssetRemeshMethod::Remesh"))
	double SmoothingRender = 0.25;

	/** If checked, attempt to find matching vertices along Render mesh boundaries and remesh these separately */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (EditCondition = "bRemeshRender"))
	bool bRemeshRenderSeams = false;

	/** Number of remesh iterations over the Render mesh seams */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (UIMin = "0", UIMax = "20", ClampMin = "0", ClampMax = "100", EditCondition = "bRemeshRender && bRemeshRenderSeams"))
	int32 RenderSeamRemeshIterations = 1;

	virtual void Evaluate(UE::Dataflow::FContext & Context, const FDataflowOutput * Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDraw() const override;
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif

	void GetSimSelectionFromContext(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& Collection, const FString& SimDensityMap,  TSet<int32>& OutSelection) const;
	void GetRenderSelectionFromContext(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& Collection, const FString& RenderDensityMap, TSet<int32>& OutSelection) const;
};



/** Remesh the cloth surface(s) to get the specified mesh resolution(s).
 *  NOTE: Weight Maps, Skinning Data, Self Collision Spheres, and Long Range Attachment Constraints will be reconstructed on the output mesh, however all other Selections will be removed
 */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.6"))
struct UE_DEPRECATED(5.6, "Use the newer version of this node instead.") FChaosClothAssetRemeshNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetRemeshNode, "Remesh", "Cloth", "Cloth Remesh")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	UPROPERTY(EditAnywhere, Category = "Sim Mesh")
	bool bRemeshSim = true;

	UPROPERTY(EditAnywhere, Category = "Sim Mesh", meta=(UIMin = "1", UIMax = "200", ClampMin = "1", EditCondition = "bRemeshSim"))
	int32 TargetPercentSim = 100;

	UPROPERTY(EditAnywhere, Category = "Sim Mesh", meta = (UIMin = "0", UIMax = "20", ClampMin = "0", ClampMax = "200", EditCondition = "bRemeshSim"))
	int32 IterationsSim = 10;

	UPROPERTY(EditAnywhere, Category = "Sim Mesh", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bRemeshSim"))
	double SmoothingSim = 0.25;

	UPROPERTY(EditAnywhere, Category = "Sim Mesh", Meta = (EditCondition = "bRemeshSim"))
	FChaosClothAssetConnectableIStringValue DensityMapSim;

	UPROPERTY(EditAnywhere, Category = "Render Mesh")
	bool bRemeshRender = false;

	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta=(UIMin = "1", UIMax = "200", ClampMin = "1", EditCondition = "bRemeshRender"))
	int32 TargetPercentRender = 100;

	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bRemeshRender"))
	EChaosClothAssetRemeshMethod RemeshMethodRender = EChaosClothAssetRemeshMethod::Remesh;

	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (UIMin = "0", UIMax = "20", ClampMin = "0", ClampMax = "100", EditCondition = "bRemeshRender && RemeshMethodRender == EChaosClothAssetRemeshMethod::Remesh"))
	int32 IterationsRender = 10;

	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bRemeshRender && RemeshMethodRender == EChaosClothAssetRemeshMethod::Remesh"))
	double SmoothingRender = 0.25;

	/** If checked, attempt to find matching vertices along Render mesh boundaries and remesh these separately */ 
	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (EditCondition = "bRemeshRender"))
	bool bRemeshRenderSeams = false;

	/** Number of remesh iterations over the Render mesh seams */
	UPROPERTY(EditAnywhere, Category = "Render Mesh", meta = (UIMin = "0", UIMax = "20", ClampMin = "0", ClampMax = "100", EditCondition = "bRemeshRender && bRemeshRenderSeams"))
	int32 RenderSeamRemeshIterations = 1;

	UPROPERTY(EditAnywhere, Category = "Render Mesh", Meta = (EditCondition = "bRemeshRender && RemeshMethodRender == EChaosClothAssetRemeshMethod::Remesh"))
	FChaosClothAssetConnectableIStringValue DensityMapRender;

	FChaosClothAssetRemeshNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

private:

	void RemeshSimMesh(const TSharedRef<const FManagedArrayCollection>& ClothCollection,
		const FString& DensityMapName,
		const TSharedRef<FManagedArrayCollection>& OutClothCollection) const;

	void RemeshRenderMesh(const TSharedRef<const FManagedArrayCollection>& ClothCollection,
		const FString& DensityMapName,
		const TSharedRef<FManagedArrayCollection>& OutClothCollection) const;

};
