// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ReverseNormalsNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** Reverse the geometry's normals or/and winding order of the simulation or/and render meshes stored in the cloth collection. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetReverseNormalsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetReverseNormalsNode, "ReverseNormals", "Cloth", "Cloth Reverse Simulation Render Mesh Normals")

public:
	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** List of sim patterns to apply the operation on. All patterns will be used if left empty. */
	UPROPERTY(EditAnywhere, Category = "Reverse Normals|Simulation Mesh")
	TArray<int32> SimPatterns;

	/** Whether to reverse the simulation mesh normals. */
	UPROPERTY(EditAnywhere, Category = "Reverse Normals|Simulation Mesh")
	bool bReverseSimMeshNormals = true;

	/** Whether to reverse the simulation mesh triangles' winding order. */
	UPROPERTY(EditAnywhere, Category = "Reverse Normals|Simulation Mesh")
	bool bReverseSimMeshWindingOrder = false;

	/** List of render patterns to apply the operation on. All patterns will be used if left empty. */
	UPROPERTY(EditAnywhere, Category = "Reverse Normals|Render Mesh")
	TArray<int32> RenderPatterns;

	/** Whether to reverse the render mesh normals. */
	UPROPERTY(EditAnywhere, Category = "Reverse Normals|Render Mesh")
	bool bReverseRenderMeshNormals = true;

	/** Whether to reverse the render mesh triangles' winding order. */
	UPROPERTY(EditAnywhere, Category = "Reverse Normals|Render Mesh")
	bool bReverseRenderMeshWindingOrder = false;

	FChaosClothAssetReverseNormalsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
