// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ChaosClothAsset/AddWeightMapNode.h"
#include "ChaosClothAsset/ClothCollectionGroup.h"
#include "SelectionNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** What type of element is selected in the Selection */
UENUM()
enum class UE_DEPRECATED(5.4, "Use FChaosClothAssetNodeSelectionGroup instead") EChaosClothAssetSelectionType : uint8
{
	/** 2D simulation vertices */
	SimVertex2D,

	/** 3D simulation vertices */
	SimVertex3D,

	/** Render vertices */
	RenderVertex,

	/** Simulation faces (2D/3D are the same) */
	SimFace,

	/** Render faces */
	RenderFace,

	/** Deprecated marker */
	Deprecated UMETA(Hidden)
};

/**
 * The managed array collection group used in the selection.
 * This separate structure is required to allow for customization of the UI.
 */
USTRUCT()
struct FChaosClothAssetNodeSelectionGroup
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Selection Group")
	FString Name = TEXT("SimVertices3D");
};

/** How the sets stored on the SelectionNode should be applied to an existing set. If no set exists, it is treated as empty.*/
UENUM()
enum class EChaosClothAssetSelectionOverrideType : uint8
{
	/** Replace the existing set with values in [Secondary]Indices.*/
	ReplaceAll,
	/** Append values in [Secondary]Indices. Remove values in [Secondary]RemoveIndices.*/
	Modify
};

USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSelectionNode_v2 final : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSelectionNode_v2, "Selection", "Cloth", "Cloth Selection")
	DATAFLOW_NODE_RENDER_TYPE("SurfaceRender", FName("FClothCollection"), "Collection")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The collection used to transfer sets from. */
	UPROPERTY(Meta = (DataflowInput))
	FManagedArrayCollection TransferCollection;

	/** The name to be use as a selection. */
	UPROPERTY(EditAnywhere, Category = "Selection")
	FChaosClothAssetConnectableOStringValue OutputName;

	/** The name to populate this set from and override based on Selection Override Type. Output Name will be used if Input Name is empty.*/
	UPROPERTY(EditAnywhere, Category = "Selection")
	FChaosClothAssetConnectableIStringValue InputName;

	/** How to apply this node's Indices onto existing sets. Changing this value will change the output set.
	 *  To change how the node's stored indices are calculated, change the equivalent value on the Selection Tool context.*/
	UPROPERTY(EditAnywhere, Category = "Selection")
	EChaosClothAssetSelectionOverrideType SelectionOverrideType = EChaosClothAssetSelectionOverrideType::ReplaceAll;

	/** The type of element the selection refers to */
	UPROPERTY(EditAnywhere, Category = "Selection")
	FChaosClothAssetNodeSelectionGroup Group;

	/** Selected element indices */
	UPROPERTY(EditAnywhere, Category = "Selection", Meta = (ClampMin = "0"))
	TSet<int32> Indices;

	/** Indices to remove from the Input selection */
	UPROPERTY(EditAnywhere, Category = "Selection", Meta = (ClampMin = "0", EditCondition = "SelectionOverrideType == EChaosClothAssetSelectionOverrideType::Modify"))
	TSet<int32> RemoveIndices;

	/** Import (replace) the current selection from the input Collection's selection with the given Input Name (or Output Name if Input Name is empty). */
	UPROPERTY(EditAnywhere, Category = "Selection", Meta = (ButtonImage = "Icons.Refresh"))
	FDataflowFunctionProperty Import;

	/**
	 * Import (replace) the current selection from the input Collection's secondary selection with the given Input Name (or Output Name if Input Name is empty).
	 * Secondary selections are only supported in v1 of this node. This function is provided as a migration tool to this current version.
	 */
	UPROPERTY(EditAnywhere, Category = "Selection", Meta = (ButtonImage = "Icons.Refresh"))
	FDataflowFunctionProperty ImportSecondary;

	/**
	 * The type of transfer used to transfer the sim mesh sets when a TransferCollection is connected.
	 * This property is disabled when no TransferCollection input has been connected.
	 */
	UPROPERTY(EditAnywhere, Category = "Selection Transfer")
	EChaosClothAssetWeightMapTransferType SimTransferType = EChaosClothAssetWeightMapTransferType::Use2DSimMesh;

	/**
	 * Selections are internally converted to maps in order to do the transfer and then converted back.
	 * This value is used to do the conversion back. Decrease this value to (possibly) expand the converted selection.
	 */
	UPROPERTY(EditAnywhere, Category = "Selection Transfer", Meta = (ClampMin = "0", ClampMax = "1"))
	float TransferSelectionThreshold = 0.95f;

	/** Transfer the selection from the connected Transfer Collection containing a selection with Input Name (or Output Name if Input Name is empty). */
	UPROPERTY(EditAnywhere, Category = "Selection Transfer", Meta = (ButtonImage = "Icons.Convert"))
	FDataflowFunctionProperty Transfer;

	FChaosClothAssetSelectionNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	friend class UClothMeshSelectionTool;

	class FSelectionNodeChange;

	FName CHAOSCLOTHASSETDATAFLOWNODES_API GetInputName(UE::Dataflow::FContext& Context) const;
	void CHAOSCLOTHASSETDATAFLOWNODES_API SetIndices(const TSet<int32>& InputSet, const TSet<int32>& FinalSet);
	void CHAOSCLOTHASSETDATAFLOWNODES_API CalculateFinalSet(const TSet<int32>& InputSet, TSet<int32>& FinalSet) const;
	static TUniquePtr<class FToolCommandChange> CHAOSCLOTHASSETDATAFLOWNODES_API MakeSelectedNodeChange(const FChaosClothAssetSelectionNode_v2& Node);

	//~ Begin FDataflowNode implementation
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode implementation

	void OnImport(UE::Dataflow::FContext& Context);
	void OnImportSecondary(UE::Dataflow::FContext& Context);
	void OnTransfer(UE::Dataflow::FContext& Context);
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS  // For EChaosClothAssetSelectionType

/** Integer index set selection node. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.5"))
struct UE_DEPRECATED(5.5, "Use the newer version of this node instead.") FChaosClothAssetSelectionNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSelectionNode, "Selection", "Cloth", "Cloth Selection")
	DATAFLOW_NODE_RENDER_TYPE("SurfaceRender", FName("FClothCollection"), "Collection")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/**
	 * The collection used to transfer sets from.
	 * Connecting a collection containing a set with Input Name (or Name if Input Name is empty) and compatible Group
	 * will transfer the set to the input collection topology.
	 * Note this operation only happens once when the TransferCollection is first connected, or updated.
	 * Changing the InputName or the TransferType will also redo the transfer operation.
	 */
	UPROPERTY(Meta = (DataflowInput))
	FManagedArrayCollection TransferCollection;

	/** The name to give the selection attribute */
	UPROPERTY(EditAnywhere, Category = "Selection", Meta = (DataflowOutput))
	FString Name;

	/** The name to populate this set from and override based on Selection Override Type. Name will be used if Input Name is empty.*/
	UPROPERTY(EditAnywhere, Category = "Selection")
	FChaosClothAssetConnectableIStringValue InputName = { TEXT("") };

	/** How to apply this node's Indices onto existing sets. Changing this value will change the output set.
	 *  To change how the node's stored indices are calculated, change the equivalent value on the Selection Tool context.*/
	UPROPERTY(EditAnywhere, Category = "Selection")
	EChaosClothAssetSelectionOverrideType SelectionOverrideType = EChaosClothAssetSelectionOverrideType::ReplaceAll;

	/** The type of element the selection refers to */
	UE_DEPRECATED(5.4, "Use Group instead")
	UPROPERTY()
	EChaosClothAssetSelectionType Type_DEPRECATED = EChaosClothAssetSelectionType::Deprecated;

	/** The type of element the selection refers to */
	UPROPERTY(EditAnywhere, Category = "Selection")
	FChaosClothAssetNodeSelectionGroup Group;

	/** Selected element indices */
	UPROPERTY(EditAnywhere, Category = "Selection", Meta = (ClampMin = "0"))
	TSet<int32> Indices;

	/** Indices to remove from the Input selection */
	UPROPERTY(EditAnywhere, Category = "Selection", Meta = (ClampMin = "0", EditCondition = "SelectionOverrideType == EChaosClothAssetSelectionOverrideType::Modify"))
	TSet<int32> RemoveIndices;

	/** The type of element the secondary selection refers to */
	UPROPERTY(EditAnywhere, Category = "Secondary Selection")
	FChaosClothAssetNodeSelectionGroup SecondaryGroup;

	/** Secondary set of element indices */
	UPROPERTY(EditAnywhere, Category = "Secondary Selection", Meta = (ClampMin = "0"))
	TSet<int32> SecondaryIndices;

	/** Secondary set indices to remove from the Input selection */
	UPROPERTY(EditAnywhere, Category = "Secondary Selection", Meta = (ClampMin = "0", EditCondition = "SelectionOverrideType == EChaosClothAssetSelectionOverrideType::Modify"))
	TSet<int32> RemoveSecondaryIndices;

	/**
	 * The type of transfer used to transfer the sim mesh sets when a TransferCollection is connected.
	 * This property is disabled when no TransferCollection input has been connected.
	 */
	UPROPERTY(EditAnywhere, Category = "Selection Transfer", Meta = (EditCondition = "TransferCollectionHash != 0"))
	EChaosClothAssetWeightMapTransferType SimTransferType = EChaosClothAssetWeightMapTransferType::Use2DSimMesh;

	/**
	 * Selections are internally converted to maps in order to do the transfer and then converted back.
	 * This value is used to do the conversion back. Decrease this value to (possibly) expand the converted selection.
	 */
	UPROPERTY(EditAnywhere, Category = "Selection Transfer", Meta = (EditCondition = "TransferCollectionHash != 0", ClampMin = "0", ClampMax = "1"))
	float TransferSelectionThreshold = 0.95f;

	FChaosClothAssetSelectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	/** Return a cached array of all the groups used by the input collection during at the time of the latest evaluation. */
	UE_DEPRECATED(5.5, "This function is deprecated and will now return an empty array.")
	const TArray<FName>& GetCachedCollectionGroupNames() const { return CachedCollectionGroupNames; }

	FName CHAOSCLOTHASSETDATAFLOWNODES_API GetInputName(UE::Dataflow::FContext& Context) const;

	// Set Indices based on SelectionOverrideType
	void CHAOSCLOTHASSETDATAFLOWNODES_API SetIndices(const TSet<int32>& InputSet, const TSet<int32>& FinalSet);
	UE_DEPRECATED(5.6, "Secondary selection sets are no longer supported.")
	void CHAOSCLOTHASSETDATAFLOWNODES_API SetSecondaryIndices(const TSet<int32>& InputSet, const TSet<int32>& FinalSet);

	void CHAOSCLOTHASSETDATAFLOWNODES_API CalculateFinalSet(const TSet<int32>& InputSet, TSet<int32>& FinalSet) const;
	UE_DEPRECATED(5.6, "Secondary selection sets are no longer supported.")
	void CHAOSCLOTHASSETDATAFLOWNODES_API CalculateFinalSecondarySet(const TSet<int32>& InputSet, TSet<int32>& FinalSet) const;

private:
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	UE_DEPRECATED(5.5, "This function is deprecated and will not be called on selection/deselection.")
	virtual void OnSelected(UE::Dataflow::FContext& /*Context*/) {}
	UE_DEPRECATED(5.5, "This function is deprecated and will not be called on selection/deselection.")
	virtual void OnDeselected() {}
	virtual void Serialize(FArchive& Ar);

	TArray<FName> CachedCollectionGroupNames;

	UPROPERTY()
	uint32 TransferCollectionHash = 0;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS  // For EChaosClothAssetSelectionType

