// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "SelectionToIntMapNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** Convert an integer index selection to an integer map. Map type will match the selection type.*/
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSelectionToIntMapNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSelectionToIntMapNode, "SelectionToIntMap", "Cloth", "Cloth Selection To Int Map")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The name of the selection to convert. */
	UPROPERTY(EditAnywhere, Category = "Selection To Int Map")
	FChaosClothAssetConnectableIStringValue SelectionName;

	/**
	 * The name of the integer map attribute that will be added to the collection.
	 * If left empty the same name as the selection name will be used instead.
	 */
	UPROPERTY(EditAnywhere, Category = "Selection To Int Map")
	FChaosClothAssetConnectableIOStringValue IntMapName;


	/** If the IntMapName already exists, keep existing values rather than overwriting them with 'Unselected Value'.*/
	UPROPERTY(EditAnywhere, Category = "Selection To Int Map")
	bool bKeepExistingUnselectedValues = true;

	/** The value unselected elements receive. Unselected existing values can be kept by setting 'Keep Existing Unselected Values'*/
	UPROPERTY(EditAnywhere, Category = "Selection To Int Map")
	int32 UnselectedValue = 0;

	/** The value selected elements receive. */
	UPROPERTY(EditAnywhere, Category = "Selection To Int Map")
	int32 SelectedValue = 1;

	FChaosClothAssetSelectionToIntMapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
