// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "SelectionToWeightMapNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** Convert an integer index selection to a vertex weight map where different map values can be set for selected and unselected vertices. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSelectionToWeightMapNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSelectionToWeightMapNode, "SelectionToWeightMap", "Cloth", "Cloth Selection To Weight Map")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The name of the selection to convert. */
	UPROPERTY(Meta = (DataflowInput))
	FString SelectionName;

	/**
	 * The name of the weight map attribute that will be added to the collection.
	 * If left empty the same name as the selection name will be used instead.
	 */
	UPROPERTY(EditAnywhere, Category = "Selection To Weight Map", Meta = (DataflowOutput))
	FString WeightMapName;

	/** The value unselected vertices receive. */
	UPROPERTY(EditAnywhere, Category = "Selection To Weight Map", Meta = (ClampMin = "0", ClampMax = "1"))
	float UnselectedValue = 0.f;

	/** The value selected vertices receive. */
	UPROPERTY(EditAnywhere, Category = "Selection To Weight Map", Meta = (ClampMin = "0", ClampMax = "1"))
	float SelectedValue = 1.f;

	FChaosClothAssetSelectionToWeightMapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
