// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "SetPhysicsAssetNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class UPhysicsAsset;

/** Replace the current physics assets to collide the simulation mesh against. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSetPhysicsAssetNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSetPhysicsAssetNode, "SetPhysicsAsset", "Cloth", "Cloth Set Physics Asset")

public:
	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The physics asset to assign to the Cloth Collection. */
	UPROPERTY(EditAnywhere, Category = "Set Physics Asset", Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "PhysicsAsset"))
	TObjectPtr<UPhysicsAsset> PhysicsAsset;

	FChaosClothAssetSetPhysicsAssetNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
