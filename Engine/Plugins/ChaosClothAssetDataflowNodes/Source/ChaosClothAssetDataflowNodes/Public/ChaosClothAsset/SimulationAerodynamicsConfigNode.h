// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "Chaos/SoftsSimulationSpace.h"
#include "SimulationAerodynamicsConfigNode.generated.h"

/** Aerodynamics properties configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationAerodynamicsConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationAerodynamicsConfigNode, "SimulationAerodynamicsConfig", "Cloth", "Cloth Simulation Aerodynamics Config")

public:
	/**
	 * Use the deprecated "Legacy" point-based wind model in order to preserve behavior of cloth assets
	 * authored against the legacy UChaosClothConfig pipeline. The point-based model is not an accurate
	 * aerodynamic model and as such should be avoided. Being point based, it doesn't take into account
	 * the surface area that gets hit by the wind. Using this model disables the aerodynamically
	 * accurate wind model (Drag / Lift weighted values, outer drag/lift) and water-body interaction.
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (InteractorName = "UsePointBasedWindModel"))
	bool bUsePointBasedWindModel = false;

	/**
	 * The density of the medium in which the aerodynamic forces take place, usually air.
	 * The fluid density is given in kg/m^3.
	 * Air density is considered to be around 1.225 kg/m^3 in average atmospheric conditions.
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "10000", InteractorName = "FluidDensity"))
	float FluidDensity = 1.225f;

	/**
	 * Wind velocity is specified in this space.
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (InteractorName = "WindVelocitySpace"))
	EChaosSoftsSimulationSpace WindVelocitySpace = EChaosSoftsSimulationSpace::WorldSpace;
	
	/**
	 * The fixed wind velocity [m/s] for this asset.
	 * For reference a wind gust is above 8m/s (18mph).
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (UIMin = "0", UIMax = "10", InteractorName = "WindVelocity"))
	FVector3f WindVelocity = { 0.f, 0.f, 0.f };

	/**
	 * Ratio of aerodynamic forces applied as turbulent rather than laminar flow.
	 * Typically, you will want to use turbulent flow for wind and laminar flow for swimming underwater (unless the water is flowing very rapidly).
	 * Turbulent forces scale like velocity squared. Laminar forces scale like velocity.
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (ClampMin = "0", ClampMax = "1", InteractorName = "TurbulenceRatio"))
	float TurbulenceRatio = 1.f;

	/**
	 * The aerodynamic coefficient of drag applying on each particle.
	 * When "Outer Drag" is enabled, this acts as the "Inner Drag", i.e., drag applied when the air velocity is
	 * moving in the mesh normal direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "10", EditCondition = "!bUsePointBasedWindModel", EditConditionHides, InteractorName = "Drag"))
	FChaosClothAssetWeightedValue Drag = { true, 0.035f, 1.f, TEXT("Drag"), true };

	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (InlineEditConditionToggle))
	bool bEnableOuterDrag = false;

	/**
	 * The aerodynamic coefficient of drag applying on each particle when the air velocity is moving
	 * against the mesh normal direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "10", EditCondition = "bEnableOuterDrag", InteractorName = "OuterDrag"))
	FChaosClothAssetWeightedValue OuterDrag = { true, 0.035f, 1.f, TEXT("OuterDrag"), true };


	/**
	 * The aerodynamic coefficient of lift applying on each particle.
	 * When "Outer Lift" is enabled, this acts as the "Inner Lift", i.e., lift applied when the air velocity is
	 * moving in the mesh normal direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "10", EditCondition = "!bUsePointBasedWindModel", EditConditionHides, InteractorName = "Lift"))
	FChaosClothAssetWeightedValue Lift = { true, 0.035f, 1.f, TEXT("Lift"), true };

	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (InlineEditConditionToggle))
	bool bEnableOuterLift = false;

	/**
	 * The aerodynamic coefficient of lift applying on each particle when the air velocity is moving
	 * against the mesh normal direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "10", EditCondition = "bEnableOuterLift", InteractorName = "OuterLift"))
	FChaosClothAssetWeightedValue OuterLift = { true, 0.035f, 1.f, TEXT("OuterLift"), true };

	/*
	 * If true, drag and buoyancy will be used when cloth is submerged under water bodies
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (EditCondition = "!bUsePointBasedWindModel", EditConditionHides, InteractorName = "UseWithWaterBodies"))
	bool bUseWithWaterBodies = false;
	
	/* 
	 * The density of the cloth for buoyancy calculations when submerged in water.
	 * Given in kg/m^3
	 * If you want bodies to be neutrally buoyancy, make this the water density.  Larger values make it sink, smaller values make it
	 * rise to the surface.
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (UIMin = "0", UIMax = "10000", InteractorName = "ClothDensityInWater", EditCondition = "bUseWithWaterBodies && !bUsePointBasedWindModel", EditConditionHides))
	float ClothDensityInWater = 1000.;

	/**
	 * The density of the water where buoyancy is taking effect
	 * The fluid density is given in kg/m^3.
	 * Water density is 1000 kg/m^3
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "10000", InteractorName = "WaterDensity", EditCondition = "bUseWithWaterBodies && !bUsePointBasedWindModel", EditConditionHides))
	float WaterDensity = 1000.;

	/**
	 * Ratio of aerodynamic forces applied as turbulent rather than laminar flow.
	 * Typically, you will want to use turbulent flow for wind and laminar flow for swimming underwater (unless the water is flowing very rapidly).
	 * Turbulent forces scale like velocity squared. Laminar forces scale like velocity.
	 * A value of 0 is appropriate for water.
	 */
	UPROPERTY(EditAnywhere, Category = "Aerodynamics", Meta = (ClampMin = "0", ClampMax = "1", InteractorName = "WaterTurbulenceRatio", EditCondition = "bUseWithWaterBodies && !bUsePointBasedWindModel", EditConditionHides))
	float WaterTurbulenceRatio = 0.f;

	FChaosClothAssetSimulationAerodynamicsConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
