// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "SimulationAnimDriveConfigNode.generated.h"

/** Anim drive properties configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationAnimDriveConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationAnimDriveConfigNode, "SimulationAnimDriveConfig", "Cloth", "Cloth Simulation AnimDrive Config")

public:
	/**
	 * The strength of the constraint driving the cloth towards the animated skinned/goal mesh.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "AnimDrive Properties", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", InteractorName = "AnimDriveStiffness"))
	FChaosClothAssetWeightedValue AnimDriveStiffness = { true, 0.f, 1.f, TEXT("AnimDriveStiffness") };

	/**
	 * The damping amount of the anim drive.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "AnimDrive Properties", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", InteractorName = "AnimDriveDamping"))
	FChaosClothAssetWeightedValue AnimDriveDamping = { true, 0.f, 1.f, TEXT("AnimDriveDamping") };
	
	FChaosClothAssetSimulationAnimDriveConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
