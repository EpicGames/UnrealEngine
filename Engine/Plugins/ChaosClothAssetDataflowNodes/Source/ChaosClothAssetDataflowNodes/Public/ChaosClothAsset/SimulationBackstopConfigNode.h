// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "SimulationBackstopConfigNode.generated.h"

/** Backstop properties configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationBackstopConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationBackstopConfigNode, "SimulationBackstopConfig", "Cloth", "Cloth Simulation Backstop Config")

public:
	/**
	 * The distance from the surface of a backstop collision sphere to its associated particle's skinned position along the mesh normal.
	 * Can be positive or negative depending on the desired effect.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Backstop Properties", Meta = (UIMin = "0", UIMax = "100", InteractorName = "BackstopDistance"))
	FChaosClothAssetWeightedValue BackstopDistance = { true, 0.f, 100.f, TEXT("BackstopDistance") };

	/**
	 * The radius of the backstop sphere that each particle collides with.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Backstop Properties", Meta = (UIMin = "0", UIMax = "100", ClampMin = "0", InteractorName = "BackstopRadius"))
	FChaosClothAssetWeightedValue BackstopRadius = { true, 0.f, 100.f, TEXT("BackstopRadius") };

	/** The name of the accessory mesh to use as the backstop mesh. If none is specified, the default animated mesh will be used */
	UPROPERTY(EditAnywhere, Category = "Backstop Properties")
	FChaosClothAssetConnectableIStringValue BackstopMeshName;


	FChaosClothAssetSimulationBackstopConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
