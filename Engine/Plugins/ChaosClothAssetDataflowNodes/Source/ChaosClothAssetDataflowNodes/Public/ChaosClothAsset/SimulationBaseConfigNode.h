// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosClothAsset/CollectionClothFacade.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "ChaosClothAsset/SimulationConfigNodePropertyTypes.h"
#include "ChaosClothAsset/ImportedValue.h"
#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "SimulationBaseConfigNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/**
 * Base abstract class for all cloth asset config nodes.
 * Inherited class must call RegisterCollectionConnections() in constructor to use this base class Collection.
 */
USTRUCT(meta = (Abstract))
struct FChaosClothAssetSimulationBaseConfigNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()

public:
	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Warn when overriding an existing property in the collection.*/
	UPROPERTY(EditAnywhere, Category = "Simulation Config")
	bool bWarnDuplicateProperty = true;

	FChaosClothAssetSimulationBaseConfigNode() = default;

	CHAOSCLOTHASSETDATAFLOWNODES_API FChaosClothAssetSimulationBaseConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

protected:
	using ECollectionPropertyFlags = ::Chaos::Softs::ECollectionPropertyFlags;  // Short hand for property flag enums

	struct FPropertyHelper
	{
	public:
		FPropertyHelper(const FChaosClothAssetSimulationBaseConfigNode& InConfigNode, UE::Dataflow::FContext& InContext, ::Chaos::Softs::FCollectionPropertyMutableFacade& InProperties, const TSharedRef<FManagedArrayCollection>& InClothCollection);

		template<typename PropertyType UE_REQUIRES(::Chaos::Softs::TIsWeightedType<PropertyType>::Value && !std::is_same_v<PropertyType, bool>)>
		inline int32 SetProperty(const FName& PropertyName, const PropertyType& PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable);

		template<typename PropertyType UE_REQUIRES(::Chaos::Softs::TIsWeightedType<PropertyType>::Value && !std::is_same_v<PropertyType, bool>)>
		inline int32 SetPropertyAndString(const FName& PropertyName, const PropertyType& PropertyValue, const FString& StringValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable);

		template<typename T, typename PropertyType UE_REQUIRES(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T> && ::Chaos::Softs::TIsWeightedType<PropertyType>::Value && !std::is_same_v<PropertyType, bool>)>
		inline int32 SetProperty(const T* ConfigStruct, const PropertyType* PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable);

		CHAOSCLOTHASSETDATAFLOWNODES_API int32 SetPropertyBool(const FName& PropertyName, bool PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable);

		template<typename T UE_REQUIRES(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T>)>
		inline int32 SetPropertyBool(const T* ConfigStruct, const bool* PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable);

		template<typename EnumType UE_REQUIRES(TIsEnumClass<EnumType>::Value)>
		inline int32 SetPropertyEnum(const FName& PropertyName, const EnumType& PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable);

		template<typename T, typename EnumType UE_REQUIRES(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T> && TIsEnumClass<EnumType>::Value)>
		inline int32 SetPropertyEnum(const T* ConfigStruct, const EnumType* PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable);

		int32 SetPropertyString(const FName& PropertyName, const FString& PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable);

		int32 SetPropertyString(const FName& PropertyName, const FChaosClothAssetConnectableIStringValue& PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable);

		template<typename T, typename ConnectableStringValueType UE_REQUIRES(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T>)>
		inline int32 SetPropertyString(const T* ConfigStruct, const ConnectableStringValueType* PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable);
		template<typename T UE_REQUIRES(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T>)>
		inline int32 SetPropertyString(const T* ConfigStruct, const FString* PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable);

		int32 SetPropertyWeighted(const FName& PropertyName, const FVector2f& PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::None);
		int32 SetPropertyWeighted(const FName& PropertyName, const FChaosClothAssetWeightedValue& PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::None);
		int32 SetPropertyWeighted(const FName& PropertyName, const FChaosClothAssetWeightedValueNonAnimatable& PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::None);
		int32 SetPropertyWeighted(const FName& PropertyName, const FChaosClothAssetWeightedValueNonAnimatableNoLowHighRange& PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::None);

		void OverridePropertiesBool(const TArray<FName>& PropertyNames, bool bPropertyValue);
		void OverridePropertiesFloat(const TArray<FName>& PropertyNames, const EChaosClothAssetConstraintOverrideType OverrideType, const float OverrideValue);
		void OverridePropertiesWeighted(const TArray<FName>& PropertyNames, const EChaosClothAssetConstraintOverrideType OverrideType, const FChaosClothAssetWeightedValueOverride& OverrideValue);

		template<typename T, typename WeightedValueType UE_REQUIRES(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T>)>
		inline int32 SetPropertyWeighted(const T* ConfigStruct, const WeightedValueType* PropertyValue, const TArray<FName>& SimilarPropertyNames = {}, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::None);

		FString GetPropertyString(const FString* PropertyReference) const;

		TSharedRef<FManagedArrayCollection> GetClothCollection() const
        {
        	return ClothCollection;
        }

		/** Set an imported solver value onto the property */
		template<typename PropertyType>
		void SetSolverProperty(const FName& PropertyName, const PropertyType& PropertyValue,
			const TFunction<typename PropertyType::ImportedType(UE::Chaos::ClothAsset::FCollectionClothFacade&)>& SolverValueFunction,
			const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::None);

		/** Set an imported averaged fabric value onto the property */
		template<typename PropertyType>
		void SetFabricProperty(const FName& PropertyName, const PropertyType& PropertyValue, 
			const TFunction<typename PropertyType::ImportedType(UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
			const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::None);

		/** Set an imported solver value onto the weighted value property (animatable or not) */
		template<typename PropertyType>
		void SetSolverPropertyWeighted(const FName& PropertyName, const PropertyType& PropertyValue,
			const TFunction<float(const UE::Chaos::ClothAsset::FCollectionClothFacade&)>& SolverValueFunction,
			const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::None);

		/** Set an imported fabric value onto the weighted value property (animatable or not) */
		template<typename PropertyType>
		void SetFabricPropertyWeighted(const FName& PropertyName, const PropertyType& PropertyValue,
			const TFunction<float(const UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
			const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::None);

		template<typename MapType, typename PropertyType>
		void SetFabricPropertyString(
			const FName& PropertyName, const PropertyType& PropertyValue,
			const TFunction<MapType(const UE::Chaos::ClothAsset::FCollectionClothFabricFacade&)>& FabricValueFunction,
			const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::None,
			const FName& GroupName = "");
		
	private:
		
		template<typename T UE_REQUIRES(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T>)>
		inline FName FindPropertyNameByAddress(const T* ConfigStruct, const void* PropertyAddress);
		
		int32 SetPropertyWeighted(const FName& PropertyName, const bool bIsAnimatable, const float& PropertyLow, const float& PropertyHigh, const FString& WeightMap, const TArray<FName>& SimilarPropertyNames, ECollectionPropertyFlags PropertyFlags) const;

		const FChaosClothAssetSimulationBaseConfigNode& ConfigNode;
		UE::Dataflow::FContext& Context;
		::Chaos::Softs::FCollectionPropertyMutableFacade& Properties;
		TSharedRef<FManagedArrayCollection> ClothCollection;
	};

	CHAOSCLOTHASSETDATAFLOWNODES_API virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

	UE_DEPRECATED(5.4, "Use AddProperties(FPropertyHelper&) instead.")
	virtual void AddProperties(UE::Dataflow::FContext& Context, ::Chaos::Softs::FCollectionPropertyMutableFacade& Properties) const {}

	virtual void AddProperties(struct FPropertyHelper& PropertyHelper) const
	PURE_VIRTUAL(FChaosClothAssetSimulationBaseConfigNode::AddProperties, );

	/* Override this to do additional node-specific evaluate on the cloth collection output. AddProperties has already been called when this is called. */
	virtual void EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const {}

	CHAOSCLOTHASSETDATAFLOWNODES_API void RegisterCollectionConnections();

	UE_DEPRECATED(5.4, "Use FPropertyHelper instead.")
	int32 AddPropertyHelper(
		::Chaos::Softs::FCollectionPropertyMutableFacade& Properties,
		const FName& PropertyName,
		bool bIsAnimatable,
		const TArray<FName>& SimilarPropertyNames) const
	{
		using namespace Chaos::Softs;
		const ECollectionPropertyFlags Flags = bIsAnimatable ? ECollectionPropertyFlags::Animatable : ECollectionPropertyFlags::None;
		return AddPropertyHelper(Properties, PropertyName, SimilarPropertyNames, Flags);
	}

	CHAOSCLOTHASSETDATAFLOWNODES_API int32 AddPropertyHelper(
		::Chaos::Softs::FCollectionPropertyMutableFacade& Properties,
		const FName& PropertyName,
		const TArray<FName>& SimilarPropertyNames = {},
		ECollectionPropertyFlags PropertyFlags = ECollectionPropertyFlags::Animatable) const;
};

template<>
struct TStructOpsTypeTraits<FChaosClothAssetSimulationBaseConfigNode> : public TStructOpsTypeTraitsBase2<FChaosClothAssetSimulationBaseConfigNode>
{
	enum
	{
		WithPureVirtual = true,
	};
};

template<typename PropertyType UE_REQUIRES_DEFINITION(::Chaos::Softs::TIsWeightedType<PropertyType>::Value && !std::is_same_v<PropertyType, bool>)>
inline int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetProperty(
	const FName& PropertyName,
	const PropertyType& PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	const int32 PropertyKeyIndex = ConfigNode.AddPropertyHelper(Properties, PropertyName, SimilarPropertyNames, PropertyFlags);
	Properties.SetValue(PropertyKeyIndex, PropertyValue);
	return PropertyKeyIndex;
}

template<typename PropertyType UE_REQUIRES_DEFINITION(::Chaos::Softs::TIsWeightedType<PropertyType>::Value && !std::is_same_v<PropertyType, bool>)>
inline int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyAndString(
	const FName& PropertyName,
	const PropertyType& PropertyValue,
	const FString& StringValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	const int32 PropertyKeyIndex = ConfigNode.AddPropertyHelper(Properties, PropertyName, SimilarPropertyNames, PropertyFlags);
	Properties.SetValue(PropertyKeyIndex, PropertyValue);
	Properties.SetStringValue(PropertyKeyIndex, StringValue);
	return PropertyKeyIndex;
}

template<typename T, typename PropertyType UE_REQUIRES_DEFINITION(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T> && ::Chaos::Softs::TIsWeightedType<PropertyType>::Value && !std::is_same_v<PropertyType, bool>)>
inline int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetProperty(
	const T* ConfigStruct,
	const PropertyType* PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	check(PropertyValue);
	const FName PropertyName = FindPropertyNameByAddress(ConfigStruct, PropertyValue);
	checkf(PropertyName != NAME_None, TEXT("Unknown property."));
	return SetProperty(PropertyName, *PropertyValue, SimilarPropertyNames, PropertyFlags);
}

template<typename T UE_REQUIRES_DEFINITION(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T>)>
inline int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyBool(
	const T* ConfigStruct,
	const bool* PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	check(PropertyValue);
	FName PropertyName = FindPropertyNameByAddress(ConfigStruct, PropertyValue);
	checkf(PropertyName != NAME_None, TEXT("Unknown property."));
	FString PropertyNameString = PropertyName.ToString();
	const bool bStartsWithLowerCaseB = PropertyNameString.RemoveFromStart(TEXT("b"), ESearchCase::CaseSensitive);
	PropertyName = FName(*PropertyNameString);
	checkf(bStartsWithLowerCaseB, TEXT("A C++ Boolean property name must start with a 'b'."));
	return SetPropertyBool(PropertyName, *PropertyValue, SimilarPropertyNames, PropertyFlags);
}

template<typename EnumType UE_REQUIRES_DEFINITION(TIsEnumClass<EnumType>::Value)>
inline int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyEnum(
	const FName& PropertyName,
	const EnumType& PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	const int32 PropertyKeyIndex = ConfigNode.AddPropertyHelper(Properties, PropertyName, SimilarPropertyNames, PropertyFlags);
	Properties.SetValue(PropertyKeyIndex, (int32)PropertyValue);
	return PropertyKeyIndex;
}

template<typename T, typename EnumType UE_REQUIRES_DEFINITION(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T> && TIsEnumClass<EnumType>::Value)>
inline int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyEnum(
	const T* ConfigStruct,
	const EnumType* PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	check(PropertyValue);
	const FName PropertyName = FindPropertyNameByAddress(ConfigStruct, PropertyValue);
	checkf(PropertyName != NAME_None, TEXT("Unknown property."));
	return SetPropertyEnum(PropertyName, *PropertyValue, SimilarPropertyNames, PropertyFlags);
}

template<typename T UE_REQUIRES_DEFINITION(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T>)>
inline int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyString(
	const T* ConfigStruct,
	const FString* PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	check(PropertyValue);
	const FName PropertyName = FindPropertyNameByAddress(ConfigStruct, PropertyValue);
	checkf(PropertyName != NAME_None, TEXT("Unknown property."));
	return SetPropertyString(PropertyName, *PropertyValue, SimilarPropertyNames, PropertyFlags);
}

template<typename T, typename WeightedValueType UE_REQUIRES_DEFINITION(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T>)>
inline int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyWeighted(
	const T* ConfigStruct,
	const WeightedValueType* PropertyValue,
	const TArray<FName>& SimilarPropertyNames,
	ECollectionPropertyFlags PropertyFlags)
{
	check(PropertyValue);
	const FName PropertyName = FindPropertyNameByAddress(ConfigStruct, PropertyValue);
	checkf(PropertyName != NAME_None, TEXT("Unknown property."));
	return SetPropertyWeighted(PropertyName, *PropertyValue, SimilarPropertyNames, PropertyFlags);
}

template<typename T, typename ConnectableStringValueType UE_REQUIRES_DEFINITION(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T>)>
inline int32 FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::SetPropertyString(
	const T* ConfigStruct, 
	const ConnectableStringValueType* PropertyValue,
	const TArray<FName>& SimilarPropertyNames, 
	ECollectionPropertyFlags PropertyFlags)
{
	check(PropertyValue);
	const FName PropertyName = FindPropertyNameByAddress(ConfigStruct, PropertyValue);
	checkf(PropertyName != NAME_None, TEXT("Unknown property."));
	return SetPropertyString(PropertyName, *PropertyValue, SimilarPropertyNames, PropertyFlags);
}

template<typename T UE_REQUIRES_DEFINITION(std::is_base_of_v<FChaosClothAssetSimulationBaseConfigNode, T>)>
inline FName FChaosClothAssetSimulationBaseConfigNode::FPropertyHelper::FindPropertyNameByAddress(const T* ConfigStruct, const void* PropertyAddress)
{
	for (TFieldIterator<FProperty> FieldIt(T::StaticStruct()); FieldIt; ++FieldIt)
	{
		const FProperty* const Property = *FieldIt;
		if (PropertyAddress == Property->ContainerPtrToValuePtr<void>(ConfigStruct))
		{
			return Property->GetFName();
		}
	}
	return NAME_None;
}
