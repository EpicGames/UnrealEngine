// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "ChaosClothAsset/SimulationConfigNodePropertyTypes.h"
#include "SimulationBendingConfigNode.generated.h"

/** Bending constraint property configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationBendingConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationBendingConfigNode, "SimulationBendingConfig", "Cloth", "Cloth Simulation Bending Config")

public:


	/**
	 * Method for calculating the rest angles of the constraints.
	 * Use3DRestAngles: Use the 3D rest mesh angle. This will cause any creases and folds to be baked in the simulation.
	 * FlatnessRatio: Calculate rest angles as a ratio between completely flat and whatever is the 3D rest angle.
	 *   When FlatnessRatio = 0, this is equivalent to "Use3DRestAngles".
	 *   When FlatnessRatio = 1, the rest angle will be 0 (completely flat).
	 *   If a valid weight map is found with the given Weight Map name, then both Low and High values
	 *   are interpolated with the per particle weight to make the final value used for the simulation.
	 *   Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 * RestAngle: 
	 *   Set rest angles to be the explicit value set here (in degrees).
	 *   0 = Flat, Positive values fold away from the edge normal, Negative values fold toward the edge normal.
	 *   When converting vertex weight values to edge values, the value with the smallest absolute value is selected.
	 *   If a valid weight map is found with the given Weight Map name, then both Low and High values
	 *   are interpolated with the per particle weight to make the final value used for the simulation.
	 *   Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Types")
	EChaosClothAssetRestAngleConstructionType RestAngleType = EChaosClothAssetRestAngleConstructionType::Use3DRestAngles;

	/**
	 * Calculate rest angles as a ratio between completely flat and whatever is the 3D rest angle.
	 * When FlatnessRatio = 0, this is equivalent to "Use3DRestAngles".
	 * When FlatnessRatio = 1, the rest angle will be 0 (completely flat).
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", EditCondition = "RestAngleType == EChaosClothAssetRestAngleConstructionType::FlatnessRatio", EditConditionHides))
	FChaosClothAssetWeightedValueNonAnimatable FlatnessRatio = { 0.f, 0.f, TEXT("FlatnessRatio") };

	/**
	 * Set rest angles to be the explicit value set here (in degrees).
	 * 0 = Flat, Positive values fold away from the edge normal, Negative values fold toward the edge normal.
	 * When converting vertex weight values to edge values, the value with the smallest absolute value is selected.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "-180", UIMax = "180", ClampMin = "-180", ClampMax = "180", EditCondition = "RestAngleType == EChaosClothAssetRestAngleConstructionType::RestAngle", EditConditionHides))
	FChaosClothAssetWeightedValueNonAnimatable RestAngle = { 0.f, 0.f, TEXT("RestAngle") };

	/** Constraint solver type */
	UPROPERTY(EditAnywhere, Category = "Bending Types")
	EChaosClothAssetConstraintSolverType SolverType = EChaosClothAssetConstraintSolverType::PBD;
	
	/** Constraint distribution type */
    UPROPERTY(EditAnywhere, Category = "Bending Types", Meta = (EditCondition = "SolverType == EChaosClothAssetConstraintSolverType::XPBD", EditConditionHides))
    EChaosClothAssetConstraintDistributionType DistributionType = EChaosClothAssetConstraintDistributionType::Isotropic;
	
	/** Constraint method type */
	UPROPERTY(EditAnywhere, Category = "Bending Types", Meta = (EditCondition = "SolverType == EChaosClothAssetConstraintSolverType::PBD || DistributionType == EChaosClothAssetConstraintDistributionType::Isotropic", EditConditionHides))
	EChaosClothAssetBendingConstraintType ConstraintType = EChaosClothAssetBendingConstraintType::HingeAngles;
	
	/**
	 * The stiffness of the bending constraints in the warp (vertical) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic && SolverType == EChaosClothAssetConstraintSolverType::XPBD", EditConditionHides, InteractorName = "BendingStiffnessWarp"))
	FChaosClothAssetWeightedValue BendingStiffnessWarp = { true,  100.f, 100.f, TEXT("BendingStiffnessWarp"), true };

	/**
	 * The stiffness of the bending constraints in the weft (horizontal) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic && SolverType == EChaosClothAssetConstraintSolverType::XPBD", EditConditionHides, InteractorName = "BendingStiffnessWeft"))
	FChaosClothAssetWeightedValue BendingStiffnessWeft = { true, 100.f, 100.f, TEXT("BendingStiffnessWeft"),  true };

	/**
	 * The stiffness of the bending constraints in the bias (diagonal) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic && SolverType == EChaosClothAssetConstraintSolverType::XPBD", EditConditionHides, InteractorName = "BendingStiffnessBias"))
	FChaosClothAssetWeightedValue BendingStiffnessBias = { true, 100.f, 100.f, TEXT("BendingStiffnessBias"), true };

	/**
	 * The damping of the bending anisotropic constraints, relative to critical damping.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000"), Meta = (EditCondition = "SolverType == EChaosClothAssetConstraintSolverType::XPBD && DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic", EditConditionHides, InteractorName = "BendingDamping"))
	FChaosClothAssetWeightedValue BendingAnisoDamping = { true, 1.f, 1.f, TEXT("BendingAnisoDamping"), true };

	/**
	 * Once the element has bent such that it's folded more than this ratio from its rest angle ("buckled"), switch to using Buckling Stiffness instead of BendingElement Stiffness.
	 * When Buckling Ratio = 0, the Buckling Stiffness will never be used. When BucklingRatio = 1, the Buckling Stiffness will be used as soon as it's bent past its rest configuration.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", DisplayName = "Aniso Buckling Ratio", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"), Meta = (EditCondition = "SolverType == EChaosClothAssetConstraintSolverType::XPBD && DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic", EditConditionHides, InteractorName = "BucklingRatio"))
	FChaosClothAssetWeightedValue AnisoBucklingRatio= { true, 0.5f, 0.5f, TEXT("BucklingRatio"), true };
	
	/**
	 * The stiffness after buckling in the warp (vertical) direction.
	 * The constraint will use this stiffness instead of element Stiffness once the cloth has buckled, i.e., bent beyond a certain angle.
	 * Typically, Buckling Stiffness is set to be less than BendingElement Stiffness.
	 * Buckling Ratio determines the switch point between using BendingElement Stiffness and Buckling Stiffness.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic  && SolverType == EChaosClothAssetConstraintSolverType::XPBD", EditConditionHides, InteractorName = "BucklingStiffnessWarp"))
	FChaosClothAssetWeightedValue BucklingStiffnessWarp = { true, 50.f, 50.f, TEXT("BucklingStiffnessWarp"), true };

	/**
	 * The stiffness after buckling in the weft (horizontal) direction.
	 * The constraint will use this stiffness instead of element Stiffness once the cloth has buckled, i.e., bent beyond a certain angle.
	 * Typically, Buckling Stiffness is set to be less than BendingElement Stiffness.
	 * Buckling Ratio determines the switch point between using BendingElement Stiffness and Buckling Stiffness.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic && SolverType == EChaosClothAssetConstraintSolverType::XPBD", EditConditionHides, InteractorName = "BucklingStiffnessWeft"))
	FChaosClothAssetWeightedValue BucklingStiffnessWeft = { true, 50.f, 50.f, TEXT("BucklingStiffnessWeft"),  true };

	/**
	 * The stiffness after buckling in the bias (diagonal) direction.
	 * The constraint will use this stiffness instead of element Stiffness once the cloth has buckled, i.e., bent beyond a certain angle.
	 * Typically, Buckling Stiffness is set to be less than BendingElement Stiffness.
	 * Buckling Ratio determines the switch point between using BendingElement Stiffness and Buckling Stiffness.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic && SolverType == EChaosClothAssetConstraintSolverType::XPBD", EditConditionHides, InteractorName = "BucklingStiffnessBias"))
	FChaosClothAssetWeightedValue BucklingStiffnessBias = { true, 50.f, 50.f, TEXT("BucklingStiffnessBias"), true };

	/**
	 * The Stiffness of the bending constraints. Increase the iteration count for stiffer materials. 
	 * Note that PBD stiffnesses will be internally clamped to [0,1].
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000"), Meta = (EditCondition = "((DistributionType == EChaosClothAssetConstraintDistributionType::Isotropic && SolverType == EChaosClothAssetConstraintSolverType::XPBD) || SolverType == EChaosClothAssetConstraintSolverType::PBD)", EditConditionHides, InteractorName = "BendingStiffness"))
	FChaosClothAssetWeightedValue BendingStiffness = { true, 1.f, 1.f, TEXT("BendingStiffness") };

	/**
	 * The damping of the bending constraints, relative to critical damping.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000"), Meta = (EditCondition = "SolverType == EChaosClothAssetConstraintSolverType::XPBD && DistributionType == EChaosClothAssetConstraintDistributionType::Isotropic ", EditConditionHides, InteractorName = "BendingDamping"))
	FChaosClothAssetWeightedValue BendingDamping = { true, 1.f, 1.f, TEXT("BendingDamping") };
	
	/**
	 * The stiffness after buckling.
	 * The constraint will use this stiffness instead of bending Stiffness once the cloth has buckled, i.e., bent beyond a certain angle.
	 * Typically, Buckling Stiffness is set to be less than Bending Stiffness.
	 * Note that PBD stiffnesses will be internally clamped to [0,1].
	 * Buckling Ratio determines the switch point between using BendingElement Stiffness and Buckling Stiffness.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000", EditCondition = "((DistributionType == EChaosClothAssetConstraintDistributionType::Isotropic && SolverType == EChaosClothAssetConstraintSolverType::XPBD) || SolverType == EChaosClothAssetConstraintSolverType::PBD) && ConstraintType == EChaosClothAssetBendingConstraintType::HingeAngles", EditConditionHides, InteractorName = "BucklingStiffness"))
	FChaosClothAssetWeightedValue BucklingStiffness = { true, 0.9f, 0.9f, TEXT("BucklingStiffness") };

	/**
	 * Once the element has bent such that it's folded more than this ratio from its rest angle ("buckled"), switch to using Buckling Stiffness instead of BendingElement Stiffness.
	 * When Buckling Ratio = 0, the Buckling Stiffness will never be used. When BucklingRatio = 1, the Buckling Stiffness will be used as soon as it's bent past its rest configuration.
	 */
	UPROPERTY(EditAnywhere, Category = "Bending Properties", DisplayName = "Buckling Ratio", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"), Meta = (EditCondition = "((DistributionType == EChaosClothAssetConstraintDistributionType::Isotropic && SolverType == EChaosClothAssetConstraintSolverType::XPBD) || SolverType == EChaosClothAssetConstraintSolverType::PBD) && ConstraintType == EChaosClothAssetBendingConstraintType::HingeAngles", EditConditionHides, InteractorName = "BucklingRatio"))
	FChaosClothAssetWeightedValue BucklingRatioWeighted = { true, 0.5f, 0.5f, TEXT("BucklingRatio") };

	FChaosClothAssetSimulationBendingConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Serialize(FArchive& Ar) override;
private:
	// Deprecated properties
#if WITH_EDITORONLY_DATA
	static constexpr float BucklingRatioDeprecatedDefault = 0.5f;
	UPROPERTY()
	float BucklingRatio_DEPRECATED = BucklingRatioDeprecatedDefault;
#endif
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
