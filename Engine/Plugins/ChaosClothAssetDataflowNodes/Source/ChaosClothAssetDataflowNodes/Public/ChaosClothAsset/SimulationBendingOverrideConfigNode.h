// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "ChaosClothAsset/SimulationConfigNodePropertyTypes.h"
#include "SimulationBendingOverrideConfigNode.generated.h"

/** Bending constraint property override configuration node. */
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetSimulationBendingOverrideConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationBendingOverrideConfigNode, "SimulationBendingOverrideConfig", "Cloth", "Cloth Simulation Bending Override Config")

public:
	
	/** Flatness override type.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override")
	EChaosClothAssetConstraintOverrideType OverrideFlatnessRatio = EChaosClothAssetConstraintOverrideType::None;

	/** Flatness override value.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (ClampMin = "0", EditCondition = "OverrideFlatnessRatio != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride FlatnessRatio;

	/** Bending stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override")
	EChaosClothAssetConstraintOverrideType OverrideBendingStiffness = EChaosClothAssetConstraintOverrideType::None;

	/** Bending stiffness override value.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (ClampMin = "0", EditCondition = "OverrideBendingStiffness != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride BendingStiffness;

	/** Whether or not to apply the Bending Stiffness Override to warp, weft, and bias stiffnesses of anisotropic bending elements.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (EditCondition = "OverrideBendingStiffness != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	bool bApplyUniformBendingStiffnessOverride = true;

	/** Whether or not to apply the Bending Stiffness Override to buckling stiffnesses.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (EditCondition = "OverrideBendingStiffness != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	bool bApplyBendingStiffnessOverrideToBuckling = true;

	/** Buckling ratio override type.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override")
	EChaosClothAssetConstraintOverrideType OverrideBucklingRatio = EChaosClothAssetConstraintOverrideType::None;

	/** Buckling ratio override value.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (ClampMin = "0", EditCondition = "OverrideBucklingRatio != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	float BucklingRatio = 1.0;

	/** Buckling stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (EditCondition = "OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyBendingStiffnessOverrideToBuckling", EditConditionHides))
	EChaosClothAssetConstraintOverrideType OverrideBucklingStiffness = EChaosClothAssetConstraintOverrideType::None;

	/** Buckling stiffness override value.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (ClampMin = "0", EditCondition = "(OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyBendingStiffnessOverrideToBuckling) && OverrideBucklingStiffness != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride BucklingStiffness;

	/** Whether or not to apply the Buckling Stiffness Override to warp, weft, and bias stiffnesses of anisotropic bending elements.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (EditCondition = "(OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyBendingStiffnessOverrideToBuckling) && OverrideBucklingStiffness != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	bool bApplyUniformBucklingStiffnessOverride = true;

	/** Warp stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (EditCondition = "OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride", EditConditionHides))
	EChaosClothAssetConstraintOverrideType OverrideBendingStiffnessWarp = EChaosClothAssetConstraintOverrideType::None;

	/** Bending stiffness warp override value.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (ClampMin = "0", EditCondition = "(OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride) && OverrideBendingStiffnessWarp != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride BendingStiffnessWarp;

	/** Weft stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (EditCondition = "OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride", EditConditionHides))
	EChaosClothAssetConstraintOverrideType OverrideBendingStiffnessWeft = EChaosClothAssetConstraintOverrideType::None;

	/** Bending stiffness weft override value.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (ClampMin = "0", EditCondition = "(OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride) && OverrideBendingStiffnessWeft != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride BendingStiffnessWeft;

	/** Bias stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (EditCondition = "OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride", EditConditionHides))
	EChaosClothAssetConstraintOverrideType OverrideBendingStiffnessBias = EChaosClothAssetConstraintOverrideType::None;

	/** Bending stiffness bias override value.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (ClampMin = "0", EditCondition = "(OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride) && OverrideBendingStiffnessBias != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride BendingStiffnessBias;

	/** Warp buckling stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (EditCondition = "(OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride || !bApplyBendingStiffnessOverrideToBuckling) && (OverrideBucklingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBucklingStiffnessOverride)", EditConditionHides))
	EChaosClothAssetConstraintOverrideType OverrideBucklingStiffnessWarp = EChaosClothAssetConstraintOverrideType::None;

	/** Buckling stiffness warp override value.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (ClampMin = "0", EditCondition = "(OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride || !bApplyBendingStiffnessOverrideToBuckling) && (OverrideBucklingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBucklingStiffnessOverride) && OverrideBucklingStiffnessWarp != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride BucklingStiffnessWarp;
	
	/** Weft buckling stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (EditCondition = "(OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride || !bApplyBendingStiffnessOverrideToBuckling) && (OverrideBucklingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBucklingStiffnessOverride)", EditConditionHides))
	EChaosClothAssetConstraintOverrideType OverrideBucklingStiffnessWeft = EChaosClothAssetConstraintOverrideType::None;

	/** Buckling stiffness Weft override value.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (ClampMin = "0", EditCondition = "(OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride || !bApplyBendingStiffnessOverrideToBuckling) && (OverrideBucklingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBucklingStiffnessOverride) && OverrideBucklingStiffnessWeft != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride BucklingStiffnessWeft;

	/** Bias buckling stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (EditCondition = "(OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride || !bApplyBendingStiffnessOverrideToBuckling) && (OverrideBucklingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBucklingStiffnessOverride)", EditConditionHides))
	EChaosClothAssetConstraintOverrideType OverrideBucklingStiffnessBias = EChaosClothAssetConstraintOverrideType::None;

	/** Buckling stiffness Bias override value.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (ClampMin = "0", EditCondition = "(OverrideBendingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBendingStiffnessOverride || !bApplyBendingStiffnessOverrideToBuckling) && (OverrideBucklingStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformBucklingStiffnessOverride) && OverrideBucklingStiffnessBias != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride BucklingStiffnessBias;

	/** Damping override type.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override")
	EChaosClothAssetConstraintOverrideType OverrideBendingDamping = EChaosClothAssetConstraintOverrideType::None;

	/** Bending damping override value.*/
	UPROPERTY(EditAnywhere, Category = "Bending Override", meta = (ClampMin = "0", EditCondition = "OverrideBendingDamping != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride BendingDamping;

	FChaosClothAssetSimulationBendingOverrideConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
