// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "SimulationClothVertexFaceSpringConfigNode.generated.h"


/** Method for generating springs between source vertices and target faces*/
UENUM()
enum struct EChaosClothAssetClothVertexFaceSpringConstructionMethod
{
	// For each source, connect the closest target point
	SourceToClosestTarget,
	// For each source, shoot a ray in Normal direction 
	SourceToRayIntersectionTarget,
	// For each source, find all targets within a radius.
	AllWithinRadius,
	// Create a tet mesh and find corresponding tet face-vertex pairs
	Tetrahedralize,
};

/** Data to procedurally generate ClothVertexFaceSpring Constraints.*/
USTRUCT()
struct FChaosClothAssetSimulationClothVertexFaceSpringConstructionSet
{
	GENERATED_USTRUCT_BODY()

	/** Source Vertex Set*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction")
	FChaosClothAssetConnectableIStringValue SourceVertexSelection = { TEXT("SourceVertices") };

	/** Target Face Set*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction")
	FChaosClothAssetConnectableIStringValue TargetFaceSelection = { TEXT("TargetFaces") };

	/** Construction method used to connect sources and targets */
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction")
	EChaosClothAssetClothVertexFaceSpringConstructionMethod ConstructionMethod = EChaosClothAssetClothVertexFaceSpringConstructionMethod::SourceToRayIntersectionTarget;

	/** Flip normal when doing ray intersection.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction", Meta = (EditCondition ="ConstructionMethod == EChaosClothAssetClothVertexFaceSpringConstructionMethod::SourceToRayIntersectionTarget", EditConditionHides))
	bool bFlipRayNormal = false;

	/** Max ray length for intersection test.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction", Meta = (EditCondition = "ConstructionMethod == EChaosClothAssetClothVertexFaceSpringConstructionMethod::SourceToRayIntersectionTarget", EditConditionHides, ClampMin = "0", UIMax = "100"))
	float MaxRayLength = 100.f;

	/** Radius for search.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction", Meta = (EditCondition = "ConstructionMethod == EChaosClothAssetClothVertexFaceSpringConstructionMethod::AllWithinRadius", EditConditionHides, ClampMin = "0", UIMax = "100"))
	float Radius = 2.f;

	/** Do not consider vertices within this N-ring of connectivity distance.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction", Meta = (EditCondition = "ConstructionMethod == EChaosClothAssetClothVertexFaceSpringConstructionMethod::AllWithinRadius", EditConditionHides, ClampMin = "1", UIMax = "5"))
	int32 DisableNeighborDistance = 2;

	/** Cull zero-volume tets when doing Tetrahedralize.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction", Meta = (EditCondition = "ConstructionMethod == EChaosClothAssetClothVertexFaceSpringConstructionMethod::Tetrahedralize", EditConditionHides))
	bool bSkipZeroVolumeTets = false;

};

/** Node for creating vertex-face constraints and setting their simulation properties.*/
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetSimulationClothVertexFaceSpringConfigNode final : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationClothVertexFaceSpringConfigNode, "SimulationClothVertexFaceSpringConfig", "Cloth", "Cloth Simulation Vertex Face Spring")

public:

	FChaosClothAssetSimulationClothVertexFaceSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override 
	{ 
		return true; 
	}
	virtual bool CanRemovePin() const override 
	{ 
		return ConstructionSets.Num() > NumInitialConstructionSets; 
	}
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface

	//~ Begin FChaosClothAssetSimulationBaseConfigNode interface
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
	virtual void EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const override;
	//~ End FChaosClothAssetSimulationBaseConfigNode interface

	UE::Dataflow::TConnectionReference<FString> GetSourceConnectionReference(int32 Index) const;
	UE::Dataflow::TConnectionReference<FString> GetTargetConnectionReference(int32 Index) const;

	static constexpr int32 NumRequiredInputs = 2; // non-construction set inputs
	static constexpr int32 NumInitialConstructionSets = 1;

	void CreateConstraints(UE::Dataflow::FContext& Context);

	struct FConstructionSetData
	{
		FName SourceSetName;
		FName TargetSetName;
		EChaosClothAssetClothVertexFaceSpringConstructionMethod ConstructionMethod;
		bool bFlipRayNormal;
		float MaxRayLength;
		float Radius;
		int32 DisableNeighborDistance = 2;
		bool bSkipZeroVolumeTets;
	};
	TArray<FConstructionSetData> GetConstructionSetData(UE::Dataflow::FContext& Context) const;

	/** Append to existing set of constraints. Stiffnesses inherited from existing constraints.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Properties")
	bool bAppendToExisting = false;

	/** Treat as tetrahedral repulsion constraints (e.g., for self collisions) rather than spring constraints */
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Properties")
	bool bUseTetRepulsionConstraints = false;

	/** Extension Stiffness is the spring stiffness applied when the spring is currently longer than its rest length. This is a low-high range, but there are currently no ways to author per-spring stiffnesses, so only Low is used in practice.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", InteractorName = "VertexFaceSpringExtensionStiffness", EditCondition = "!bUseTetRepulsionConstraints && !bAppendToExisting", EditConditionHides))
	FVector2f VertexFaceSpringExtensionStiffness = { 100.f, 100.f };

	/** Compression Stiffness is the spring stiffness applied when the spring is currently shorter than its rest length. This is a low-high range, but there are currently no ways to author per-spring stiffnesses, so only Low is used in practice.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", InteractorName = "VertexFaceSpringCompressionStiffness", EditCondition = "!bUseTetRepulsionConstraints && !bAppendToExisting", EditConditionHides))
	FVector2f VertexFaceSpringCompressionStiffness = { 100.f, 100.f };

	/** This damping is the relative to critical damping. This is a low-high range, but there are currently no ways to author per-spring stiffnesses, so only Low is used in practice.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000", InteractorName = "VertexFaceSpringDamping", EditCondition = "!bUseTetRepulsionConstraints && !bAppendToExisting", EditConditionHides))
	FVector2f VertexFaceSpringDamping = { 0.f, 0.f };

	/** Stiffness for repulsion constraints */
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Properties", Meta = (ClampMin = "0", ClampMax = "1", InteractorName = "VertexFaceRepulsionStiffness", EditCondition = "bUseTetRepulsionConstraints && !bAppendToExisting", EditConditionHides))
	float VertexFaceRepulsionStiffness = 0.5f;

	/** Max Number of iterations to apply (per solver iteration). Helps resolve more collisions, but at additional compute cost. */
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Properties", Meta = (ClampMin = "1", UIMax = "5", InteractorName = "VertexFaceRepulsionStiffness", EditCondition = "bUseTetRepulsionConstraints && !bAppendToExisting", EditConditionHides))
	int32 VertexFaceMaxRepulsionIters = 1;

	/** Construction data for procedurally generating constraints.*/
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Cloth Vertex Face Spring Construction", Meta = (SkipInDisplayNameChain))
	TArray<FChaosClothAssetSimulationClothVertexFaceSpringConstructionSet> ConstructionSets;

	/** Use Thickness rather than current rest collection state to determine rest lengths.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction")
	bool bUseThicknessMap = false;

	/** Thickness for calculating rest lengths. Rest length will be combined value of thickness on both end points. */
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction", meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", EditCondition = "bUseThicknessMap", EditConditionHides))
	FChaosClothAssetWeightedValue Thickness = { false, 0.5f, 0.5f, TEXT("SpringThickness") };

	/** Scale applied to the rest lengths of the springs. A value of 1 will preserve the distance in the rest collection. */
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction", Meta = (ClampMin = "0", UIMax = "10", EditCondition = "!bUseThicknessMap", EditConditionHides))
	float RestLengthScale = 1.f;

	/** Click on this button to generate constraints from the construction data. */
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Construction", Meta = (DisplayName = "Generate Constraints"))
	FDataflowFunctionProperty GenerateConstraints;

	/** Raw constraint end point data. Modify at your own risk.*/
	UPROPERTY()
	TArray<int32> SourceVertices;

	/** Raw constraint end point data. Modify at your own risk.*/
	UPROPERTY()
	TArray<FIntVector> TargetVertices;

	/** Raw constraint end point data. Modify at your own risk.*/
	UPROPERTY()
	TArray<FVector3f> TargetWeights;

	/** Raw constraint rest length data. Modify at your own risk.*/
	UPROPERTY()
	TArray<float> RestLengths;
};


