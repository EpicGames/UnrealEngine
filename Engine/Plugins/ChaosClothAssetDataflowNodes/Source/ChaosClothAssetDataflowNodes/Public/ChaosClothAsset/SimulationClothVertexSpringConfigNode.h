// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "SimulationClothVertexSpringConfigNode.generated.h"


/** Method for generating springs between source to target vertices */
UENUM()
enum struct EChaosClothAssetClothVertexSpringConstructionMethod
{
	// For each source, connect the closest target
	SourceToClosestTarget,
	// For each source/target, connect the closest vertex in the other set.
	ClosestSourceToClosestTarget,
	// For each source/target, connect to all vertices in the other set.
	AllSourceToAllTargets,
};

/** Data to procedurally generate ClothVertexSpring Constraints.*/
USTRUCT()
struct FChaosClothAssetSimulationClothVertexSpringConstructionSet
{
	GENERATED_USTRUCT_BODY()

	/** Source Vertex Set*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Spring Construction")
	FChaosClothAssetConnectableIStringValue SourceVertexSelection = { TEXT("SourceVertices") };

	/** Target Vertex Set*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Spring Construction")
	FChaosClothAssetConnectableIStringValue TargetVertexSelection = { TEXT("TargetVertices") };

	/** Construction method used to connect sources and targets */
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Spring Construction")
	EChaosClothAssetClothVertexSpringConstructionMethod ConstructionMethod = EChaosClothAssetClothVertexSpringConstructionMethod::SourceToClosestTarget;
};

/** Node for creating vertex-vertex constraints and setting their simulation properties.*/
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetSimulationClothVertexSpringConfigNode final : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationClothVertexSpringConfigNode, "SimulationClothVertexSpringConfig", "Cloth", "Cloth Simulation Vertex Spring")

public:

	FChaosClothAssetSimulationClothVertexSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override 
	{ 
		return true; 
	}
	virtual bool CanRemovePin() const override 
	{ 
		return ConstructionSets.Num() > NumInitialConstructionSets; 
	}
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface

	//~ Begin FChaosClothAssetSimulationBaseConfigNode interface
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
	virtual void EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const override;
	//~ End FChaosClothAssetSimulationBaseConfigNode interface

	UE::Dataflow::TConnectionReference<FString> GetSourceConnectionReference(int32 Index) const;
	UE::Dataflow::TConnectionReference<FString> GetTargetConnectionReference(int32 Index) const;

	static constexpr int32 NumRequiredInputs = 1; // non-construction set inputs
	static constexpr int32 NumInitialConstructionSets = 1;

	void CreateConstraints(UE::Dataflow::FContext& Context);

	struct FConstructionSetData
	{
		FName SourceSetName;
		FName TargetSetName;
		EChaosClothAssetClothVertexSpringConstructionMethod ConstructionMethod;
	};
	TArray<FConstructionSetData> GetConstructionSetData(UE::Dataflow::FContext& Context) const;

	/** Append to existing set of constraints. Stiffnesses inherited from existing constraints.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Face Spring Properties")
	bool bAppendToExisting = false;

	/** Extension Stiffness is the spring stiffness applied when the spring is currently longer than its rest length. This is a low-high range, but there are currently no ways to author per-spring stiffnesses, so only Low is used in practice.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Spring Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", InteractorName = "VertexSpringExtensionStiffness", EditCondition = "!bAppendToExisting", EditConditionHides))
	FVector2f VertexSpringExtensionStiffness = { 100.f, 100.f };

	/** Compression Stiffness is the spring stiffness applied when the spring is currently shorter than its rest length. This is a low-high range, but there are currently no ways to author per-spring stiffnesses, so only Low is used in practice.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Spring Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", InteractorName = "VertexSpringCompressionStiffness", EditCondition = "!bAppendToExisting", EditConditionHides))
	FVector2f VertexSpringCompressionStiffness = { 100.f, 100.f };

	/** This damping is the relative to critical damping. This is a low-high range, but there are currently no ways to author per-spring stiffnesses, so only Low is used in practice.*/
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Spring Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000", InteractorName = "VertexSpringDamping", EditCondition = "!bAppendToExisting", EditConditionHides))
	FVector2f VertexSpringDamping = { 0.f, 0.f };

	/** Construction data for procedurally generating constraints.*/
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Cloth Vertex Spring Construction", Meta = (SkipInDisplayNameChain))
	TArray<FChaosClothAssetSimulationClothVertexSpringConstructionSet> ConstructionSets;

	/** Scale applied to the rest lengths of the springs. A value of 1 will preserve the distance in the rest collection. */
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Spring Construction", Meta = (ClampMin = "0", UIMax = "10"))
	float RestLengthScale = 1.f;

	/** Click on this button to generate constraints from the construction data. */
	UPROPERTY(EditAnywhere, Category = "Cloth Vertex Spring Construction", Meta = (DisplayName = "Generate Constraints"))
	FDataflowFunctionProperty GenerateConstraints;

	/** Raw constraint end point data. Modify at your own risk.*/
	UPROPERTY(EditAnywhere, Category = "Advanced")
	TArray<FIntVector2> ConstraintVertices;

	/** Raw constraint rest length data. Modify at your own risk.*/
	UPROPERTY(EditAnywhere, Category = "Advanced")
	TArray<float> RestLengths;
};


