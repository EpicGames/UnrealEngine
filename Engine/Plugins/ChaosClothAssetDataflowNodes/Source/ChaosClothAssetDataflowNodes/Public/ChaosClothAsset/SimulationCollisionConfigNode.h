// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "SimulationCollisionConfigNode.generated.h"

/** Physics mesh collision properties configuration node. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationCollisionConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_BODY()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationCollisionConfigNode, "SimulationCollisionConfig", "Cloth", "Cloth Simulation Collision Config")

public:
	/** The added thickness of collision shapes. */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", DisplayName = "Collision Thickness", Meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", InteractorName = "CollisionThickness"))
	FChaosClothAssetImportedFloatValue CollisionThicknessImported = {UE::Chaos::ClothAsset::FDefaultFabric::CollisionThickness};

	/** Friction coefficient for cloth - collider interaction. Currently only Skinned Triangle Meshes use the weighted value. All other collisions only use the Low value. */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", DisplayName = "Friction Coefficient", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "10", InteractorName = "FrictionCoefficient"))
	FChaosClothAssetWeightedValue FrictionCoefficientWeighted = 
	{ true, UE::Chaos::ClothAsset::FDefaultFabric::Friction, UE::Chaos::ClothAsset::FDefaultFabric::Friction, TEXT("FrictionCoefficient"), true };

	/** Enable colliding against any simple (e.g., capsules, convexes, spheres, boxes) colliders..*/
	UPROPERTY(EditAnywhere, Category = "Collision Properties", Meta = (InteractorName = "EnableSimpleColliders"))
	bool bEnableSimpleColliders = true;

	/** Use Planar Constraints for simple (e.g., capsules, convexes, spheres, boxes) colliders when doing multiple iterations. Planar constraints are cheaper than full collision detection, but less accurate. */
	UPROPERTY(EditAnywhere, Category="Collision Properties", Meta = (EditCondition = "bEnableSimpleColliders", InteractorName = "UsePlanarConstraintForSimpleColliders"))
	bool bUsePlanarConstraintForSimpleColliders = false;

	/** Enable colliding against any complex (e.g., SkinnedLevelSet, MLLevelSet) colliders.*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Collision Properties", Meta = (InteractorName = "EnableComplexColliders"))
	bool bEnableComplexColliders = true;

	/** Use Planar Constraints for complex (e.g., SkinnedLevelSet, MLLevelSet) colliders when doing multiple iterations. Planar constraints are cheaper than full collision detection, but less accurate. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Collision Properties", Meta = (EditCondition = "bEnableComplexColliders", InteractorName = "UsePlanarConstraintForComplexColliders"))
	bool bUsePlanarConstraintForComplexColliders = true;

	/** Enable colliding against any Skinned Triangle Mesh colliders.*/
	UPROPERTY(EditAnywhere, Category = "Collision Properties", Meta = (InteractorName = "EnableSkinnedTriangleMeshCollisions"))
	bool bEnableSkinnedTriangleMeshCollisions = true;

	/** Use 'NumSelfCollisionSubsteps' (Located on SimulationSolverConfig) to also control Skinned Triangle Mesh collision updates */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", Meta = (EditCondition = "bEnableSkinnedTriangleMeshCollisions", InteractorName = "UseSelfCollisionSubstepsForSkinnedTriangleMeshes"))
	bool bUseSelfCollisionSubstepsForSkinnedTriangleMeshes = true;

	/** Thickness added to the cloth when colliding against collision shapes. */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", DisplayName = "Cloth Collision Thickness", Meta = (InteractorName = "ClothCollisionThickness"))
	FChaosClothAssetWeightedValue ClothCollisionThicknessImported = { true, UE::Chaos::ClothAsset::FDefaultFabric::ClothCollisionThickness,
		UE::Chaos::ClothAsset::FDefaultFabric::ClothCollisionThickness, TEXT("ClothCollisionThickness"), true };

	/** Stiffness for proximity repulsion forces (Force-based solver only). Units = kg cm/ s^2 (same as XPBD springs)*/
	UE_DEPRECATED(5.8, "This property is not used.")
	UPROPERTY()
	float ProximityStiffness = 100.f;

	/**
	 * Use continuous collision detection (CCD) to prevent any missed collisions between fast moving particles and colliders.
	 * This has a negative effect on performance compared to when resolving collision without using CCD.
	 */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", Meta = (InteractorName = "UseCCD"))
	bool bUseCCD = false;

	/**
	 * Experimental feature, use at your own risk.
	 * Improve collision behavior by running the CCD code path, but only where particles penetration exceeds the given inner thickness value.
	 * This is using a faster collision code path than CCD, but unlike CCD it does not resolve fast moving objects.
	 * Use an inner thickness that is about a few centimeters to start with, up to a quarter of your collider diameter for best results.
	 * If the inner thickness is too large (e.g. more than half the collider's diameter), the CCD code path won't ever be activated.
	 * This property can be used in conjunction with a weight map to selectively allow certain particles to use the extra calculations or to locally adjust the effect.
	 * Set both the low and high values to 0 to disable.
	 */
	UPROPERTY(EditAnywhere, Category = "Collision Properties", DisplayName = "Inner Collision Thickness (Experimental)", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", EditCondition = "bUseCCD == false"))
	FChaosClothAssetWeightedValueNonAnimatable InnerCollisionThickness = { 0.f, 0.f, TEXT("InnerCollisionThickness"), false };

	UE_DEPRECATED(5.6, "Use FrictionCoefficientWeighted instead.")
	UPROPERTY()
	FChaosClothAssetImportedFloatValue FrictionCoefficientImported = { UE::Chaos::ClothAsset::FDefaultFabric::Friction };

	UE_DEPRECATED(5.7, "Use ClothCollisionThicknessImported instead.")
	UPROPERTY(Meta = (DataflowSkipConnection))
	FChaosClothAssetWeightedValue ClothCollisionThickness = { true, 0.f, 0.f, TEXT("ClothCollisionThickness"), true };

	FChaosClothAssetSimulationCollisionConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Serialize(FArchive& Ar) override;

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;

	// Deprecated properties
#if WITH_EDITORONLY_DATA
	static constexpr float CollisionThicknessDeprecatedDefault = 1.0f;
	UPROPERTY()
	float CollisionThickness_DEPRECATED = CollisionThicknessDeprecatedDefault;

	static constexpr float FrictionCoefficientDeprecatedDefault = 0.8f;
	UPROPERTY()
	float FrictionCoefficient_DEPRECATED  = FrictionCoefficientDeprecatedDefault;
#endif
};
