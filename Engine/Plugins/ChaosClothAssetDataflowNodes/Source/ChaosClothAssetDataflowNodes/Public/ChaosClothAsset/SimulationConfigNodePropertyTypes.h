// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "Chaos/PBDBendingConstraintsBase.h"
#include "SimulationConfigNodePropertyTypes.generated.h"

UENUM()
enum class EChaosClothAssetRestAngleConstructionType : uint8
{
	/** Calculate rest angles using the 3D draped space simulation mesh. */
	Use3DRestAngles = (uint8)Chaos::Softs::FPBDBendingConstraintsBase::ERestAngleConstructionType::Use3DRestAngles,
	/** Calculate rest angles using the FlatnessRatio property. */
	FlatnessRatio = (uint8)Chaos::Softs::FPBDBendingConstraintsBase::ERestAngleConstructionType::FlatnessRatio,
	/** Calculate rest angles using the RestAngle property. */
	RestAngle = (uint8)Chaos::Softs::FPBDBendingConstraintsBase::ERestAngleConstructionType::ExplicitRestAngles
};

UENUM()
enum class EChaosClothAssetBendingConstraintType : uint8
{
	/** Add a spring in between the opposite vertices of the adjacent faces */
	FacesSpring,
	/** Add an angular constraint in between the 2 faces */
	HingeAngles,
};

UENUM()
enum class EChaosClothAssetConstraintDistributionType : uint8
{
	/** Having a separate stiffness along the warp, weft, bias direction */
	Anisotropic,
	/** Uniform stiffness in all directions */
	Isotropic,
};

UENUM()
enum class EChaosClothAssetConstraintSolverType : uint8
{
	/** Physically based PBD with compliance */
	XPBD,
	/** PBD style with no damping/stiffness */
	PBD,
};

UENUM()
enum class EChaosClothAssetConstraintOverrideType : uint8
{
	/** Do not override */
	None,
	/** Override any existing values with this new value.*/
	Override,
	/** Multiply any existing values with this new value.*/
	Multiply
};
