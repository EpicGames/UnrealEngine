// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "Chaos/SoftsSimulationSpace.h"
#include "SimulationDampingConfigNode.generated.h"

/** Damping properties configuration node.*/
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationDampingConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationDampingConfigNode, "SimulationDampingConfig", "Cloth", "Cloth Simulation Damping Config")

public:
	/**
	 * The amount of global damping applied to the cloth velocities, also known as point damping.
	 * Point damping improves simulation stability, but can also cause an overall slow-down effect and therefore is best left to very small percentage amounts.
	 */
	UPROPERTY(EditAnywhere, DisplayName = "Damping Coefficient", Category = "Damping Properties", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", InteractorName = "DampingCoefficient"))
	FChaosClothAssetWeightedValue DampingCoefficientWeighted = {true, 0.01f, 0.01f, TEXT("DampingCoefficient")};

	/**
	* The space in which local damping is calculated. Center of mass adds the expense of calculating the center of mass.
	*/
	UPROPERTY(EditAnywhere, Category = "Damping Properties", Meta = (InteractorName = "LocalDampingSpace"))
	EChaosSoftsLocalDampingSpace LocalDampingSpace = EChaosSoftsLocalDampingSpace::CenterOfMass;

	/**
	 * The amount of local linear damping applied to the cloth velocities.
	 * This type of damping only damps individual deviations of the particles velocities from the global linear motion.
	 * It makes the cloth deformations more cohesive and reduces jitter without affecting the overall movement.
	 * It can also produce synchronization artifacts where part of the cloth mesh are disconnected (which might well be desirable, or not).
	 */
	UPROPERTY(EditAnywhere, Category = "Damping Properties", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", InteractorName = "LocalDampingLinearCoefficient"))
	float LocalDampingLinearCoefficient = 0.f;

	/**
	 * The amount of local angular damping applied to the cloth velocities.
	 * This type of damping only damps individual deviations of the particles velocities from the global angular motion.
	 * It makes the cloth deformations more cohesive and reduces jitter without affecting the overall movement.
	 * It can also produce synchronization artifacts where part of the cloth mesh are disconnected (which might well be desirable, or not).
	 */
	UPROPERTY(EditAnywhere, Category = "Damping Properties", Meta = (EditCondition = "bEnableLocalDampingAngular", UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", InteractorName = "LocalDampingAngularCoefficient"))
	float LocalDampingAngularCoefficient = 0.f;

	FChaosClothAssetSimulationDampingConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Serialize(FArchive& Ar) override;
private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;

	// Deprecated properties
#if WITH_EDITORONLY_DATA
	static constexpr float DeprecatedDampingCoefficientValue = -1.f; // This is outside the settable range when it wasn't deprecated.
	UPROPERTY()
	float DampingCoefficient_DEPRECATED = DeprecatedDampingCoefficientValue;

	UPROPERTY()
	float LocalDampingCoefficient_DEPRECATED = DeprecatedDampingCoefficientValue;
#endif 

};
