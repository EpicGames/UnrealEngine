// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "SimulationDefaultConfigNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class UChaosClothConfig;
class UChaosClothSharedSimConfig;

/** Add default simulation properties to the cloth collection in the format of the skeletal mesh cloth editor. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationDefaultConfigNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationDefaultConfigNode, "SimulationDefaultConfig", "Cloth", "Cloth Simulation Default Config")

public:
	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Cloth Simulation Properties. */
	UPROPERTY(EditAnywhere, Instanced, NoClear, Category = "Simulation Default Config")
	TObjectPtr<UChaosClothConfig> SimulationConfig;

	/** Cloth Shared Simulation Properties. */
	UPROPERTY(EditAnywhere, Instanced, NoClear, Category = "Simulation Default Config")
	TObjectPtr<UChaosClothSharedSimConfig> SharedSimulationConfig;

	FChaosClothAssetSimulationDefaultConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode Interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual void Serialize(FArchive& Ar) override;
	//~ End FDataflowNode Interface

	UObject* OwningObject;  // For backward compatibility when null SimulationConfig and SharedSimulationConfig are reloaded
};
