// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "SimulationGravityConfigNode.generated.h"

/** Gravity properties configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationGravityConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationGravityConfigNode, "SimulationGravityConfig", "Cloth", "Cloth Simulation Gravity Config")

public:
	/** Use the config gravity value instead of world gravity. */
	UPROPERTY(EditAnywhere, Category = "Gravity Properties", Meta = (InteractorName = "UseGravityOverride"))
	bool bUseGravityOverride = false;

	/** Scale factor applied to the world gravity and also to the clothing simulation interactor gravity. Does not affect the gravity if set using the override below. */
	UPROPERTY(EditAnywhere, Category = "Gravity Properties", DisplayName = "Gravity Scale", Meta = (UIMin = "0", UIMax = "10", EditCondition = "!bUseGravityOverride", InteractorName = "GravityScale"))
	FChaosClothAssetWeightedValue GravityScaleWeighted = {true, 1.f, 1.f, TEXT("GravityScale")};

	/** The gravitational acceleration vector [cm/s^2]. */
	UPROPERTY(EditAnywhere, Category = "Gravity Properties", DisplayName = "Gravity Override", Meta = (UIMin = "0", UIMax = "10", EditCondition = "bUseGravityOverride", InteractorName = "GravityOverride"))
	FChaosClothAssetImportedVectorValue GravityOverrideImported = {UE::Chaos::ClothAsset::FDefaultSolver::Gravity};  // TODO: Should we make this a S.I. unit?
	
	FChaosClothAssetSimulationGravityConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Serialize(FArchive& Ar) override;

private:

	// Deprecated properties
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float GravityScale_DEPRECATED = 1.f;

	UPROPERTY()
	FVector3f GravityOverride_DEPRECATED = UE::Chaos::ClothAsset::FDefaultSolver::Gravity;  // TODO: Should we make this a S.I. unit?
#endif

	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
