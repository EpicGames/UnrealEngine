// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "SimulationLongRangeAttachmentConfigNode.generated.h"

USTRUCT()
struct FChaosClothAssetTetherGenerationSet
{
	GENERATED_BODY()

	/**
	* Set of Kinematic vertices that tethers will attach to. Will be intersected with (node level) FixedEndSet.
	*/
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment Properties")
	FChaosClothAssetConnectableIStringValue CustomFixedEndSet = { TEXT("KinematicVertices3D") };

	/**
	* Set of Dynamic vertices that will attach to the closest vertex in FixedEndSelection. (Node level) FixedEndSet will be excluded. 
	*/
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment Properties")
	FChaosClothAssetConnectableIStringValue CustomDynamicEndSet = { TEXT("DynamicVertices3D") };
};

/** Long range attachment constraint property configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2 : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2, "SimulationLongRangeAttachmentConfig", "Cloth", "Cloth Simulation Long Range Attachment Config")

public:

	/**
	 * The tethers' stiffness of the long range attachment constraints.
	 * The long range attachment connects each of the cloth particles to its closest fixed point with a spring constraint.
	 * This can be used to compensate for a lack of stretch resistance when the iterations count is kept low for performance reasons.
	 * Can lead to an unnatural pull string puppet like behavior.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment Properties", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", InteractorName = "TetherStiffness"))
	FChaosClothAssetWeightedValue TetherStiffness = { true, 1.f, 1.f, TEXT("TetherStiffness") };

	/**
	 * The limit scale of the long range attachment constraints (aka tether limit).
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	*/
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment Properties", Meta = (UIMin = "1.", UIMax = "1.1", ClampMin = "0.01", ClampMax = "10", InteractorName = "TetherScale"))
	FChaosClothAssetWeightedValue TetherScale = { true, 1.f, 1.f, TEXT("TetherScale") };

	/**
	 * Use geodesic instead of euclidean distance calculations for the Long Range Attachment constraint,
	 * which is slower at setup but more accurate at establishing the correct position and length of the tethers,
	 * and therefore is less prone to artifacts during the simulation.
	 */
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment Properties")
	bool bUseGeodesicTethers = true;

	/**
	 * Enable more granular control over tether generation via the custom selection sets CustomTetherData.
	 * When using custom tether generation, FixedEndSet is still required to contain all kinematic vertices.
	 * It is also not possible to provide CustomFixedEndSet indices that are not part of the FixedEndSet.
	 * Similarly, it is not possible to provide CustomDynamicEndSet indices that are part of the FixedEndSet.
	 * Any fixed tether end provided to a CustomFixedEndSet must be directly connected to a dynamic particles
	 * group (island) where at least one of the CustomDynamicEndSet particles resides in order to be valid.
	 * When enabling this option, the default tether generation is completely disabled (dynamic particles are
	 * no longer connected to their closest kinematic vertex as defined by FixedEndSet).
	 */
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment Properties")
	bool bEnableCustomTetherGeneration = false;

	/**
	 * The name of the vertex selection set used as fixed tether ends.
	 * When using custom tether generation, this set is still needed to contain all kinematic vertices.
	 */
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment Properties")
	FChaosClothAssetConnectableIStringValue FixedEndSet = { TEXT("KinematicVertices3D") };

	/** Pairs of vertex selections used for custom tether generation. See EnableCustomTetherGeneration. */
	UPROPERTY(EditAnywhere, EditFixedSize, Category = "Long Range Attachment Properties", Meta = (EditCondition = "bEnableCustomTetherGeneration", SkipInDisplayNameChain))
	TArray<FChaosClothAssetTetherGenerationSet> CustomTetherData;

	FChaosClothAssetSimulationLongRangeAttachmentConfigNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;

	virtual void EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const override;

	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return true; }
	virtual bool CanRemovePin() const override { return CustomTetherData.Num() > NumInitialCustomTetherSets; }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;

	UE::Dataflow::TConnectionReference<FString> GetFixedEndConnectionReference(int32 Index) const;
	UE::Dataflow::TConnectionReference<FString> GetDynamicEndConnectionReference(int32 Index) const;

	static constexpr int32 NumRequiredInputs = 4; // non-filter set inputs
	static constexpr int32 NumInitialCustomTetherSets = 1;
};

/** Long range attachment constraint property configuration node. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.5"))
struct UE_DEPRECATED(5.5, "Use the newer version of this node instead.") FChaosClothAssetSimulationLongRangeAttachmentConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationLongRangeAttachmentConfigNode, "SimulationLongRangeAttachmentConfig", "Cloth", "Cloth Simulation Long Range Attachment Config")

public:

	/**
	 * The tethers' stiffness of the long range attachment constraints.
	 * The long range attachment connects each of the cloth particles to its closest fixed point with a spring constraint.
	 * This can be used to compensate for a lack of stretch resistance when the iterations count is kept low for performance reasons.
	 * Can lead to an unnatural pull string puppet like behavior.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment Properties", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", InteractorName = "TetherStiffness"))
	FChaosClothAssetWeightedValue TetherStiffness = { true, 1.f, 1.f, TEXT("TetherStiffness") };

	/**
	 * The limit scale of the long range attachment constraints (aka tether limit).
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	*/
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment Properties", Meta = (UIMin = "1.", UIMax = "1.1", ClampMin = "0.01", ClampMax = "10", InteractorName = "TetherScale"))
	FChaosClothAssetWeightedValue TetherScale = { true, 1.f, 1.f, TEXT("TetherScale") };

	/**
	 * Use geodesic instead of euclidean distance calculations for the Long Range Attachment constraint,
	 * which is slower at setup but more accurate at establishing the correct position and length of the tethers,
	 * and therefore is less prone to artifacts during the simulation.
	 */
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment Properties")
	bool bUseGeodesicTethers = true;

	/** The name of the weight map used to calculate fixed tether ends. All vertices with weight = 0 will be considered fixed. */
	UPROPERTY(EditAnywhere, Category = "Long Range Attachment Properties", Meta = (DataflowInput))
	FString FixedEndWeightMap = TEXT("MaxDistance");

	FChaosClothAssetSimulationLongRangeAttachmentConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;

	virtual void EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const override;
};
