// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ClothConfig.h"  // For EClothMassMode
#include "SimulationMassConfigNode.generated.h"

/** Mass properties configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationMassConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationMassConfigNode, "SimulationMassConfig", "Cloth", "Cloth Simulation Mass Config")

public:
	/**
	 * How cloth particle mass is determined
	 * -	Uniform Mass: Every particle's mass will be set to the value specified in the UniformMass setting. Mostly to avoid as it can cause some serious issues with irregular tessellations.
	 * -	Total Mass: The total mass is distributed equally over all the particles. Useful when referencing a specific garment size and feel.
	 * -	Density: A constant mass density is used. Density is usually the preferred way of setting mass since it allows matching real life materials' density values.
	 */
	UPROPERTY(EditAnywhere, Category = "Mass Properties")
	EClothMassMode MassMode = EClothMassMode::Density;

	/** The value used when the Mass Mode is set to Uniform Mass. */
	UPROPERTY(EditAnywhere, Category = "Mass Properties", DisplayName = "Uniform Mass", Meta = (UIMin = "0.000001", UIMax = "0.001", ClampMin = "0", EditCondition = "MassMode == EClothMassMode::UniformMass", EditConditionHides))
	FChaosClothAssetWeightedValueNonAnimatable UniformMassWeighted = {0.00015f, 0.00015f, TEXT("UniformMass")};

	/** The value used when Mass Mode is set to TotalMass. */
	UPROPERTY(EditAnywhere, Category = "Mass Properties", Meta = (UIMin = "0.001", UIMax = "10", ClampMin = "0", EditCondition = "MassMode == EClothMassMode::TotalMass", EditConditionHides))
	float TotalMass = 0.5f;

	/**
	 * Density in kg/m^2.
	 * Melton Wool: 0.7
	 * Heavy leather: 0.6
	 * Polyurethane: 0.5
	 * Denim: 0.4
	 * Light leather: 0.3
	 * Cotton: 0.2
	 * Silk: 0.1
	 */
	UPROPERTY(EditAnywhere, Category = "Mass Properties", DisplayName = "Density", meta = (UIMin = "0.001", UIMax = "1", ClampMin = "0", EditCondition = "MassMode == EClothMassMode::Density", EditConditionHides))
	FChaosClothAssetWeightedValueNonAnimatable DensityWeighted = {0.35f, 0.35f, TEXT("Density"),  true};

	/** Calculated particle masses will be clamped to this minimum value (or 1e-8, whichever is larger). */
	UPROPERTY(EditAnywhere, Category = "Mass Properties", meta = (ClampMin = "0"))
	float MinPerParticleMass = 0.0001f;

	FChaosClothAssetSimulationMassConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Serialize(FArchive& Ar) override;

private:

	// Deprecated properties
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float UniformMass_DEPRECATED = 0.00015f;

	UPROPERTY()
	float Density_DEPRECATED = 0.35f;
#endif

	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
