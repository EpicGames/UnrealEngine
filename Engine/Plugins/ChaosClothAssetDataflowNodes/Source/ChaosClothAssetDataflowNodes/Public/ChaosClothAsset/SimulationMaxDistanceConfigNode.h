// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "SimulationMaxDistanceConfigNode.generated.h"

/** Maximum distance constraint property configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationMaxDistanceConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationMaxDistanceConfigNode, "SimulationMaxDistanceConfig", "Cloth", "Cloth Simulation MaxDistance Config")

public:
	/**
	 * The maximum distance a simulated particle can reach from its animated skinned cloth mesh position.
	 * If a particle has 0 for its maximum distance, it is no longer considered dynamic, and becomes kinematic to follow its animated position.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "MaxDistance Properties", Meta = (UIMin = "0", UIMax = "100", ClampMin = "0", InteractorName = "MaxDistance"))
	FChaosClothAssetWeightedValue MaxDistance = { true, 0.f, 100.f, TEXT("MaxDistance") };

	/**
	 * Selection set of SimVertices3D that should be made kinematic regardless of the MaxDistance map value.
	 */
	UPROPERTY(VisibleAnywhere, Category = "MaxDistance Properties")
	FChaosClothAssetConnectableIStringValue InKinematic = { TEXT("") };

	/**
	 * The name of the kinematic vertex set generated by this node.
	 * This is the union of InKinematic and any vertices which are below the solver max distance threshold.
	 * The name of this set cannot be changed and is only provided for further tweaking.
	 */
	UPROPERTY(VisibleAnywhere, Category = "MaxDistance Properties", Meta = (DataflowOutput))
	FString KinematicVertices3D;

	FChaosClothAssetSimulationMaxDistanceConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
	virtual void EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const override;

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;

#if WITH_EDITOR
	virtual bool CanDebugDraw() const override;
	virtual bool CanDebugDrawViewMode(const FName& ViewModeName) const override;
	virtual void DebugDraw(UE::Dataflow::FContext& Context, IDataflowDebugDrawInterface& DataflowRenderingInterface, const FDebugDrawParameters& DebugDrawParameters) const override;
#endif
};
