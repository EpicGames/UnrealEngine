// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "SimulationMorphTargetConfigNode.generated.h"

/** The active sim morph target.
*/
USTRUCT()
struct FChaosClothAssetMorphTargetSelection
{
	GENERATED_BODY()

	/** Morph target name */
	UPROPERTY(EditAnywhere, Category ="Simulation Morph Target Properties", meta = (InteractorName = "ActiveMorphTarget"))
	FString Name;
	/** Morph target weight */
	UPROPERTY(EditAnywhere, Category = "Simulation Morph Target Properties", meta = (InteractorName = "ActiveMorphTarget"))
	float Weight = 0.f;
};

/** Simulation Morph Target configuration node. This node is necessary to set sim morph targets (e.g., via BP interactor)*/
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationMorphTargetConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationMorphTargetConfigNode, "SimulationMorphTargetConfig", "Cloth", "Cloth Simulation Morph Target Config")

public:
	/**
	 * The name of the active sim morph target
	 */
	UPROPERTY(EditAnywhere, Category = "Simulation Morph Target Properties", meta = (InteractorName = "ActiveMorphTarget"))
	FChaosClothAssetMorphTargetSelection ActiveMorphTarget;

	FChaosClothAssetSimulationMorphTargetConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
