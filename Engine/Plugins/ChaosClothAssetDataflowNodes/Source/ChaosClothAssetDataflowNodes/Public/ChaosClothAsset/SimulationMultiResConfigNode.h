// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "SimulationMultiResConfigNode.generated.h"

/** Experimental Solver multires configuration node. */
USTRUCT(Meta = (DataflowCloth, Deprecated=5.8, Experimental))
struct UE_DEPRECATED(5.8, "MultiRes Constraints have been deprecated.")FChaosClothAssetSimulationMultiResConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationMultiResConfigNode, "SimulationMultiResConfig", "Cloth", "Cloth Simulation MultiRes Config")

public:

	/** Enable multi-res simulation for this LOD.*/
	UPROPERTY(EditAnywhere, Category = "Multi-resolution", meta = (EditCondition = "!bIsCoarseLOD"))
	bool bIsFineLOD = false;

	/** This LOD is a coarse LOD for a finer LOD's multi-res simulation.*/
	UPROPERTY(EditAnywhere, Category = "Multi-resolution", meta = (EditCondition = "!bIsFineLOD", InteractorName = "IsCoarseMultiResLOD"))
	bool bIsCoarseMultiResLOD = false;

	/** Index of the coarse LOD for multi-res simulation. That LOD also needs a Multi Res Config node with "Is Coarse Multi Res LOD" = true.*/
	UPROPERTY(EditAnywhere, Category = "Multi-resolution", meta = (EditCondition = "bIsFineLOD", EditConditionHides, InteractorName = "MultiResCoarseLODIndex"))
	int32 MultiResCoarseLODIndex = INDEX_NONE;

	/** Use XPBD-style constraints */
	UPROPERTY(EditAnywhere, Category = "Multi-resolution", meta = (EditCondition = "bIsFineLOD", EditConditionHides, InteractorName = "MultiResUseXPBD"))
	bool bMultiResUseXPBD = false;

	/** MultiRes Spring Stiffness */
	UPROPERTY(EditAnywhere, Category = "Multi-resolution", meta = (UIMin = "0", UIMax = "100000", ClampMin = "0", ClampMax = "100000", EditCondition = "bIsFineLOD", EditConditionHides, InteractorName = "MultiResStiffness"))
	FChaosClothAssetWeightedValue MultiResStiffness = {true, 1.f, 1.f};

	/** MultiRes Velocity Target Spring Stiffness (non-XPBD only) */
	UPROPERTY(EditAnywhere, Category = "Multi-resolution", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", EditCondition = "bIsFineLOD && !bMultiResUseXPBD", EditConditionHides, InteractorName = "MultiResVelocityTargetStiffness"))
	FChaosClothAssetWeightedValue MultiResVelocityTargetStiffness = { true, 1.f, 1.f };

	FChaosClothAssetSimulationMultiResConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};