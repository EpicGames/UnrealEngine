// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "SimulationPBDAreaSpringConfigNode.generated.h"

/** Area spring constraint property configuration node. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.4"))
struct FChaosClothAssetSimulationPBDAreaSpringConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationPBDAreaSpringConfigNode, "SimulationPBDAreaSpringConfig", "Cloth", "Cloth Simulation PBD Area Spring Config")

public:
	/**
	 * The stiffness of the surface area preservation constraints.
	 * Increase the solver iteration count for stiffer materials.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "PBDAreaSpring Properties", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	FChaosClothAssetWeightedValue AreaSpringStiffness = { true, 1.f, 1.f, TEXT("AreaSpringStiffness") };

	FChaosClothAssetSimulationPBDAreaSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
