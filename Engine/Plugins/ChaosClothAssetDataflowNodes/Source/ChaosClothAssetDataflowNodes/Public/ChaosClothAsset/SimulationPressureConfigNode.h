// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "SimulationPressureConfigNode.generated.h"

/** Pressure properties configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationPressureConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationPressureConfigNode, "SimulationPressureConfig", "Cloth", "Cloth Simulation Pressure Config")

public:
	/**
	 * Pressure force strength applied in the normal direction(use negative value to push toward backface)
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Pressure Properties", meta = (UIMin = "-10", UIMax = "10", ClampMin = "-100", ClampMax = "100", InteractorName = "Pressure"))
	FChaosClothAssetWeightedValue Pressure = { true, 0.0f, 1.f, TEXT("Pressure"),true };

	FChaosClothAssetSimulationPressureConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
