// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "SimulationResolveExtremeDeformationConfigNode.generated.h"

/** Resolve extreme deformation properties configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationResolveExtremeDeformationConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationResolveExtremeDeformationConfigNode, "SimulationResolveExtremeDeformationConfig", "Cloth", "Cloth Simulation Resolve ExtremeDeformation Config")

public:
	/** Vertex Selection to check for extreme deformations */
	UPROPERTY(EditAnywhere, Category = "Input Selection")
	FChaosClothAssetConnectableIStringValue InputSelection = { TEXT("InputSelection") };

	/** Edges deformed above this threshold will trigger position reset. */
	UPROPERTY(EditAnywhere, Category = "Edge Ratio Threshold Properties", meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", InteractorName = "ExtremeDeformationEdgeRatioThreshold"))
	float ExtremeDeformationEdgeRatioThreshold = 5.f;

	/**
	 * The name of the extreme deformation vertex set generated by this node.
	 * The name of this set cannot be changed and is only provided for further tweaking.
	 */
	UPROPERTY(VisibleAnywhere, Category = "Output Selection Name", Meta = (DataflowOutput))
	FString ExtremeDeformationVertexSelection = GET_MEMBER_NAME_STRING_CHECKED(FChaosClothAssetSimulationResolveExtremeDeformationConfigNode, ExtremeDeformationVertexSelection);

	FChaosClothAssetSimulationResolveExtremeDeformationConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;

	virtual void EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
