// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "SimulationSelfCollisionConfigNode.generated.h"


/** Self-collision repulsion forces (point-face) properties configuration node. 
 *  Note that the kinematic collider has been deprecated in favor of SkinnedTriangleMesh Physics Asset bodies */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationSelfCollisionConfigNode_v2 final : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationSelfCollisionConfigNode_v2, "SimulationSelfCollisionConfig", "Cloth", "Cloth Simulation Self Collision Config")

public:

	FChaosClothAssetSimulationSelfCollisionConfigNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** Activating this node will enable self collisions.*/
	UPROPERTY(VisibleAnywhere, Category = "Self-Collision Properties", meta = (InteractorName = "UseSelfCollisions"))
	bool bUseSelfCollisions = true;

	/** The self collision offset per side. Total thickness of cloth is 2x this value. */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", InteractorName = "SelfCollisionThickness"))
	FChaosClothAssetWeightedValue SelfCollisionThickness = { true, UE::Chaos::ClothAsset::FDefaultFabric::SelfCollisionThickness,
		UE::Chaos::ClothAsset::FDefaultFabric::SelfCollisionThickness, TEXT("SelfCollisionThickness"), true };

	/** The stiffness of the springs used to control self collision. */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", InteractorName = "SelfCollisionStiffness"))
	float SelfCollisionStiffness = 0.5f;

	/** Friction coefficient for cloth - cloth interaction. */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", InteractorName = "SelfCollisionFriction"))
	FChaosClothAssetImportedFloatValue SelfCollisionFriction = { UE::Chaos::ClothAsset::FDefaultFabric::SelfFriction };

	/** Disabled neighbor collision ring. Collisions are disabled between vertices within this N-ring connectivity distance.*/
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (UIMin = "1", UIMax = "5", ClampMin = "1", ClampMax = "10"))
	int32 SelfCollisionDisableNeighborDistance = 5;

	/** Self collision layers face int map. Generate this map using the SelectionsToIntMap node with SimFace Selections.
	* Faces labeled with -1 will collide normally without any layering behavior.
	* Faces labeled with any other number will keep higher layer numbers outside lower layer numbers (outside = front facing normal direction).
	*/
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (InteractorName = "SelfCollisionLayers"))
	FChaosClothAssetConnectableIStringValue SelfCollisionLayers = { TEXT("SelfCollisionLayers"), true };

	/** Sim face selection set of faces which should not self collide */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (InteractorName = "SelfCollisionDisabledFaces"))
	FChaosClothAssetConnectableIStringValue SelfCollisionDisabledFaces = { TEXT("SelfCollisionDisabledFaces") };

	/** Enable self intersection resolution. This will try to fix any cloth intersections that are not handled by collision repulsions. Can be expensive.*/
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (InteractorName = "UseSelfIntersections"))
	bool bUseSelfIntersections = true;

	/** Do global intersection analysis to determine the correct normals for the collision springs */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (EditCondition = "bUseSelfIntersections", InteractorName = "UseGlobalIntersectionAnalysis"))
	bool bUseGlobalIntersectionAnalysis = true;

	/** Do a step of contour minimization at the beginning of the timestep. 
	 * Contour minimization will attempt to resolve intersections by shortening the intersection edge. Helpful with open intersections that global intersection analysis can't fix.*/
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (EditCondition = "bUseSelfIntersections", InteractorName = "UseContourMinimization"))
	bool bUseContourMinimization = true;

	/** Number of post timestep contour minimization steps to do. (Very Expensive!)
	 * Contour minimization will attempt to resolve intersections by shortening the intersection edge.Helpful with open intersections that global intersection analysis can't fix.*/
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (ClampMin = "0", EditCondition = "bUseSelfIntersections", InteractorName = "NumContourMinimizationPostSteps"))
	int32 NumContourMinimizationPostSteps = 0;

	/** Use global contour gradients when doing post timestep contour minimization */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (EditCondition = "bUseSelfIntersections && NumContourMinimizationPostSteps > 0", InteractorName = "UseGlobalPostStepContours"))
	bool bUseGlobalPostStepContours = true;
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};

/** Self-collision repulsion forces (point-face) properties configuration node. */
USTRUCT(Meta = (DataflowCloth, Deprecated = 5.7))
struct  UE_DEPRECATED(5.7, "Use the newer version of this node instead.") FChaosClothAssetSimulationSelfCollisionConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationSelfCollisionConfigNode, "SimulationSelfCollisionConfig", "Cloth", "Cloth Simulation Self Collision Config")

public:
	/** Activating this node will enable self collisions.*/
	UPROPERTY(VisibleAnywhere, Category = "Self-Collision Properties", meta = (InteractorName = "UseSelfCollisions"))
	bool bUseSelfCollisions = true;

	/** The self collision offset per side. Total thickness of cloth is 2x this value. */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", InteractorName = "SelfCollisionThickness"))
	FChaosClothAssetWeightedValue SelfCollisionThicknessWeighted = {true, UE::Chaos::ClothAsset::FDefaultFabric::SelfCollisionThickness,
		UE::Chaos::ClothAsset::FDefaultFabric::SelfCollisionThickness, TEXT("SelfCollisionThickness"), true};

	/** The stiffness of the springs used to control self collision (PBD Solver). */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", EditCondition = "!bSelfCollideAgainstKinematicCollidersOnly", InteractorName = "SelfCollisionStiffness"))
	float SelfCollisionStiffness = 0.5f;

	/** Friction coefficient for cloth - cloth interaction. */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", DisplayName = "Self Collision Friction", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", EditCondition = "!bSelfCollideAgainstKinematicCollidersOnly", InteractorName = "SelfCollisionFriction"))
	FChaosClothAssetImportedFloatValue SelfCollisionFrictionImported = {UE::Chaos::ClothAsset::FDefaultFabric::SelfFriction};

	/** Disabled neighbor collision ring. Collisions are disabled between vertices within this N-ring connectivity distance.*/
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (UIMin = "1", UIMax = "5", ClampMin = "1", EditCondition = "!bSelfCollideAgainstKinematicCollidersOnly"))
	int32 SelfCollisionDisableNeighborDistance = 5;

	/** Self collision layers face int map. Generate this map using the SelectionsToIntMap node with SimFace Selections.
	* Faces labeled with -1 will collide normally without any layering behavior.
	* Faces labeled with any other number will keep higher layer numbers outside lower layer numbers (outside = front facing normal direction).
	*/
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (EditCondition = "!bSelfCollideAgainstKinematicCollidersOnly", InteractorName = "SelfCollisionLayers"))
	FChaosClothAssetConnectableIStringValue SelfCollisionLayers = { TEXT("SelfCollisionLayers"), true };

	/** Sim face selection set of faces which should not self collide */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties", meta = (InteractorName = "SelfCollisionDisabledFaces"))
	FChaosClothAssetConnectableIStringValue SelfCollisionDisabledFaces = { TEXT("SelfCollisionDisabledFaces") };
	
	/** Collide only against kinematic colliders (no dynamic self collisions). Kinematic colliders do not do Self Intersections. They always collide against the front-face.*/
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties - Kinematic Colliders", meta = (InteractorName = "SelfCollideAgainstKinematicCollidersOnly"))
	bool bSelfCollideAgainstKinematicCollidersOnly = false;

	/** Sim face selection set of kinematic faces which should self collide. Kinematic colliders do not do Self Intersections. They always collide against the front-face. */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties - Kinematic Colliders", meta = (EditCondition = "!bSelfCollideAgainstAllKinematicVertices", InteractorName = "SelfCollisionEnabledKinematicFaces"))
	FChaosClothAssetConnectableIStringValue SelfCollisionEnabledKinematicFaces = { TEXT("SelfCollisionEnabledKinematicFaces") };

	/** Thickness of kinematic colliders. Total offset between cloth and kinematic colliders is SelfCollisionThickness + SelfCollisionKinematicColliderThickness. */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties - Kinematic Colliders", meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", InteractorName = "SelfCollisionKinematicColliderThickness"))
	float SelfCollisionKinematicColliderThickness = 0.f;

	/** The stiffness of the springs used to control self collision (PBD Solver). */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties - Kinematic Colliders", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", InteractorName = "SelfCollisionKinematicColliderStiffness"))
	float SelfCollisionKinematicColliderStiffness = 1.f;

	/** Friction coefficient for cloth - kinematic cloth interaction. Weight map is on the dynamic cloth, not the collider.*/
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties - Kinematic Colliders", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", InteractorName = "SelfCollisionKinematicColliderFriction"))
	FChaosClothAssetWeightedValue SelfCollisionKinematicColliderFrictionWeighted = {true, 0.0f, 0.f, TEXT("SelfCollisionKinematicColliderFriction")};

	/** Self collide against all kinematic vertices. Kinematic colliders do not do Self Intersections. They always collide against the front-face.*/
	UPROPERTY(EditAnywhere, Category = "Self-Collision Properties - Kinematic Colliders", meta = (InteractorName = "SelfCollideAgainstAllKinematicVertices"))
	bool bSelfCollideAgainstAllKinematicVertices = false;

	/** Enable self intersection resolution. This will try to fix any cloth intersections that are not handled by collision repulsions. */
	UPROPERTY(EditAnywhere, Category = "Experimental", meta = (EditCondition = "!bSelfCollideAgainstKinematicCollidersOnly", InteractorName = "UseSelfIntersections"))
	bool bUseSelfIntersections = false;

	/** Do global intersection analysis to determine the correct normals for the collision springs */
	UPROPERTY(EditAnywhere, Category = "Experimental", meta = (EditCondition = "bUseSelfIntersections && !bSelfCollideAgainstKinematicCollidersOnly", InteractorName = "UseGlobalIntersectionAnalysis"))
	bool bUseGlobalIntersectionAnalysis = true;

	/** Do a step of contour minimization at the beginning of the timestep. */
	UPROPERTY(EditAnywhere, Category = "Experimental", meta = (EditCondition = "bUseSelfIntersections && !bSelfCollideAgainstKinematicCollidersOnly", InteractorName = "UseContourMinimization"))
	bool bUseContourMinimization = true;

	/** Number of post timestep contour minimization steps to do. (Expensive!)*/
	UPROPERTY(EditAnywhere, Category = "Experimental", meta = (ClampMin = "0", EditCondition = "bUseSelfIntersections && !bSelfCollideAgainstKinematicCollidersOnly", InteractorName = "NumContourMinimizationPostSteps"))
	int32 NumContourMinimizationPostSteps = 0;

	/** Use global contour gradients when doing post timestep contour minimization */
	UPROPERTY(EditAnywhere, Category = "Experimental", meta = (EditCondition = "bUseSelfIntersections && NumContourMinimizationPostSteps > 0 && !bSelfCollideAgainstKinematicCollidersOnly", InteractorName = "UseGlobalPostStepContours"))
	bool bUseGlobalPostStepContours = true;

	/** The stiffness of the proximity repulsions used to control self collision (Force-based Solver). Units = kg cm/ s^2 (same as XPBD springs) */
	UPROPERTY(EditAnywhere, Category = "Experimental", meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "!bSelfCollideAgainstAllKinematicVertices", InteractorName = "SelfCollisionProximityStiffness"))
	float SelfCollisionProximityStiffness = 1.f;

	FChaosClothAssetSimulationSelfCollisionConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Serialize(FArchive& Ar) override;
private:
	// Deprecated properties
#if WITH_EDITORONLY_DATA
	static constexpr float FrictionDeprecatedValue = -1.f;
	UPROPERTY()
	float SelfCollisionKinematicColliderFriction_DEPRECATED = FrictionDeprecatedValue;

	static constexpr float SelfFrictionDeprecatedValue = 0.0f;
	UPROPERTY()
	float SelfCollisionFriction_DEPRECATED = SelfFrictionDeprecatedValue;

	static constexpr float SelfCollisionThicknessDeprecatedValue = -1.f;
	UPROPERTY()
	float SelfCollisionThickness_DEPRECATED = SelfCollisionThicknessDeprecatedValue;
#endif
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
