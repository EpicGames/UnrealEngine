// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "SimulationSelfCollisionSpheresConfigNode.generated.h"

/** Self-collision spheres properties configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationSelfCollisionSpheresConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationSelfCollisionSpheresConfigNode, "SimulationSelfCollisionSpheresConfig", "Cloth", "Cloth Simulation Self Collision Spheres Config")

public:
	/** The radius of the spheres used in self collision centered at each vertex. */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Spheres Properties", meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", InteractorName = "SelfCollisionSphereRadius"))
	float SelfCollisionSphereRadius = 0.5f;

	/** The stiffness of the springs used to control self collision. */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Spheres Properties", meta = (UIMin = "0", UIMax = "100", ClampMin = "0", ClampMax = "1000", InteractorName = "SelfCollisionSphereStiffness"))
	float SelfCollisionSphereStiffness = 1.f;

	/** 
	 * Multiplier for culling the spheres generated by this node. Spheres are seeded on every vertex, 
	 * and culled based on SelfCollisionSphereRadius * SelfCollisionSphereRadiusCullMultiplier.
	 */
	UPROPERTY(EditAnywhere, Category = "Self-Collision Spheres Properties", meta = (ClampMin = "1"))
	float SelfCollisionSphereRadiusCullMultiplier = 1.f;

	/** The name of the output selection generated from the unculled vertices that receives the collision spheres. */
	UPROPERTY(VisibleAnywhere, Category = "Self-Collision Spheres Properties", Meta = (DataflowOutput))
	FString SelfCollisionSphereSetName = GET_MEMBER_NAME_STRING_CHECKED(FChaosClothAssetSimulationSelfCollisionSpheresConfigNode, SelfCollisionSphereSetName);

	FChaosClothAssetSimulationSelfCollisionSpheresConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;

	virtual void EvaluateClothCollection(UE::Dataflow::FContext& Context, const TSharedRef<FManagedArrayCollection>& ClothCollection) const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
