// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "SimulationSolverConfigNode.generated.h"

/** Solver properties configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationSolverConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationSolverConfigNode, "SimulationSolverConfig", "Cloth", "Cloth Simulation Solver Config")

public:
	/**
	 * The number of time step dependent solver iterations. This sets iterations at 60fps.
	 * NumIterations can never be bigger than MaxNumIterations.
	 * At lower fps up to MaxNumIterations may be used instead. At higher fps as low as one single iteration might be used.
	 * Higher number of iterations will increase the stiffness of all constraints and improve convergence, but will also increase the CPU cost of the simulation.
	 */
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (UIMin = "1", UIMax = "10", ClampMin = "0", ClampMax = "100", InteractorName = "NumIterations"))
	int32 NumIterations = 1;

	/**
	 * The maximum number of solver iterations.
	 * This is the upper limit of the number of iterations set in solver, when the frame rate is lower than 60fps.
	 */
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (UIMin = "1", UIMax = "10", ClampMin = "1", ClampMax = "100", InteractorName = "MaxNumIterations"))
	int32 MaxNumIterations = 6;

	/**
	 * The number of solver substeps.
	 * This will increase the precision of the collision inputs and help with constraint resolutions but will increase the CPU cost.
	 */
	UPROPERTY(EditAnywhere, Category = Simulation, DisplayName = "Num Substeps", meta = (UIMin = "1", UIMax = "10", ClampMin = "1", ClampMax = "100", InteractorName = "NumSubsteps"))
	FChaosClothAssetImportedIntValue NumSubstepsImported = {UE::Chaos::ClothAsset::FDefaultSolver::SubSteps};

	/**
	 * Enable dynamic substepping.
	 */
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (InlineEditConditionToggle))
	bool bEnableDynamicSubstepping = false;

	/**
	 * Choose the number of substeps based on a target substep delta time in milliseconds. Substeps are clamped to [1, NumSubsteps]. 
	 */
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (UIMin = "1", UIMax = "30", ClampMin = "0", ClampMax = "1000", EditCondition = "bEnableDynamicSubstepping", InteractorName = "DynamicSubstepDeltaTime"))
	float DynamicSubstepDeltaTime = 16.67f;

	/**
	* Enable setting separate SelfCollisionSubsteps. Otherwise, self collisions will be detected every substep.
	*/
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (InteractorName = "EnableNumSelfCollisionSubsteps"))
	bool bEnableNumSelfCollisionSubsteps = false;

	/**
	 * Set a separate number of self collision substeps. Lower this number to increase speed at the expense of lower self collision accuracy.
	 * Actual value always clamped between [1, NumSubsteps].
	 */
	UPROPERTY(EditAnywhere, Category = Simulation, meta = (UIMin = "1", UIMax = "10", ClampMin = "1", ClampMax = "100", EditCondition = "bEnableNumSelfCollisionSubsteps", InteractorName = "NumSelfCollisionSubsteps"))
	int32 NumSelfCollisionSubsteps = 1;

	FChaosClothAssetSimulationSolverConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Serialize(FArchive& Ar) override;

private:

	// Deprecated properties
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	int32 NumSubsteps_DEPRECATED = UE::Chaos::ClothAsset::FDefaultSolver::SubSteps;

	UPROPERTY()
	bool bEnableForceBasedSolver_DEPRECATED = false;

	UPROPERTY()
	int32 NumNewtonIterations_DEPRECATED = 0;

	UPROPERTY()
	int32 MaxNumCGIterations_DEPRECATED = 50;

	UPROPERTY()
	float CGResidualTolerance_DEPRECATED = 1e-4f;

	UPROPERTY()
	bool bDoQuasistatics_DEPRECATED = false;
#endif
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
