// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "ChaosClothAsset/SimulationConfigNodePropertyTypes.h"
#include "SimulationStretchConfigNode.generated.h"

/** Stretching constraint property configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationStretchConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationStretchConfigNode, "SimulationStretchConfig", "Cloth", "Cloth Simulation Stretching Config")

public:
	/** Whether to use the 3D draped space as rest lengths, or use the 2D pattern space instead. */
	UPROPERTY(EditAnywhere, Category = "Stretch Types")
	bool bStretchUse3dRestLengths = true;

	/** Constraint solver type.  */
	UPROPERTY(EditAnywhere, Category = "Stretch Types")
	EChaosClothAssetConstraintSolverType SolverType = EChaosClothAssetConstraintSolverType::PBD;

	/**  Constraint distribution type. */
	UPROPERTY(EditAnywhere, Category = "Stretch Types", Meta = (EditCondition = "SolverType == EChaosClothAssetConstraintSolverType::XPBD", EditConditionHides))
	EChaosClothAssetConstraintDistributionType DistributionType = EChaosClothAssetConstraintDistributionType::Isotropic;
	
	/** Add an area constraint in case of isotropic distribution  */
	UPROPERTY(EditAnywhere, Category = "Stretch Types", Meta = (EditCondition = "((SolverType == EChaosClothAssetConstraintSolverType::XPBD && DistributionType == EChaosClothAssetConstraintDistributionType::Isotropic) || SolverType == EChaosClothAssetConstraintSolverType::PBD)", EditConditionHides))
	bool bAddAreaConstraint = true;
	
	/**
	 * The stiffness of the stretch constraints in the warp (vertical) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Stretch Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic && SolverType == EChaosClothAssetConstraintSolverType::XPBD", EditConditionHides, InteractorName = "StretchStiffnessWarp"))
	FChaosClothAssetWeightedValue StretchStiffnessWarp = { true, 100.f, 100.f, TEXT("StretchStiffnessWarp"),true };

	/**
	 * The stiffness of the stretch constraints in the weft (horizontal) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Stretch Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic && SolverType == EChaosClothAssetConstraintSolverType::XPBD", EditConditionHides, InteractorName = "StretchStiffnessWeft"))
	FChaosClothAssetWeightedValue StretchStiffnessWeft = { true, 100.f, 100.f, TEXT("StretchStiffnessWeft"), true };

	/**
	 * The stiffness of the stretch constraints in the bias (diagonal) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Stretch Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic && SolverType == EChaosClothAssetConstraintSolverType::XPBD", EditConditionHides, InteractorName = "StretchStiffnessBias"))
	FChaosClothAssetWeightedValue StretchStiffnessBias = { true, 100.f, 100.f, TEXT("StretchStiffnessBias"), true };

	/**
	 * 
	 * The damping of the stretch anisotropic constraints, relative to critical damping.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Stretch Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000", EditCondition = "SolverType == EChaosClothAssetConstraintSolverType::XPBD && DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic", EditConditionHides, InteractorName = "StretchDamping"))
	FChaosClothAssetWeightedValue StretchAnisoDamping = { true, 1.f, 1.f, TEXT("StretchAnisoDamping"), true };
	
	/**
     * The stiffness of the stretch constraints. Note that PBD stiffnesses will be internally clamped to [0,1].
     * If a valid weight map is found with the given Weight Map name, then both Low and High values
     * are interpolated with the per particle weight to make the final value used for the simulation.
     * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
     */
    UPROPERTY(EditAnywhere, Category = "Stretch Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "1000000000", EditCondition = "(SolverType == EChaosClothAssetConstraintSolverType::PBD || DistributionType == EChaosClothAssetConstraintDistributionType::Isotropic)", EditConditionHides, InteractorName = "StretchStiffness"))
    FChaosClothAssetWeightedValue StretchStiffness = { true, 1.f, 1.f, TEXT("StretchStiffness") };
	
	/**
	 * 
	 * The damping of the stretch constraints, relative to critical damping.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Stretch Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000", EditCondition = "SolverType == EChaosClothAssetConstraintSolverType::XPBD && DistributionType == EChaosClothAssetConstraintDistributionType::Isotropic", EditConditionHides, InteractorName = "StretchDamping"))
	FChaosClothAssetWeightedValue StretchDamping = { true, 1.f, 1.f, TEXT("StretchDamping") };

	/**
	* Enable StretchWarp and WeftScale.
	* This adds a small amount of memory and computational cost, so this is optional for the least expensive constraint type (PBD)
	*/
	UPROPERTY(EditAnywhere, Category = "Stretch Properties", Meta = (EditCondition = "SolverType == EChaosClothAssetConstraintSolverType::PBD", EditConditionHides))
	bool bEnableStretchWarpAndWeftScale = false;

	/**
	 * The scale of the stretch constraints at rest in the warp (vertical) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Stretch Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "10000000", EditCondition = "(SolverType == EChaosClothAssetConstraintSolverType::XPBD && DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic) || (bEnableStretchWarpAndWeftScale && SolverType == EChaosClothAssetConstraintSolverType::PBD)", EditConditionHides, InteractorName = "StretchWarpScale"))
	FChaosClothAssetWeightedValue StretchWarpScale = { true, 1.f, 1.f, TEXT("StretchWarpScale") };

	/**
	 * The scale of the stretch constraints at rest in the weft (horizontal) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Stretch Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "10000000", EditCondition = "(SolverType == EChaosClothAssetConstraintSolverType::XPBD && DistributionType == EChaosClothAssetConstraintDistributionType::Anisotropic) || (bEnableStretchWarpAndWeftScale && SolverType == EChaosClothAssetConstraintSolverType::PBD)", EditConditionHides, InteractorName = "StretchWeftScale"))
	FChaosClothAssetWeightedValue StretchWeftScale = { true, 1.f, 1.f, TEXT("StretchWeftScale") };

	/**
	 * The stiffness of the surface area preservation constraints. Note that PBD stiffnesses will be internally clamped to [0,1].
	 * Increase the solver iteration count for stiffer materials.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "Stretch Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "bAddAreaConstraint == true && ((SolverType == EChaosClothAssetConstraintSolverType::XPBD && DistributionType == EChaosClothAssetConstraintDistributionType::Isotropic) || SolverType == EChaosClothAssetConstraintSolverType::PBD)", EditConditionHides, InteractorName = "AreaStiffness"))
	FChaosClothAssetWeightedValue AreaStiffness = { true, 1.f, 1.f, TEXT("AreaStiffness") };

	FChaosClothAssetSimulationStretchConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
