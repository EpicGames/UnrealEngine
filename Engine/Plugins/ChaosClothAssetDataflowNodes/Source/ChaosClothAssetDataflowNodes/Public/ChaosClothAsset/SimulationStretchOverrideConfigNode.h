// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "ChaosClothAsset/SimulationConfigNodePropertyTypes.h"
#include "SimulationStretchOverrideConfigNode.generated.h"

/** Stretching constraint property override configuration node. */
USTRUCT(Meta = (DataflowCloth, Experimental))
struct FChaosClothAssetSimulationStretchOverrideConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationStretchOverrideConfigNode, "SimulationStretchOverrideConfig", "Cloth", "Cloth Simulation Stretching Override Config")

public:

	/** Enable overriding Stretch Use 3d Rest Lengths */
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (InlineEditConditionToggle))
	bool bOverrideStretchUse3dRestLengths = false;

	/** Whether to use the 3D draped space as rest lengths, or use the 2D pattern space instead. */
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (EditCondition = "bOverrideStretchUse3dRestLengths"))
	bool bStretchUse3dRestLengths = true;

	/** Stretch stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override")
	EChaosClothAssetConstraintOverrideType OverrideStretchStiffness = EChaosClothAssetConstraintOverrideType::None;

	/** Stretch stiffness override value.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (ClampMin = "0", EditCondition = "OverrideStretchStiffness != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride StretchStiffness;

	/** Whether or not to apply the Stretch Stiffness Override to warp, weft, and bias stiffnesses of anisotropic springs.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (EditCondition = "OverrideStretchStiffness != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	bool bApplyUniformStretchStiffnessOverride = true;

	/** Warp stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (EditCondition = "OverrideStretchStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformStretchStiffnessOverride", EditConditionHides))
	EChaosClothAssetConstraintOverrideType OverrideStretchStiffnessWarp = EChaosClothAssetConstraintOverrideType::None;

	/** Stretch stiffness warp override value.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (ClampMin = "0", EditCondition = "(OverrideStretchStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformStretchStiffnessOverride) && OverrideStretchStiffnessWarp != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride StretchStiffnessWarp;

	/** Weft stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (EditCondition = "OverrideStretchStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformStretchStiffnessOverride", EditConditionHides))
	EChaosClothAssetConstraintOverrideType OverrideStretchStiffnessWeft = EChaosClothAssetConstraintOverrideType::None;

	/** Stretch stiffness weft override value.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (ClampMin = "0", EditCondition = "(OverrideStretchStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformStretchStiffnessOverride) && OverrideStretchStiffnessWeft != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride StretchStiffnessWeft;

	/** Bias stiffness override type.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (EditCondition = "OverrideStretchStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformStretchStiffnessOverride", EditConditionHides))
	EChaosClothAssetConstraintOverrideType OverrideStretchStiffnessBias = EChaosClothAssetConstraintOverrideType::None;

	/** Stretch stiffness bias override value.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (ClampMin = "0", EditCondition = "(OverrideStretchStiffness == EChaosClothAssetConstraintOverrideType::None || !bApplyUniformStretchStiffnessOverride) && OverrideStretchStiffnessBias != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride StretchStiffnessBias;

	/** Damping override type.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override")
	EChaosClothAssetConstraintOverrideType OverrideStretchDamping = EChaosClothAssetConstraintOverrideType::None;

	/** Stretch damping override value.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (ClampMin = "0", EditCondition = "OverrideStretchDamping != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride StretchDamping;

	/** Warp scale override type.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override")
	EChaosClothAssetConstraintOverrideType OverrideWarpScale = EChaosClothAssetConstraintOverrideType::None;

	/** Stretch damping override value.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (ClampMin = "0", EditCondition = "OverrideWarpScale != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride WarpScale;

	/** Weft scale override type.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override")
	EChaosClothAssetConstraintOverrideType OverrideWeftScale = EChaosClothAssetConstraintOverrideType::None;

	/** Stretch damping override value.*/
	UPROPERTY(EditAnywhere, Category = "Stretch Override", meta = (ClampMin = "0", EditCondition = "OverrideWeftScale != EChaosClothAssetConstraintOverrideType::None", EditConditionHides))
	FChaosClothAssetWeightedValueOverride WeftScale;


	FChaosClothAssetSimulationStretchOverrideConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
