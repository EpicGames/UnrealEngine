// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Chaos/SoftsSimulationSpace.h"
#include "ChaosCloth/ChaosClothingSimulationSolver.h"
#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "SimulationVelocityScaleConfigNode.generated.h"

/** Velocity scale properties configuration node. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSimulationVelocityScaleConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationVelocityScaleConfigNode, "SimulationVelocityScaleConfig", "Cloth", "Cloth Simulation Velocity Scale Config")

public:

	/**
	 * All vector properties on this node (e.g., Linear Velocity Scale, Max Linear Acceleration)
	 * will be evaluated in this space. 
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (InteractorName = "VelocityScaleSpace"))
	EChaosSoftsSimulationSpace VelocityScaleSpace = EChaosSoftsSimulationSpace::ReferenceBoneSpace;

	/**
	 * The amount of linear velocities sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 * This value will be clamped by "Max Velocity Scale". A velocity scale of > 1 will amplify the velocities from the reference bone.
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "100", InteractorName = "LinearVelocityScale"))
	FVector3f LinearVelocityScale = { 0.75f, 0.75f, 0.75f };

	/**
	 * Enable linear velocity clamping.
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (InlineEditConditionToggle))
	bool bEnableLinearVelocityClamping = false;

	/**
	 * The maximum amount of linear velocity sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (ClampMin = "0", EditCondition = "bEnableLinearVelocityClamping", InteractorName = "MaxLinearVelocity"))
	FVector3f MaxLinearVelocity = { 1000.f, 1000.f, 1000.f }; // Approx 22mph or 36kph per direction

	/**
	 * Enable linear acceleration clamping.
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (InlineEditConditionToggle))
	bool bEnableLinearAccelerationClamping = false;

	/**
	 * The maximum amount of linear acceleration sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (ClampMin = "0", EditCondition = "bEnableLinearAccelerationClamping", InteractorName = "MaxLinearAcceleration"))
	FVector3f MaxLinearAcceleration = { 60000.f, 60000.f, 60000.f };

	/**
	 * The amount of angular velocities sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 * This value will be clamped by "Max Velocity Scale". A velocity scale of > 1 will amplify the velocities from the reference bone.
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "100", InteractorName = "AngularVelocityScale"))
	float AngularVelocityScale = 0.75f;

	/**
	 * Enable angular velocity clamping.
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (InlineEditConditionToggle))
	bool bEnableAngularVelocityClamping = false;

	/**
	 * The maximum amount of angular velocity sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (ClampMin = "0", EditCondition = "bEnableAngularVelocityClamping", InteractorName = "MaxAngularVelocity"))
	float MaxAngularVelocity = 200.f;

	/**
	 * Enable angular acceleration clamping.
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (InlineEditConditionToggle))
	bool bEnableAngularAccelerationClamping = false;

	/**
	 * The maximum amount of angular acceleration sent to the local cloth space from the reference bone
	 * (the closest bone to the root on which the cloth section has been skinned, or the root itself if the cloth isn't skinned).
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (ClampMin = "0", EditCondition = "bEnableAngularAccelerationClamping", InteractorName = "MaxAngularAcceleration"))
	float MaxAngularAcceleration = 12000.f;

	/**
	 * Clamp on Linear and Angular Velocity Scale. The final velocity scale (e.g., including contributions from blueprints) will be clamped to this value.
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "100", InteractorName = "MaxVelocityScale"))
	float MaxVelocityScale = 1.f;

	/**
	 * Fictitious forces model.
	 * Decide how fictitious forces (e.g. centrifugal force) are applied to the simulation.
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (InteractorName = "FictitiousForcesModel"))
	EChaosClothingSimulationSolverFictitiousForcesModel FictitiousForcesModel = EChaosClothingSimulationSolverFictitiousForcesModel::FullScale;

	/**
	 * The portion of the angular velocity that is used to calculate the strength of all fictitious forces (e.g. centrifugal force).
	 * In Legacy model, this parameter is only having an effect on the portion of the reference bone's angular velocity that has been
	 * removed from the simulation via the Angular Velocity Scale parameter. This means it has no effect when AngularVelocityScale is
	 * set to 1 and Angular Velocity and Acceleration clamps are disabled, in which case the cloth is simulated with full world space
	 * angular velocities and subjected to the true physical world inertial forces.
	 * In FullScale model, this parameter has also an effect when the cloth is simulated with full world space angular velocities, and
	 * isn't affected by the Angular Velocity Scale parameter. 
	 * Values range from 0 to 2, with 0 showing no centrifugal effect, 1 full centrifugal effect, and 2 an overdriven centrifugal effect.
	 */
	UPROPERTY(EditAnywhere, Category = "Velocity Scale Properties", meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "2", EditCondition = "FictitiousForcesModel != EChaosClothingSimulationSolverFictitiousForcesModel::None", InteractorName = "FictitiousAngularScale"))
	float FictitiousAngularScale = 1.f;

	FChaosClothAssetSimulationVelocityScaleConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FChaosClothAssetSimulationBaseConfigNode interface
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
	//~ End FChaosClothAssetSimulationBaseConfigNode interface

	//~ Begin FDataflowNode interface
	virtual void Serialize(FArchive& Ar) override;
	//~ End FDataflowNode interface
};
