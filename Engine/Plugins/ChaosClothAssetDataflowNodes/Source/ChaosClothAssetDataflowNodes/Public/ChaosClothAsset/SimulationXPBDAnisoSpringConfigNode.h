// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "SimulationXPBDAnisoSpringConfigNode.generated.h"

/** XPBD anisotropic spring constraint property configuration node. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.4"))
struct FChaosClothAssetSimulationXPBDAnisoSpringConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationXPBDAnisoSpringConfigNode, "SimulationXPBDAnisoSpringConfig", "Cloth", "Cloth Simulation XPBD Anisotropic Spring Config")

public:
	/** Whether to use the 3D draped space as rest lengths, or use the 2D pattern space instead. */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoSpring Properties")
	bool bXPBDAnisoSpringUse3dRestLengths = true;

	/**
	 * The stiffness of the stretch constraints in the warp (vertical) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoSpring Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000"))
	FChaosClothAssetWeightedValue XPBDAnisoSpringStiffnessWarp = { true, 100.f, 100.f, TEXT("XPBDAnisoSpringStiffnessWarp") };

	/**
	 * The stiffness of the stretch constraints in the weft (horizontal) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoSpring Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000"))
	FChaosClothAssetWeightedValue XPBDAnisoSpringStiffnessWeft = { true, 100.f, 100.f, TEXT("XPBDAnisoSpringStiffnessWeft") };

	/**
	 * The stiffness of the stretch constraints in the bias (diagonal) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoSpring Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000"))
	FChaosClothAssetWeightedValue XPBDAnisoSpringStiffnessBias = { true, 100.f, 100.f, TEXT("XPBDAnisoSpringStiffnessBias") };

	/**
	 * The damping of the stretch constraints, relative to critical damping.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoSpring Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000"))
	FChaosClothAssetWeightedValue XPBDAnisoSpringDamping = { true, 1.f, 1.f, TEXT("XPBDAnisoSpringDamping") };

	/**
	 * The scale of the stretch constraints at rest in the warp (vertical) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoSpring Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "10000000"))
	FChaosClothAssetWeightedValue XPBDAnisoSpringWarpScale = { true, 1.f, 1.f, TEXT("XPBDAnisoSpringWarpScale") };

	/**
	 * The scale of the stretch constraints at rest in the weft (horizontal) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoSpring Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "10000000"))
	FChaosClothAssetWeightedValue XPBDAnisoSpringWeftScale = { true, 1.f, 1.f, TEXT("XPBDAnisoStretchWeftSpringScale") };

	FChaosClothAssetSimulationXPBDAnisoSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
