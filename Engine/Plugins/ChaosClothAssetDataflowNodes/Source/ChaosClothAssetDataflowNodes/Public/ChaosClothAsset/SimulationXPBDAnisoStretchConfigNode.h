// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "SimulationXPBDAnisoStretchConfigNode.generated.h"

/** XPBD anisotropic stretch constraint property configuration node. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.4"))
struct FChaosClothAssetSimulationXPBDAnisoStretchConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationXPBDAnisoStretchConfigNode, "SimulationXPBDAnisoStretchConfig", "Cloth", "Cloth Simulation XPBD Anisotropic Stretch Config")

public:
	/** Whether to use the 3D draped space as rest lengths, or use the 2D pattern space instead. */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoStretch Properties")
	bool bXPBDAnisoStretchUse3dRestLengths = true;

	/**
	 * The stiffness of the stretch constraints in the warp (vertical) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoStretch Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000"))
	FChaosClothAssetWeightedValue XPBDAnisoStretchStiffnessWarp = { true, 100.f, 100.f, TEXT("XPBDAnisoStretchStiffnessWarp") };

	/**
	 * The stiffness of the stretch constraints in the weft (horizontal) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoStretch Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000"))
	FChaosClothAssetWeightedValue XPBDAnisoStretchStiffnessWeft = { true, 100.f, 100.f, TEXT("XPBDAnisoStretchStiffnessWeft") };

	/**
	 * The stiffness of the stretch constraints in the bias (diagonal) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoStretch Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000"))
	FChaosClothAssetWeightedValue XPBDAnisoStretchStiffnessBias = { true, 100.f, 100.f, TEXT("XPBDAnisoStretchStiffnessBias") };

	/**
	 * The damping of the stretch constraints, relative to critical damping.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoStretch Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000"))
	FChaosClothAssetWeightedValue XPBDAnisoStretchDamping = { true, 1.f, 1.f, TEXT("XPBDAnisoStretchDamping") };

	/**
	 * The scale of the stretch constraints at rest in the warp (vertical) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoStretch Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "10000000"))
	FChaosClothAssetWeightedValue XPBDAnisoStretchWarpScale = { true, 1.f, 1.f, TEXT("XPBDAnisoStretchWarpScale") };

	/**
	 * The scale of the stretch constraints at rest in the weft (horizontal) direction.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDAnisoStretch Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "10000000"))
	FChaosClothAssetWeightedValue XPBDAnisoStretchWeftScale = { true, 1.f, 1.f, TEXT("XPBDAnisoStretchWeftScale") };

	FChaosClothAssetSimulationXPBDAnisoStretchConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
