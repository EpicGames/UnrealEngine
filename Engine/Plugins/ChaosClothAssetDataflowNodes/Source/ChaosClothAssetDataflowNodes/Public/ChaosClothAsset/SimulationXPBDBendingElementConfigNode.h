// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "SimulationPBDBendingElementConfigNode.h"
#include "SimulationXPBDBendingElementConfigNode.generated.h"

/** XPBD bending element constraint property configuration node. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.4"))
struct FChaosClothAssetSimulationXPBDBendingElementConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationXPBDBendingElementConfigNode, "SimulationXPBDBendingElementConfig", "Cloth", "Cloth Simulation XPBD Bending Element Config")

public:
	/**
	 * Method for calculating the rest angles of the constraints.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDBendingElement Properties")
	EChaosClothAssetRestAngleConstructionType XPBDRestAngleType = EChaosClothAssetRestAngleConstructionType::Use3DRestAngles;

	/**
	 * Calculate rest angles as a ratio between completely flat and whatever is the 3D rest angle.
	 * When FlatnessRatio = 0, this is equivalent to "Use3DRestAngles".
	 * When FlatnessRatio = 1, the rest angle will be 0 (completely flat).
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDBendingElement Properties", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1", EditCondition = "XPBDRestAngleType == EChaosClothAssetRestAngleConstructionType::FlatnessRatio"))
	FChaosClothAssetWeightedValueNonAnimatable XPBDFlatnessRatio = { 0.f, 0.f, TEXT("XPBDFlatnessRatio") };

	/**
	 * Set rest angles to be the explicit value set here (in degrees).
	 * 0 = Flat, Positive values fold away from the edge normal, Negative values fold toward the edge normal.
	 * When converting vertex weight values to edge values, the value with the smallest absolute value is selected.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDBendingElement Properties", Meta = (UIMin = "-180", UIMax = "180", ClampMin = "-180", ClampMax = "180", EditCondition = "XPBDRestAngleType == EChaosClothAssetRestAngleConstructionType::RestAngle"))
	FChaosClothAssetWeightedValueNonAnimatable XPBDRestAngle = { 0.f, 0.f, TEXT("XPBDRestAngle") };

	/**
	 * The stiffness of the bending element constraints.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDBendingElement Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000"))
	FChaosClothAssetWeightedValue XPBDBendingElementStiffness = { true, 100.f, 100.f, TEXT("XPBDBendingElementStiffness") };

	/**
	 * The damping of the bending element constraints, relative to critical damping.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDBendingElement Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000"))
	FChaosClothAssetWeightedValue XPBDBendingElementDamping = { true, 1.f, 1.f, TEXT("XPBDBendingElementDamping") };

	/**
	 * Once the element has bent such that it's folded more than this ratio from its rest angle ("buckled"), switch to using Buckling Stiffness instead of BendingElement Stiffness.
	 * When Buckling Ratio = 0, the Buckling Stiffness will never be used. When BucklingRatio = 1, the Buckling Stiffness will be used as soon as its bent past its rest configuration.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDBendingElement Properties", Meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	float XPBDBucklingRatio = 0.5f;

	/**
	 * The stiffness after buckling.
	 * The constraint will use this stiffness instead of element Stiffness once the cloth has buckled, i.e., bent beyond a certain angle.
	 * Typically, Buckling Stiffness is set to be less than BendingElement Stiffness.
	 * Buckling Ratio determines the switch point between using BendingElement Stiffness and Buckling Stiffness.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDBendingElement Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "10000000", EditCondition = "XPBDBucklingRatio != 0"))
	FChaosClothAssetWeightedValue XPBDBucklingStiffness = { true, 50.f, 50.f, TEXT("XPBDBucklingStiffness") };

	FChaosClothAssetSimulationXPBDBendingElementConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
