// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/SimulationBaseConfigNode.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "SimulationXPBDBendingSpringConfigNode.generated.h"

/** XPBD bending spring constraint property configuration node. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.4"))
struct FChaosClothAssetSimulationXPBDBendingSpringConfigNode : public FChaosClothAssetSimulationBaseConfigNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSimulationXPBDBendingSpringConfigNode, "SimulationXPBDBendingSpringConfig", "Cloth", "Cloth Simulation XPBD Bending Spring Config")

public:
	/**
	 * The stiffness of the bending cross segment constraints.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDBendingSpring Properties", Meta = (UIMin = "0", UIMax = "10000", ClampMin = "0", ClampMax = "1000000000"))
	FChaosClothAssetWeightedValue XPBDBendingSpringStiffness = { true, 100.f, 100.f, TEXT("XPBDBendingSpringStiffness") };

	/**
	 * The damping of the bending cross segment constraints.
	 * If a valid weight map is found with the given Weight Map name, then both Low and High values
	 * are interpolated with the per particle weight to make the final value used for the simulation.
	 * Otherwise all particles are considered to have a zero weight, and only the Low value is meaningful.
	 */
	UPROPERTY(EditAnywhere, Category = "XPBDBendingSpring Properties", Meta = (UIMin = "0", UIMax = "10", ClampMin = "0", ClampMax = "1000"))
	FChaosClothAssetWeightedValue XPBDBendingSpringDamping = { true, 1.f, 1.f, TEXT("XPBDBendingSpringDamping") };

	FChaosClothAssetSimulationXPBDBendingSpringConfigNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void AddProperties(FPropertyHelper& PropertyHelper) const override;
};
