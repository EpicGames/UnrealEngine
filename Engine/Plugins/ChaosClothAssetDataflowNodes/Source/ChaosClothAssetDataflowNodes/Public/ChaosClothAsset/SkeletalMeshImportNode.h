// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "SkeletalMeshImportNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class USkeletalMesh;
/** Import a skeletal mesh asset into the cloth collection simulation and/or render mesh containers. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSkeletalMeshImportNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSkeletalMeshImportNode_v2, "SkeletalMeshImport", "Cloth", "Cloth Skeletal Mesh Import")

public:
	UPROPERTY(Meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/** The skeletal mesh to import. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (DataflowInput))
	TObjectPtr<const USkeletalMesh> SkeletalMesh;

	/** Reimport the imported skeletal mesh asset. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (ButtonImage = "Persona.ReimportAsset", DisplayName = "Reimport Asset"))
	FDataflowFunctionProperty Reimport;

	/** Whether to import the simulation mesh from the specified skeletal mesh. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import")
	bool bImportSimMesh = true;

	/** Whether to import the render mesh from the specified skeletal mesh. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import")
	bool bImportRenderMesh = true;

	/** The skeletal mesh LOD to import. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (ClampMin = "0", DisplayName = "LOD Index"))
	int32 LODIndex = 0;

	/** Enable single import section mode. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import")
	bool bImportSingleSection = false;

	/** The skeletal mesh LOD section to import. If not enabled, then all sections will be imported. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (ClampMin = "0", EditCondition = "bImportSingleSection"))
	int32 SectionIndex = 0;

	/**
	 * UV channel of the skeletal mesh to import the 2D simulation mesh patterns from.
	 * If set to -1, or the specified UVChannel doesn't exist then the import will unwrap the 3D simulation mesh into 2D simulation mesh patterns.
	 */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (ClampMin = "-1", EditCondition = "bImportSimMesh"))
	int32 UVChannel = 0;

	/* Apply this scale to the UVs when populating Sim Mesh positions. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (AllowPreserveRatio, EditCondition = "bImportSimMesh && UVChannel != INDEX_NONE"))
	FVector2f UVScale = { 1.f, 1.f };

	/** Set the same physics asset as the one used by the imported skeletal mesh. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import")
	bool bSetPhysicsAsset = false;

	/** Import morph targets as Sim Mesh Morph Targets */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (EditCondition = "bImportSimMesh"))
	bool bImportSimMorphTargets = false;

	FChaosClothAssetSkeletalMeshImportNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};

/** Import a skeletal mesh asset into the cloth collection simulation and/or render mesh containers. 
 * This version re-calculates the Sim Mesh Normals, and they are flipped. It also does not remove topologically degenerate triangles. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.5"))
struct UE_DEPRECATED(5.5, "Use the newer version of this node instead.") FChaosClothAssetSkeletalMeshImportNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSkeletalMeshImportNode, "SkeletalMeshImport", "Cloth", "Cloth Skeletal Mesh Import")

public:
	UPROPERTY(Meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/** The skeletal mesh to import. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (DataflowInput))
	TObjectPtr<const USkeletalMesh> SkeletalMesh;

	/** The skeletal mesh LOD to import. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (ClampMin = "0", DisplayName = "LOD Index"))
	int32 LODIndex = 0;

	/** Enable single import section mode. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import")
	bool bImportSingleSection = false;

	/** The skeletal mesh LOD section to import. If not enabled, then all sections will be imported. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (ClampMin = "0", EditCondition = "bImportSingleSection"))
	int32 SectionIndex = 0;

	/** Whether to import the simulation mesh from the specified skeletal mesh. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import")
	bool bImportSimMesh = true;

	/** Whether to import the render mesh from the specified skeletal mesh. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import")
	bool bImportRenderMesh = true;

	/**
	 * UV channel of the skeletal mesh to import the 2D simulation mesh patterns from.
	 * If set to -1, or the specified UVChannel doesn't exist then the import will unwrap the 3D simulation mesh into 2D simulation mesh patterns.
	 */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (ClampMin = "-1", EditCondition = "bImportSimMesh"))
	int32 UVChannel = 0;

	/* Apply this scale to the UVs when populating Sim Mesh positions. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import", Meta = (AllowPreserveRatio, EditCondition = "bImportSimMesh && UVChannel != INDEX_NONE"))
	FVector2f UVScale = { 1.f, 1.f };

	/** Set the same physics asset as the one used by the imported skeletal mesh. */
	UPROPERTY(EditAnywhere, Category = "Skeletal Mesh Import")
	bool bSetPhysicsAsset = false;

	FChaosClothAssetSkeletalMeshImportNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual void Serialize(FArchive& Ar) override;
};
