// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "SkinningBlendNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/**
 * Initialize the RenderDeformerSkinningBlend weight map from the ProxyDeformer mapping data.
 * 
 * The weight map is used by the cloth render shader to decide how much to blend between skinned and simulated points.
 * Value ranges between 0 for fully deformed, to 1 for fully skinned.
 * 
 * Editing the RenderDeformerSkinningBlend is possible after this node (via a WeightMap node),
 * as the RenderDeformerSkinningBlend weight map is an intrinsic attribute of the ClothCollection.
 * 
 * Note: ProxyDeformer mapping data must exist on the input ClothCollection prior to using this node.
 */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetSkinningBlendNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetSkinningBlendNode, "SkinningBlend", "Cloth", "Cloth Simulation Skinning Blend")

public:

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/**
	 * The name of a selection containing all the kinematic points. Must be of group type SimVertices2D, SimVertices3D, or SimFaces.
	 * Using an empty (or invalid) selection will make this node consider all points as dynamic.
	 * This selection is usually obtained from the MaxDistanceConfig node or built from the same weight map set to the MaxDistanceConfig node
	 * using a WeightMapToSelection node and a very low threshold.
	 */
	UPROPERTY(EditAnywhere, Category = "SkinningBlend")
	FChaosClothAssetConnectableIStringValue KinematicVertices3D;

	/**
	 * Whether to create a smoothed RenderDeformerSkinningBlend weight map to ease the transition between the deformed part and the skinned part of the render mesh.
	 * When no transition is created there will be a visible step in the rendered triangles around the edge of the kinematic/dynamic transition of the proxy simulation mesh.
	 * The RenderDeformerSkinningBlend weight map is created regardless of the transition being created smooth or not, and can be later adjusted by using the WeightMap node.
	 */
	UPROPERTY(EditAnywhere, Category = "SkinningBlend")
	bool bUseSmoothTransition = true;

	/**
	 * The name of the render mesh weight map generated by this node detailing the contribution of the proxy deformer.
	 * Value ranges between 0 (fully deformed) and 1 (fully skinned).
	 * The name of this render mesh weight map cannot be changed and is only provided for further tweaking.
	 */
	UPROPERTY(VisibleAnywhere, Category = "SkinningBlend", Meta = (DataflowOutput))
	FString SkinningBlendName;

	FChaosClothAssetSkinningBlendNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};
