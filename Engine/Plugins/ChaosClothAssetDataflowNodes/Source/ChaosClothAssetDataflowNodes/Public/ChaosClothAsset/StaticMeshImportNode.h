// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "StaticMeshImportNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class UStaticMesh;

/** Import a static mesh asset into the cloth collection simulation and/or render mesh containers.*/
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetStaticMeshImportNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetStaticMeshImportNode_v2, "StaticMeshImport", "Cloth", "Cloth Static Mesh Import")

public:
	UPROPERTY(Meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/* The Static Mesh to import from. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (DataflowInput))
	TObjectPtr<const UStaticMesh> StaticMesh;

	/** Reimport the imported static mesh asset. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (ButtonImage = "Persona.ReimportAsset", DisplayName = "Reimport Asset", EditCondition = "!bIsLocked"))
	FDataflowFunctionProperty Reimport;

	/* Import static mesh data as a simulation mesh data. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import")
	bool bImportSimMesh = true;

	/* Import static mesh data as render mesh data. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import")
	bool bImportRenderMesh = true;

	/* Which static mesh Lod to import. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (ClampMin = "0", DisplayName = "LOD Index"))
	int32 LODIndex = 0;

	/* Material section to import as sim mesh data. Use -1 to import all sections. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (ClampMin = "-1", EditCondition = "bImportSimMesh"))
	int32 SimMeshSection = INDEX_NONE;

	/* Material section to import as render mesh data. Use -1 to import all sections. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (ClampMin = "-1", EditCondition = "bImportRenderMesh"))
	int32 RenderMeshSection = INDEX_NONE;

	/**
	 * UV channel of the static mesh to import the 2D simulation mesh patterns from.
	 * If set to -1, or the specified UVChannel doesn't exist then the import will unwrap the 3D simulation mesh into 2D simulation mesh patterns.
	 */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (ClampMin = "-1", EditCondition = "bImportSimMesh"))
	int32 UVChannel = 0;

	/* Apply this scale to the UVs when populating Sim Mesh positions. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (AllowPreserveRatio, EditCondition = "bImportSimMesh && (UVChannel >= 0)"))
	FVector2f UVScale = { 1.f, 1.f };

	FChaosClothAssetStaticMeshImportNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};


/** Import a static mesh asset into the cloth collection simulation and/or render mesh containers. 
* This version re-calculates the Sim Mesh Normals, and they are flipped. It also does not remove topologically degenerate triangles.*/
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.5"))
struct UE_DEPRECATED(5.5, "Use the newer version of this node instead.") FChaosClothAssetStaticMeshImportNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetStaticMeshImportNode, "StaticMeshImport", "Cloth", "Cloth Static Mesh Import")

public:
	UPROPERTY(Meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/* The Static Mesh to import from. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (DataflowInput))
	TObjectPtr<const UStaticMesh> StaticMesh;

	/* Which static mesh Lod to import. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (ClampMin = "0", DisplayName = "LOD Index"))
	int32 LODIndex = 0;

	/* Import static mesh data as a simulation mesh data. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import")
	bool bImportSimMesh = true;

	/* Material section to import as sim mesh data. Use -1 to import all sections. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (ClampMin = "-1", EditCondition = "bImportSimMesh"))
	int32 SimMeshSection = INDEX_NONE;

	/**
	 * UV channel of the static mesh to import the 2D simulation mesh patterns from.
	 * If set to -1, or the specified UVChannel doesn't exist then the import will unwrap the 3D simulation mesh into 2D simulation mesh patterns.
	 */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (ClampMin = "-1", EditCondition = "bImportSimMesh"))
	int32 UVChannel = 0;

	/* Apply this scale to the UVs when populating Sim Mesh positions. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (AllowPreserveRatio, EditCondition = "bImportSimMesh && (UVChannel >= 0)"))
	FVector2f UVScale = { 1.f, 1.f };

	/* Import static mesh data as render mesh data. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import")
	bool bImportRenderMesh = true;

	/* Material section to import as render mesh data. Use -1 to import all sections. */
	UPROPERTY(EditAnywhere, Category = "Static Mesh Import", Meta = (ClampMin = "-1", EditCondition = "bImportRenderMesh"))
	int32 RenderMeshSection = INDEX_NONE;

	FChaosClothAssetStaticMeshImportNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
