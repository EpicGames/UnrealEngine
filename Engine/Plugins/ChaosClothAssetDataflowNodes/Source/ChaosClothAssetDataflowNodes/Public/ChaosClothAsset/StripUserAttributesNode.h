// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "StripUserAttributesNode.generated.h"

/**
 * Strip User Attributes such as Weight Maps, Sets, and Face Int Maps from a Cloth Collection if they are not referenced by properties.
 * This utility node makes it easier to remove unused attributes from your asset, reducing asset size.
 * Note that if this node is not present in your Cloth Dataflow Graph, unreferenced Sets and Face Int Maps will not be copied to the
 * simulation model. Weight Maps will continue to be copied without this node to retain legacy behavior.
 */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothStripUserAttributesNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothStripUserAttributesNode, "StripUserAttributes", "Cloth", "Cloth Strip User Attributes")

public:

	FChaosClothStripUserAttributesNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Strip all unused sim vertex weight maps from the asset. This reduces the asset size, but the weight maps will be unavailable in blueprints, for debug display, or in downstream cloth assets which import this one. */
	UPROPERTY(EditAnywhere, Category = "Strip User Attributes")
	bool bStripUnusedSimWeightMaps = true;

	/** Extra sim weight maps to keep when stripping unused weight maps. */
	UPROPERTY(EditAnywhere, Category = "Strip User Attributes", Meta = (DataflowInput, EditCondition = "bStripUnusedSimWeightMaps"))
	TArray<FString> ExtraSimWeightMaps;

	/** Strip all unused sim and render sets from the asset. This reduces the asset size, but the sets will be unavailable in blueprints, for debug display, or in downstream cloth assets which import this one. */
	UPROPERTY(EditAnywhere, Category = "Strip User Attributes")
	bool bStripUnusedSets = true;

	/** Extra sets to keep when stripping unused sets. */
	UPROPERTY(EditAnywhere, Category = "Strip User Attributes", Meta = (DataflowInput, EditCondition = "bStripUnusedSets"))
	TArray<FString> ExtraSets;

	/** Strip all unused sim face integer maps from the asset. This reduces the asset size, but the maps will be unavailable in blueprints, for debug display, or in downstream cloth assets which import this one. */
	UPROPERTY(EditAnywhere, Category = "Strip User Attributes")
	bool bStripUnusedSimFaceIntMaps = true;

	/** Extra maps to keep when stripping unused face integer maps. */
	UPROPERTY(EditAnywhere, Category = "Strip User Attributes", Meta = (DataflowInput, EditCondition = "bStripUnusedSimFaceIntMaps"))
	TArray<FString> ExtraSimFaceIntMaps;

	/** Strip all unused render vertex weight maps from the asset. This reduces the uncooked asset size, but the maps will be unavailable in downstream assets which import this one. 
	 *  Note that user render weight maps do not currently exist in the cooked asset, so this property has no effect on cooked assets. */
	UPROPERTY(EditAnywhere, Category = "Strip User Attributes")
	bool bStripUnusedRenderWeightMaps = true;

	/** Extra render weight maps to keep when stripping unused weight maps. */
	UPROPERTY(EditAnywhere, Category = "Strip User Attributes", Meta = (DataflowInput, EditCondition = "bStripUnusedRenderWeightMaps"))
	TArray<FString> ExtraRenderWeightMaps;

	/** Tell the cloth asset to transfer all user attributes from the cloth collection to the internal simulation model rather than skipping unreferenced sets and face int maps. 
	* Note that sim weight maps are always transferred.
	*/
	UPROPERTY(EditAnywhere, Category = "Strip User Attributes")
	bool bCopyAllUserAttributesToSimModel = true;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	//~ End FDataflowNode interface
};
