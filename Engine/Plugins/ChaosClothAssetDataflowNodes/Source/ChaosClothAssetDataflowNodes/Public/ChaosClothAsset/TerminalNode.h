// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowFunctionProperty.h"
#include "Dataflow/DataflowTerminalNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ChaosClothAsset/ClothLodTransitionDataCache.h"
#include "TerminalNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** Refresh structure for push buton customization. */
USTRUCT()
struct UE_DEPRECATED(5.5, "Use UE::Dataflow::FunctionProperty instead.") FChaosClothAssetTerminalNodeRefreshAsset
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = "Terminal Node Refresh Asset")
	bool bRefreshAsset = false;
};

/** Cloth terminal node to generate a cloth asset from a cloth collection. */
USTRUCT(Meta = (DataflowCloth, DataflowTerminal))
struct FChaosClothAssetTerminalNode_v2 : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetTerminalNode_v2, "ClothAssetTerminal", "Cloth", "Cloth Terminal")  // TODO: Should the category be Terminal instead like all other terminal nodes

public:
	/** Input cloth collection for this LOD. */
	UPROPERTY()
	TArray<FManagedArrayCollection> CollectionLods;

	/**
	 * Refresh the asset even if the ClothCollection hasn't changed.
	 * Note that it is not required to manually refresh the cloth asset, this is done automatically when there is a change in the Dataflow.
	 * This function is a developper utility used for debugging.
	 */
	UPROPERTY(EditAnywhere, Transient, Category = "Cloth Asset Terminal", Meta = (DisplayName = "Refresh Asset", ButtonImage = "Icons.Refresh"))
	FDataflowFunctionProperty Refresh;

	FChaosClothAssetTerminalNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context) const override {}
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return true; }
	virtual bool CanRemovePin() const override { return CollectionLods.Num() > NumInitialCollectionLods; }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface

	TArray<TSharedRef<const FManagedArrayCollection>> GetCleanedCollectionLodValues(UE::Dataflow::FContext& Context) const;
	UE::Dataflow::TConnectionReference<FManagedArrayCollection> GetConnectionReference(int32 Index) const;

	UPROPERTY()
	mutable TArray<FChaosClothAssetLodTransitionDataCache> LODTransitionDataCache;

	// This is for runtime only--used to determine if only properties need to be updated.
	mutable bool bClothCollectionChecksumValid = false;
	mutable uint32 ClothColllectionChecksum = 0;
	static constexpr int32 NumRequiredInputs = 0;
	static constexpr int32 NumInitialCollectionLods = 1;
};

/** Cloth terminal node to generate a cloth asset from a cloth collection. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // For RefreshAsset
USTRUCT(Meta = (DataflowCloth, DataflowTerminal, Deprecated = 5.5))
struct UE_DEPRECATED(5.5, "Use the newer version of this node instead.") FChaosClothAssetTerminalNode : public FDataflowTerminalNode
{
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetTerminalNode, "ClothAssetTerminal", "Cloth", "Cloth Terminal")  // TODO: Should the category be Terminal instead like all other terminal nodes

public:
	static constexpr int32 MaxLods = 6;  // Hardcoded number of LODs since it is currently not possible to use arrays for optional inputs

	/** LOD 0 input, right click on the node and add pins to add more LODs. */
	UPROPERTY(meta = (DataflowInput, DisplayName = "Collection LOD 0"))
	FManagedArrayCollection CollectionLod0;
	/** LOD 1 input, right click on the node and add pins to add more LODs. */
	UPROPERTY(meta = (DisplayName = "Collection LOD 1"))
	FManagedArrayCollection CollectionLod1;
	/** LOD 2 input, right click on the node and add pins to add more LODs. */
	UPROPERTY(meta = (DisplayName = "Collection LOD 2"))
	FManagedArrayCollection CollectionLod2;
	/** LOD 3 input, right click on the node and add pins to add more LODs. */
	UPROPERTY(meta = (DisplayName = "Collection LOD 3"))
	FManagedArrayCollection CollectionLod3;
	/** LOD 4 input, right click on the node and add pins to add more LODs. */
	UPROPERTY(meta = (DisplayName = "Collection LOD 4"))
	FManagedArrayCollection CollectionLod4;
	/** LOD 5 input, right click on the node and add pins to add more LODs. */
	UPROPERTY(meta = (DisplayName = "Collection LOD 5"))
	FManagedArrayCollection CollectionLod5;
	/** The number of LODs currently exposed to the node UI. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	int32 NumLods = NumInitialCollectionLods;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	/**
	 * Refresh the asset even if the ClothCollection hasn't changed.
	 * Note that it is not required to manually refresh the cloth asset, this is done automatically when there is a change in the Dataflow.
	 * This function is a developper utility used for debugging.
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(EditAnywhere, Category = "Cloth Asset Terminal")
	mutable FChaosClothAssetTerminalNodeRefreshAsset RefreshAsset;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FChaosClothAssetTerminalNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override;
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override {}
	virtual TArray<UE::Dataflow::FPin> AddPins() override;
	virtual bool CanAddPin() const override { return NumLods < MaxLods; }
	virtual bool CanRemovePin() const override { return NumLods > 1; }
	virtual TArray<UE::Dataflow::FPin> GetPinsToRemove() const override;
	virtual void OnPinRemoved(const UE::Dataflow::FPin& Pin) override;
	virtual void PostSerialize(const FArchive& Ar) override;
	//~ End FDataflowNode interface

	TArray<const FManagedArrayCollection*> GetCollectionLods() const;
	TArray<TSharedRef<const FManagedArrayCollection>> GetCleanedCollectionLodValues(UE::Dataflow::FContext& Context) const;
	const FManagedArrayCollection* GetCollectionLod(int32 LodIndex) const;

	UPROPERTY()
	mutable TArray<FChaosClothAssetLodTransitionDataCache> LODTransitionDataCache;

	// This is for runtime only--used to determine if only properties need to be updated.
	mutable bool bClothCollectionChecksumValid = false;
	mutable uint32 ClothColllectionChecksum = 0;

	static constexpr int32 NumRequiredInputs = 0;
	static constexpr int32 NumInitialCollectionLods = 1;
};
