// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ChaosClothAsset/WeightedValue.h"
#include "TransferSkinWeightsNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

class USkeletalMesh;

UENUM(BlueprintType)
enum class EChaosClothAssetTransferSkinWeightsMethod : uint8
{
	/** For every vertex on the target mesh, find the closest point on the surface of the source mesh and copy its weights. */
	ClosestPointOnSurface,
	
	/**
	 * For every vertex on the target mesh, find the closest point on the surface of the source mesh.
	 * If that point position is within the search radius, and their normals differ by less than the specified normal threshold,
	 * then the vertex weights are directly copied from the source point to the target mesh vertex.
	 * For all other vertices whose weights didn't get transferred, smoothed weight values are automatically computed.
	 */
	InpaintWeights
};


UENUM(BlueprintType)
enum class EChaosClothAssetMaxNumInfluences : uint8
{
	Uninitialized = 0	UMETA(Hidden),
	Four = 4			UMETA(DisplayName = "4"),
	Eight = 8			UMETA(DisplayName = "8"),
	Twelve = 12			UMETA(DisplayName = "12")
};

UENUM(BlueprintType)
enum class EChaosClothAssetTransferTargetMeshType : uint8
{
	/** Perform the skin weights transfer for both the simulation and render meshes. */
	All UMETA(DisplayName = "Sim & Render Meshes"),

	/** Perform the skin weights transfer for the simulation mesh only. */
	Simulation UMETA(DisplayName = "Sim Mesh"),
	
	/** Perform the skin weights transfer for the render mesh only. */
	Render UMETA(DisplayName = "Render Mesh")
};

UENUM(BlueprintType)
enum class EChaosClothAssetTransferRenderMeshSource : uint8
{
	/** For render mesh, transfer weights from the source Skeletal Mesh. */
	SkeletalMesh UMETA(DisplayName = "Skeletal Mesh"),

	/** For render mesh, transfer weights from the Collection input sim mesh, or Sim Collection input if connected. */
	SimulationMesh UMETA(DisplayName = "Collection/Sim Collection")
};

/** Transfer the skinning weights set on a skeletal mesh to the simulation and/or render mesh stored in the cloth collection. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetTransferSkinWeightsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetTransferSkinWeightsNode, "TransferSkinWeights", "Cloth", "Cloth Transfer Skin Weights")
	DATAFLOW_NODE_RENDER_TYPE("SurfaceRender", FName("FClothCollection"), "Collection")

public:
	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The type of cloth mesh the skeletal mesh transfer will be applied to, simulation, render mesh, or both. */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (DisplayName = "Target Mesh(es)"))
	EChaosClothAssetTransferTargetMeshType TargetMeshType = EChaosClothAssetTransferTargetMeshType::All;

private:
	/** For the sim mesh, simulation mesh transfers always use the specified skeletal mesh. */
	UPROPERTY(VisibleAnywhere, Category = "Transfer Skin Weights", Meta = (DisplayName = "Sim Mesh Transfer Source", EditCondition = "TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render", EditConditionHides))
	FString SimMeshSourceTypeHint = TEXT("Skeletal Mesh");

public:
	/** For the render mesh, choose which source to use, either the default or specified simulation mesh or the specified skeletal mesh. */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (DisplayName = "Render Mesh Transfer Source", EditCondition = "TargetMeshType != EChaosClothAssetTransferTargetMeshType::Simulation", EditConditionHides))
	EChaosClothAssetTransferRenderMeshSource RenderMeshSourceType = EChaosClothAssetTransferRenderMeshSource::SimulationMesh;

	/** The skeletal mesh to transfer the skin weights from. */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (DataflowInput, EditCondition = "TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render || RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SkeletalMesh", EditConditionHides))
	TObjectPtr<USkeletalMesh> SkeletalMesh;

	/** The collection containing the sim mesh to use when the Render Mesh Transfer Source is set to Collection/Sim Collection. When this input isn't connected, the Collection input is used instead. */
	UPROPERTY(Meta = (DataflowPassthrough = "Collection", Dataflowinput))
	FManagedArrayCollection SimCollection;

	/** The skeletal mesh LOD to transfer the skin weights from. */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (DisplayName = "LOD Index", EditCondition = "TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render || RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SkeletalMesh", EditConditionHides, DataflowInput))
	int32 LodIndex = 0;

	/** The relative transform between the skeletal mesh and the cloth asset. */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (EditCondition = "TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render || RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SkeletalMesh", EditConditionHides))
	FTransform Transform;

private:
	/** Algorithm used for the transfer method. When the Render Mesh Transfer Source is set to use the sim mesh from the Collection/Sim Collection input, only the ClosestPointOnSurface method is available. */
	UPROPERTY(VisibleAnywhere, Category = "Transfer Skin Weights", Meta = (DisplayName = "Transfer Method", EditCondition = "TargetMeshType == EChaosClothAssetTransferTargetMeshType::Render && RenderMeshSourceType != EChaosClothAssetTransferRenderMeshSource::SkeletalMesh", EditConditionHides))
	FString TransferMethodHint = TEXT("Closest Point On Surface");

public:
	/**
	 * Algorithm used for the transfer method.
	 * Use the simple ClosestPointOnSurface method or the more complex InpaintWeights method for better results.
	 * Note: When using the simulation mesh as source for the render mesh transfer, the algorithm will always be the ClosestPointOnSurface method, whatever this setting is.
	 */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (EditCondition = "TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render || RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SkeletalMesh", EditConditionHides))
	EChaosClothAssetTransferSkinWeightsMethod TransferMethod = EChaosClothAssetTransferSkinWeightsMethod::InpaintWeights;

	/**
	 * Percentage of the bounding box diagonal of the simulation mesh to use as search radius for the InpaintWeights method.
	 * All points outside of the search radius will be ignored. 
	 * When set to a negative value (e.g. -1), all points will be considered.
	 */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (UIMin = -1, UIMax = 2, ClampMin = -1, ClampMax = 2, EditCondition = "(TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render || RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SkeletalMesh) && TransferMethod == EChaosClothAssetTransferSkinWeightsMethod::InpaintWeights", EditConditionHides))
	double RadiusPercentage = 0.05;

	/**
	 * Maximum angle difference (in degrees) between the target and source point normals to be considered a match for the InpaintWeights method.
	 * If set to a negative value (e.g. -1), normals will be ignored.
	 */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (UIMin = -1, UIMax = 180, ClampMin = -1, ClampMax = 180, EditCondition = "(TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render || RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SkeletalMesh) && TransferMethod == EChaosClothAssetTransferSkinWeightsMethod::InpaintWeights", EditConditionHides))
	double NormalThreshold = 30;

	/** 
	 * If true, when the closest point doesn't pass the normal threshold test, will try again with a flipped normal. 
	 * This helps with layered meshes where the "inner" and "outer" layers are close to each other but whose normals 
	 * are pointing in the opposite directions.
	 */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (EditCondition = "(TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render || RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SkeletalMesh) && TransferMethod == EChaosClothAssetTransferSkinWeightsMethod::InpaintWeights", EditConditionHides))
	bool LayeredMeshSupport = true;

	/** The number of smoothing iterations applied to the vertices whose weights were automatically computed. */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (UIMin = 0, UIMax = 100, ClampMin = 0, ClampMax = 100, DisplayName = "Smoothing Iterations", EditCondition = "(TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render || RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SkeletalMesh) && TransferMethod == EChaosClothAssetTransferSkinWeightsMethod::InpaintWeights", EditConditionHides))
	int32 NumSmoothingIterations = 10;

	/** The smoothing strength of each smoothing iteration. */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (UIMin = 0, UIMax = 1, ClampMin = 0, ClampMax = 1, EditCondition = "(TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render || RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SkeletalMesh) && TransferMethod == EChaosClothAssetTransferSkinWeightsMethod::InpaintWeights", EditConditionHides))
	float SmoothingStrength = 0.1f;
	
	/** Optional mask where a non-zero value indicates that we want the skinning weights for the vertex to be computed automatically instead of it being copied over from the source mesh. */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (EditCondition = "(TargetMeshType != EChaosClothAssetTransferTargetMeshType::Render || RenderMeshSourceType == EChaosClothAssetTransferRenderMeshSource::SkeletalMesh) && TransferMethod == EChaosClothAssetTransferSkinWeightsMethod::InpaintWeights", EditConditionHides))
	FChaosClothAssetWeightedValueNonAnimatableNoLowHighRange InpaintMask = { TEXT("InpaintMask") };

	/** The maximum number of bones that will influence each vertex. */
	UPROPERTY(EditAnywhere, Category = "Transfer Skin Weights", Meta = (DisplayName = "Max Bone Influences"))
	EChaosClothAssetMaxNumInfluences MaxNumInfluences = EChaosClothAssetMaxNumInfluences::Eight;
	
	FChaosClothAssetTransferSkinWeightsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:

	/** 
	 * The node takes a user-selected skinned SkeletalMesh asset and transfers the skin weights to a simulation/render
	 * meshes using a selected algorithm. You can either transfer to simulation and render meshes separately 
	 * or transfer to both in one go using the TargetMeshType property. When transferring to render meshes, you can 
	 * control the source of the transfer which can either be the body or the simulation mesh (recommended). In the latter 
	 * case we always use the ClosestPointOnSurface algorithm.
	 * 
	 * 
	 * InpaintWeights algorithm:
	 * Main algorithm for transferring weights, which is based on the "Robust Skin Weights Transfer via Weight Inpainting 
	 * Siggraph Asia 2023". The implementation and explanation of the algorithm can be found in 
	 * "Engine\Plugins\Runtime\GeometryProcessing\Source\DynamicMesh\Private\Operations\TransferBoneWeights.h(cpp)"
	 * 
	 * 
	 * Handling of disconnected render meshes:
	 * It is usually the case that sim mesh is welded and manifold meaning that the inpaint method should always 
	 * succeed and give the best results. However, the render mesh is often not welded and consists of multiple 
	 * disconnected parts. This is usually fine, and inpaint should work well except in places where there 
	 * is a big crease along the stitch (like armpit areas), so vertices that are close to each other can have very 
	 * different normals which could potentially lead to different weights being computed. You can either try to 
	 * increase the normal threshold or switch to the closest point method.
	 */
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
