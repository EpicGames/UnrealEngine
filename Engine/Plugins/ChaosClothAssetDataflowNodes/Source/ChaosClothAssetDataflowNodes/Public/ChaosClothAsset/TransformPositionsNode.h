// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "TransformPositionsNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetTransformPositionsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetTransformPositionsNode, "TransformPositions", "Cloth", "Cloth Simulation Transform Positions")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Enable Transforming 2D Sim Mesh Positions */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - 2D Sim")
	bool bTransform2DSimPositions = false;

	/** 2D Sim Transform scale. */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - 2D Sim", Meta = (AllowPreserveRatio, EditCondition = "bTransform2DSimPositions"))
	FVector2f Sim2DScale = { 1.f, 1.f };

	/** 2D Sim Transform rotation angle in degrees. */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - 2D Sim", Meta = (UIMin = -360, UIMax = 360, ClampMin = -360, ClampMax = 360, EditCondition = "bTransform2DSimPositions"))
	float Sim2DRotation = 0.f;

	/** 2D Sim Transform translation. */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - 2D Sim", Meta = (EditCondition = "bTransform2DSimPositions"))
	FVector2f Sim2DTranslation = { 0.f, 0.f };

	/** Sim Pattern to transform. All patterns will be used when set to -1. */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - 2D Sim", Meta = (UIMax = 10, ClampMin = -1, EditCondition = "bTransform2DSimPositions"))
	int32 Sim2DPattern = INDEX_NONE;

	/** Enable Transforming 3D Sim Mesh Positions */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - 3D Sim")
	bool bTransform3DSimPositions = false;

	/** 3D Sim Transform scale. */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - 3D Sim", Meta = (AllowPreserveRatio, EditCondition = "bTransform3DSimPositions"))
	FVector3f Sim3DScale = { 1.f, 1.f, 1.f };

	/** 3D Sim Transform rotation angle in degrees (Euler Angles) */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - 3D Sim", Meta = (UIMin = -360, UIMax = 360, ClampMin = -360, ClampMax = 360, EditCondition = "bTransform3DSimPositions"))
	FVector3f Sim3DRotation = { 0.f, 0.f, 0.f };

	/** 3D Sim Transform translation. */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - 3D Sim", Meta = (EditCondition = "bTransform3DSimPositions"))
	FVector3f Sim3DTranslation = { 0.f, 0.f, 0.f };

	/** Enable Transforming Render Positions */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - Render")
	bool bTransformRenderPositions = false;

	/** Render Transform scale. */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - Render", Meta = (AllowPreserveRatio, EditCondition = "bTransformRenderPositions"))
	FVector3f RenderScale = { 1.f, 1.f, 1.f };

	/** Render Transform rotation angle in degrees (Euler Angles) */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - Render", Meta = (UIMin = -360, UIMax = 360, ClampMin = -360, ClampMax = 360, EditCondition = "bTransformRenderPositions"))
	FVector3f RenderRotation = { 0.f, 0.f, 0.f };

	/** Render Transform translation. */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - Render", Meta = (EditCondition = "bTransformRenderPositions"))
	FVector3f RenderTranslation = { 0.f, 0.f, 0.f };

	/** Render Pattern to transform. All patterns will be used when set to -1. */
	UPROPERTY(EditAnywhere, Category = "TransformPositions - Render", Meta = (UIMax = 10, ClampMin = -1, EditCondition = "bTransformRenderPositions"))
	int32 RenderPattern = INDEX_NONE;

	FChaosClothAssetTransformPositionsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
