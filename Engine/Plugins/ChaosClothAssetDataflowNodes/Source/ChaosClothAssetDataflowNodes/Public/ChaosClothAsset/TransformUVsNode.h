// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "TransformUVsNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetTransformUVsNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetTransformUVsNode, "TransformUVs", "Cloth", "Cloth Simulation Transform UV")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** Transform scale. */
	UPROPERTY(EditAnywhere, Category = "Transform UVs", Meta = (AllowPreserveRatio))
	FVector2f Scale = { 1.f, 1.f };

	/** Transform rotation angle in degrees. */
	UPROPERTY(EditAnywhere, Category = "Transform UVs", Meta = (UIMin = -360, UIMax = 360, ClampMin = -360, ClampMax = 360))
	float Rotation = 0.f;

	/** Transform translation. */
	UPROPERTY(EditAnywhere, Category = "Transform UVs")
	FVector2f Translation = { 0.f, 0.f };

	/** Pattern to transform. All patterns will be used when set to -1. */
	UPROPERTY(EditAnywhere, Category = "Transform UVs", Meta = (UIMax = 10, ClampMin = -1))
	int32 Pattern = INDEX_NONE;

	/** UV channel to transform. All UV channels will be used when set to -1. */
	UPROPERTY(EditAnywhere, Category = "Transform UVs", Meta = (DataflowUVChannel, UIMax = 5, ClampMin = -1))
	int32 UVChannel = INDEX_NONE;

	FChaosClothAssetTransformUVsNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
