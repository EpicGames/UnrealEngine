// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "ChaosClothAsset/ImportFilePath.h"
#include "Dataflow/DataflowTerminalNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Misc/SecureHash.h"
#include "USDImportNode.generated.h"

/** Import a USD file from a third party garment construction software. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.5"))
struct UE_DEPRECATED(5.5, "Use the newer version of this node instead.") FChaosClothAssetUSDImportNode : public FDataflowTerminalNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetUSDImportNode, "USDImport", "Cloth", "Cloth USD Import")

public:
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	UE_DEPRECATED(5.7, "This node is deprecated, and this value can no longer be edited.")
	UPROPERTY()
	FChaosClothAssetImportFilePath UsdFile;

	FChaosClothAssetUSDImportNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowTerminalNode interface
	virtual void SetAssetValue(TObjectPtr<UObject> Asset, UE::Dataflow::FContext& Context) const override {}
	//~ End FDataflowTerminalNode interface

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual void Serialize(FArchive& Archive) override;
	//~ End FDataflowNode interface

	bool ImportFromCache(const TSharedRef<FManagedArrayCollection>& ClothCollection, class FText& OutErrorText) const;
	void UpdateImportedAssets();

	/** Name of the imported USD file. This node is deprecated, and this value can no longer be edited. */
	UPROPERTY(Transient, VisibleAnywhere, Category = "USD Import", Meta = (DisplayName = "USD File"))
	FString ImportedFilePath;

	/** Content folder where all the USD assets are imported. */
	UPROPERTY(VisibleAnywhere, Category = "USD Import")
	FString PackagePath;

	/** List of all the dependent assets created from the USD import process. */
	UPROPERTY(VisibleAnywhere, Category = "USD Import")
	TArray<TObjectPtr<UObject>> ImportedAssets;

	FMD5Hash FileHash;
	FManagedArrayCollection CollectionCache;  // Content cache for data that hasn't got a USD schema yet
};
