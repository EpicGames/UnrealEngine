// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "ChaosClothAsset/ImportFilePath.h"
#include "Dataflow/DataflowNode.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "GeometryCollection/ManagedArrayCollection.h"

#include "USDImportNode_v2.generated.h"

class UStaticMesh;

/** Support struct for storage of the extra data that doesn't fit inside a static mesh. */
USTRUCT(MinimalAPI)
struct FChaosClothAssetUsdClothData final
{
	GENERATED_USTRUCT_BODY()

	TMap<FName, TSet<int32>> SimPatterns;
	TMap<FName, TSet<FIntVector2>> Sewings;
	TMap<FName, TSet<int32>> RenderPatterns;
	TMap<FName, TSet<FName>> RenderToSimPatterns;
	TMap<FName, int32> SimPatternFabricIndices;
	FManagedArrayCollection SimulationCollection;
	
	CHAOSCLOTHASSETDATAFLOWNODES_API FChaosClothAssetUsdClothData();
	CHAOSCLOTHASSETDATAFLOWNODES_API ~FChaosClothAssetUsdClothData();

	void CHAOSCLOTHASSETDATAFLOWNODES_API Reset();
	bool CHAOSCLOTHASSETDATAFLOWNODES_API Serialize(FArchive& Ar);
};

template<> struct TStructOpsTypeTraits<FChaosClothAssetUsdClothData> : public TStructOpsTypeTraitsBase2<FChaosClothAssetUsdClothData>
{
	enum
	{
		WithSerializer = true,
	};
};

/** Import a USD file from a third party garment construction software. */
USTRUCT(Meta = (DataflowCloth, Deprecated = "5.8"))
struct UE_DEPRECATED(5.8, "Use the newer version of this node instead.") FChaosClothAssetUSDImportNode_v2 : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetUSDImportNode_v2, "USDImport", "Cloth", "Cloth USD Import")

public:
	UPROPERTY(meta = (DataflowOutput))
	FManagedArrayCollection Collection;

	/** Only import the simulation mesh data from the USD file. */
	UPROPERTY(EditAnywhere, Category = "USD Import")
	bool bImportSimMesh = true;

	/** Only import the render mesh data from the USD file. */
	UPROPERTY(EditAnywhere, Category = "USD Import")
	bool bImportRenderMesh = true;

	/** Importing the render mesh with opacity requires transluscency to be enable in the project settings. */
	UPROPERTY(EditAnywhere, Category = "USD Import", Meta = (EditCondition = "bImportRenderMesh", EditConditionHides))
	bool bImportWithOpacity = false;

	UE_DEPRECATED(5.8, "This node is deprecated, and this value can no longer be edited.")
	UPROPERTY()
	FChaosClothAssetImportFilePath UsdFile;

	UE_DEPRECATED(5.8, "This node is deprecated, and this function can no longer be triggered.")
	UPROPERTY()
	FDataflowFunctionProperty ReimportUsdFile;

	/** The USD import process generates an intermediary simulation static mesh. Click on this button to reimport it without reimporting the USD file. */
	UPROPERTY(EditAnywhere, Category = "USD Import", Meta = (ButtonImage = "Icons.Refresh", EditCondition = "ImportedSimStaticMesh != nullptr && bImportSimMesh", EditConditionHides))
	FDataflowFunctionProperty ReloadSimStaticMesh;

	/** The USD import process generates an intermediary render static mesh. Click on this button to reimport it without reimporting the USD file. */
	UPROPERTY(EditAnywhere, Category = "USD Import", Meta = (ButtonImage = "Icons.Refresh", EditCondition = "ImportedRenderStaticMesh != nullptr && bImportRenderMesh", EditConditionHides))
	FDataflowFunctionProperty ReloadRenderStaticMesh;

	FChaosClothAssetUSDImportNode_v2(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual void Serialize(FArchive& Archive) override;
	//~ End FDataflowNode interface

	// Import the given static mesh as a simulation mesh into the cloth collection
	bool ImportSimStaticMesh(const TSharedRef<FManagedArrayCollection> ClothCollection, class FText& OutErrorText);
	// Import the given static mesh as a render mesh into the cloth collection
	bool ImportRenderStaticMesh(const TSharedRef<FManagedArrayCollection> ClothCollection, class FText& OutErrorText);

	// Find the two imported static meshes
	void UpdateImportedAssets(const FString& SimMeshName, const FString& RenderMeshName);

	/** Content folder where all the USD assets are imported. */
	UPROPERTY(VisibleAnywhere, Category = "USD Import")
	FString PackagePath;

	/**
	 * The static mesh created from the USD import process used as simulation mesh.
	 * Note that this property can still be empty after successfully importing a simulation mesh depending
	 * on whether the simulation mesh is imported from an older version of USD cloth schema.
	 */
	UPROPERTY(VisibleAnywhere, Category = "USD Import")
	TObjectPtr<UStaticMesh> ImportedSimStaticMesh;

	/** The UV scale used to import the patterns from the imported static mesh UV coordinates. */
	UPROPERTY(VisibleAnywhere, Category = "USD Import")
	FVector2f ImportedUVScale = { 1.f, 1.f };

	/** The static mesh created from the USD import process used as render mesh. */
	UPROPERTY(VisibleAnywhere, Category = "USD Import")
	TObjectPtr<UStaticMesh> ImportedRenderStaticMesh;

	/** List of all the simulation static mesh's dependent assets. This does not include any Engine, or Engine plugin content. */
	UPROPERTY(VisibleAnywhere, Category = "USD Import")
	TArray<TObjectPtr<UObject>> ImportedSimAssets;

	/** List of all the render static mesh's dependent assets. This does not include any Engine, or Engine plugin content. */
	UPROPERTY(VisibleAnywhere, Category = "USD Import")
	TArray<TObjectPtr<UObject>> ImportedRenderAssets;

	/** Support struct for storage of the extra data that doesn't fit inside a static mesh. */
	UPROPERTY()
	FChaosClothAssetUsdClothData UsdClothData;

	UPROPERTY()
	TArray<TObjectPtr<UObject>> ImportedAssets_DEPRECATED;
};
