// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "ChaosClothAsset/AddWeightMapNode.h"
#include "Dataflow/DataflowFunctionProperty.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ChaosClothAsset/ConnectableValue.h"
#include "Dataflow/DataflowVertexAttributeEditableNode.h"

#include "WeightMapNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

namespace UE::Chaos::ClothAsset
{
	class FCollectionClothFacade;
	class FCollectionClothConstFacade;
}

/** Which mesh to update with the node's weight map */
UENUM()
enum class EChaosClothAssetWeightMapMeshTarget : uint8
{
	Simulation,
	Render
};

/** Painted weight map attributes node. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // For Name implicit operators.
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetWeightMapNode : public FDataflowVertexAttributeEditableNode
{
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetWeightMapNode, "WeightMap", "Cloth", "Cloth Weight Map")
	DATAFLOW_NODE_RENDER_TYPE("SurfaceRender", FName("FClothCollection"), "Collection")

public:
	static constexpr float ReplaceChangedPassthroughValue = UE_BIG_NUMBER;

	UPROPERTY(Meta = (DataflowInput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The collection used to transfer weight map from. */
	UPROPERTY(Meta = (DataflowInput))
	FManagedArrayCollection TransferCollection;

	/** The name to be set as a weight map attribute. */
	UPROPERTY(EditAnywhere, Category = "Weight Map")
	FChaosClothAssetConnectableOStringValue OutputName;

	/** The name to populate this map from and override based on Map Override Type. Output Name will be used if Input Name is empty. */
	UPROPERTY(EditAnywhere, Category = "Weight Map")
	FChaosClothAssetConnectableIStringValue InputName = { TEXT("") };

	UPROPERTY(EditAnywhere, Category = "Weight Map")
	EChaosClothAssetWeightMapMeshTarget MeshTarget = EChaosClothAssetWeightMapMeshTarget::Simulation;

	/** How to apply this node's weight values onto existing maps. Changing this value will change the output map.
	 *  To change how the node's stored weights are calculated, change the equivalent value on the Weight Map Paint Tool context.*/
	UPROPERTY(EditAnywhere, Category = "Weight Map")
	EChaosClothAssetWeightMapOverrideType MapOverrideType = EChaosClothAssetWeightMapOverrideType::ReplaceChanged;

	/**
	 * The type of transfer used to transfer the weight map when a TransferCollection is connected and MeshTarget is Simulation.
	 * Use "Use Render Mesh" to transfer from an imported render mesh into the sim mesh via 3D spatial mapping.
	 * Render weight maps (MeshTarget == Render) always do a 3D transfer regardless of this setting.
	 * This property is disabled when no TransferCollection input has been connected.
	 */
	UPROPERTY(EditAnywhere, Category = "Weight Map")
	EChaosClothAssetWeightMapTransferType TransferType = EChaosClothAssetWeightMapTransferType::Use2DSimMesh;

	/** Transfer the weight map from the connected Transfer Collection containing a weight map with Input Name (or Output Name if Input Name is empty). */
	UPROPERTY(EditAnywhere, Category = "Weight Map", Meta = (ButtonImage = "Icons.Convert"))
	FDataflowFunctionProperty Transfer;

	FChaosClothAssetWeightMapNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());

private:
	friend class UClothEditorWeightMapPaintTool;

	class FWeightMapNodeChange;
	static TUniquePtr<class FToolCommandChange> CHAOSCLOTHASSETDATAFLOWNODES_API MakeWeightMapNodeChange(FChaosClothAssetWeightMapNode& Node);

	const TArray<float>& GetLegacyVertexWeights() const { return VertexWeights; }

	// These methods are exported for UClothEditorWeightMapPaintTool which lives in a different module.
	FName CHAOSCLOTHASSETDATAFLOWNODES_API GetInputName(UE::Dataflow::FContext& Context) const;

	void CHAOSCLOTHASSETDATAFLOWNODES_API SetVertexWeights(const TConstArrayView<float> InputMap, const TArray<float>& FinalValues);

	// Input and FinalOutputMap can be the same array, but should not otherwise be interleaved.
	void CHAOSCLOTHASSETDATAFLOWNODES_API CalculateFinalVertexWeightValues(UE::Chaos::ClothAsset::FCollectionClothConstFacade* ClothFacade, const TConstArrayView<float> InputMap, TArrayView<float> FinalOutputMap) const;

	//~ Begin FDataflowNode interface
	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
	virtual void Serialize(FArchive& Ar) override;
	virtual FDataflowOutput* RedirectSerializedOutput(const FName& MissingOutputName) override;
	//~ End FDataflowNode interface

	//~ Begin - FDataflowVertexAttributeEditableNode API
	virtual void GetSupportedViewModes(UE::Dataflow::FContext& Context, TArray<FName>& OutViewModeNames) const override;
	virtual void GetVertexAttributeValues(UE::Dataflow::FContext& Context, TArray<float>& OutValues) const override;
	virtual void SetVertexAttributeValues(UE::Dataflow::FContext& Context, const TArray<float>& InValues, const TArray<int32>& InWeightIndices) override;
	virtual void GetExtraVertexMapping(UE::Dataflow::FContext& Context, FName SelectedViewMode, TArray<int32>& OutMappingToWeight, TArray<TArray<int32>>& OutMappingFromWeight) const override;
	//~ End - FDataflowVertexAttributeEditableNode API

	void OnTransfer(UE::Dataflow::FContext& Context);

	/** 
	* Get weights values from a specific snaphot
	* If there's data in the snapshot, the original topology wioll be compared to the one in the optional ClothFacade, 
	* and data will be projected to match the new topology if needed
	*/
	void GetWeightsFromSnapshot(UE::Chaos::ClothAsset::FCollectionClothConstFacade* ClothFacade, const FDataflowToolNodeSnapshot& Snapshot, TArray<float>& OutValues) const;
	void GetWeightsFromActiveSnapshot(UE::Chaos::ClothAsset::FCollectionClothConstFacade* ClothFacade, TArray<float>& OutValues) const;

	/**
	* store weights values in a specific snapshot 
	* if the cloth facade is available, the topology information will be also stored in the snapshot
	*/
	void StoreWeightsInSnapshot(UE::Chaos::ClothAsset::FCollectionClothConstFacade* ClothFacade, const TArray<float>& InValues, FDataflowToolNodeSnapshot& OutSnapshot) const;

	// This property should be technically deprecated but we need to keep it because we cannot post serialize fix it as we need a dataflow Context to create a snapshot 
	// deprecated methods are still using it and we still use it to load the data when there's no pre-existing snapshot 
	UPROPERTY()
	TArray<float> VertexWeights;

	UE_DEPRECATED(5.5, "Use OutputName instead.")
	UPROPERTY()
	FString Name;  // TODO: Discard in future v2. but keep for backward compatibility here as some weight maps have been created with it
};
