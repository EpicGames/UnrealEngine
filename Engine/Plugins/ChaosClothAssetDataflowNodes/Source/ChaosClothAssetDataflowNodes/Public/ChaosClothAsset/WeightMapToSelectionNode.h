// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Dataflow/DataflowNode.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "WeightMapToSelectionNode.generated.h"

namespace UE_DEPRECATED(5.5, "Use UE::Dataflow instead.") Dataflow {}

/** What type of element to convert to */
UENUM()
enum class EChaosClothAssetWeightMapConvertableSelectionType : uint8
{
	/** 2D simulation vertices */
	SimVertices2D,

	/** 3D simulation vertices */
	SimVertices3D,

	/** Simulation faces (2D/3D are the same) */
	SimFaces,
};

/** Convert a vertex weight map to an integer selection set. */
USTRUCT(Meta = (DataflowCloth))
struct FChaosClothAssetWeightMapToSelectionNode : public FDataflowNode
{
	GENERATED_USTRUCT_BODY()
	DATAFLOW_NODE_DEFINE_INTERNAL(FChaosClothAssetWeightMapToSelectionNode, "WeightMapToSelection", "Cloth", "Cloth Weight Map To Selection")

public:

	UPROPERTY(Meta = (Dataflowinput, DataflowOutput, DataflowPassthrough = "Collection"))
	FManagedArrayCollection Collection;

	/** The name of the Weight Map to convert. */
	UPROPERTY(Meta = (DataflowInput))
	FString WeightMapName;

	/**
	 * The name of the select attribute that will be added to the collection.
	 * If left empty the same name as the Weight Map  name will be used instead.
	 */
	UPROPERTY(EditAnywhere, Category = "Weight Map To Selection", Meta = (DataflowOutput, DataflowSelection))
	FString SelectionName;

	/** The type of element the selection refers to */
	UPROPERTY(EditAnywhere, Category = "Weight Map To Selection")
	EChaosClothAssetWeightMapConvertableSelectionType SelectionType = EChaosClothAssetWeightMapConvertableSelectionType::SimVertices3D;

	/** Map values above this will be selected. */
	UPROPERTY(EditAnywhere, Category = "Weight Map To Selection", Meta = (ClampMin = "0", ClampMax = "1"))
	float SelectionThreshold = 0.95f;

	FChaosClothAssetWeightMapToSelectionNode(const UE::Dataflow::FNodeParameters& InParam, FGuid InGuid = FGuid::NewGuid());
private:

	virtual void Evaluate(UE::Dataflow::FContext& Context, const FDataflowOutput* Out) const override;
};
