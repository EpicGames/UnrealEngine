// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"
#include "Containers/UnrealString.h"
#include "WeightedValue.generated.h"

namespace UE::Chaos::ClothAsset
{
	struct FWeightMapTools
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.5, "Override properties are no longer used.")
		inline static const FString NotOverridden = TEXT("#");
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		/**
		 * Modifies a string to make it suitable as a weight map name.
		 * Replaces any deemed special characters and spaces by underscores, and remove leading and ending underscores from the name.
		 * Note that leading underscores are reserved for non user/internal attribute names.
		 */
		static CHAOSCLOTHASSETDATAFLOWNODES_API void MakeWeightMapName(FString& InOutString);
	};
	
	/** Weight value bounds that could be used to build a cloth asset weighted value */
	struct FWeightedValueBounds
	{
		float Low;
		float High;
	};
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT()
struct FChaosClothAssetWeightedValue
{
	GENERATED_BODY()
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Whether the property can ever be updated/animated in real time.
	 * This could make the simulation takes more CPU time, even more so if the weight maps needs updating.
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value")
	bool bIsAnimatable = true;

	/**
	 * Property value corresponding to the lower bound of the Weight Map.
	 * A Weight Map stores a series of Weight values assigned to each point, all between 0 and 1.
	 * The weights are used to interpolate the individual values from Low to High assigned to each different point.
	 * A Weight of 0 always corresponds to the Low parameter value, and a Weight of 1 to the High parameter value.
	 * The value for Low can be set to be bigger than for High in order to reverse the effect of the Weight Map.
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (DisplayName = "Low Weight", ChaosClothAssetShortName = "Lo", EditCondition = "!bImportFabricBounds"))
	mutable float Low = 0.f;

	/**
	 * Property value corresponding to the upper bound of the Weight Map.
	 * A Weight Map stores a series of Weight values assigned to each point, all between 0 and 1.
	 * The weights are used to interpolate the individual values from Low to High assigned to each different point.
	 * A Weight of 0 always corresponds to the Low parameter value, and a Weight of 1 to the High parameter value.
	 * The value for Low can be set to be bigger than for High in order to reverse the effect of the Weight Map.
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (DisplayName = "High Weight", ChaosClothAssetShortName = "Hi", EditCondition = "!bImportFabricBounds"))
	mutable float High = 1.f;

	/** The name of the weight map for this property. */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (DataflowInput))
	mutable FString WeightMap = TEXT("WeightMap");  // Mutable so that it can be name checked in the evaluate function

	/**
	 * Whether the property could import fabrics datas or not
	 */
	UPROPERTY(VisibleAnywhere, Transient, Category = "Weighted Value", Meta = (EditCondition = "false", EditConditionHides))
	bool bCouldUseFabrics = false;

	UE_DEPRECATED(5.5, "Override properties are no longer used.")
	UPROPERTY(Transient)
	mutable FString WeightMap_Override;
	
	/**
	 * Whether the property can use the fabric bounds (low/high) values imported from USD
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (EditCondition = "bCouldUseFabrics", EditConditionHides))
	bool bImportFabricBounds = false;

	/**
	 * Whether the property can override the weight map based on the imported fabrics
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (EditCondition = "bCouldUseFabrics", EditConditionHides))
	bool bBuildFabricMaps = false;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT()
struct FChaosClothAssetWeightedValueNonAnimatable
{
	GENERATED_BODY()
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	static constexpr bool bIsAnimatable = false;

	/**
	 * Property value corresponding to the lower bound of the Weight Map.
	 * A Weight Map stores a series of Weight values assigned to each point, all between 0 and 1.
	 * The weights are used to interpolate the individual values from Low to High assigned to each different point.
	 * A Weight of 0 always corresponds to the Low parameter value, and a Weight of 1 to the High parameter value.
	 * The value for Low can be set to be bigger than for High in order to reverse the effect of the Weight Map.
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (DisplayName = "Low Weight", ChaosClothAssetShortName = "Lo", EditCondition = "!bImportFabricBounds"))
	mutable float Low = 0.f;

	/**
	 * Property value corresponding to the upper bound of the Weight Map.
	 * A Weight Map stores a series of Weight values assigned to each point, all between 0 and 1.
	 * The weights are used to interpolate the individual values from Low to High assigned to each different point.
	 * A Weight of 0 always corresponds to the Low parameter value, and a Weight of 1 to the High parameter value.
	 * The value for Low can be set to be bigger than for High in order to reverse the effect of the Weight Map.
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (DisplayName = "High Weight", ChaosClothAssetShortName = "Hi", EditCondition = "!bImportFabricBounds"))
	mutable float High = 1.f;
	
	/** The name of the weight map for this property. */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (DataflowInput))
	mutable FString WeightMap = TEXT("WeightMap");  // Mutable so that it can be name checked in the evaluate function

	/**
	 * Whether the property could import fabrics datas or not
	 */
	UPROPERTY(VisibleAnywhere, Transient, Category = "Weighted Value", Meta = (EditCondition = "false", EditConditionHides))
	bool bCouldUseFabrics = false;
	
	UE_DEPRECATED(5.5, "Override properties are no longer used.")
	UPROPERTY(Transient)
	mutable FString WeightMap_Override;

	/**
	 * Whether the property can use the fabric bounds (low/high) values imported from USD
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (EditCondition = "bCouldUseFabrics", EditConditionHides))
	bool bImportFabricBounds = false;

	/**
	 * Whether the property can override the weight map based on the imported fabrics
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (EditCondition = "bCouldUseFabrics", EditConditionHides))
	bool bBuildFabricMaps = false;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
USTRUCT()
struct FChaosClothAssetWeightedValueNonAnimatableNoLowHighRange
{
	GENERATED_BODY()
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** The name of the weight map for this property. */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (DataflowInput))
	FString WeightMap = TEXT("WeightMap");

	UE_DEPRECATED(5.5, "Override properties are no longer used.")
	UPROPERTY(Transient)
	mutable FString WeightMap_Override;
};

USTRUCT()
struct FChaosClothAssetWeightedValueOverride
{
	GENERATED_BODY()

	/**
	 * Low value override.
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (DisplayName = "Low Weight", ChaosClothAssetShortName = "Lo"))
	float Low = 1.f;

	/**
	 * High value override
	 */
	UPROPERTY(EditAnywhere, Category = "Weighted Value", Meta = (DisplayName = "High Weight", ChaosClothAssetShortName = "Hi"))
	float High = 1.f;
};
